{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Camera Calibration",
	"Description": "Framework to support lens distortion and camera calibration in engine.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "CameraCalibrationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TrackingAlignment",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TrackingAlignmentEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "LensComponent",
			"Enabled": true
		},
		{
			"Name": "MediaFrameworkUtilities",
			"Enabled": true
		},
		{
			"Name": "OpenCVLensDistortion",
			"Enabled": true
		}
	]
}