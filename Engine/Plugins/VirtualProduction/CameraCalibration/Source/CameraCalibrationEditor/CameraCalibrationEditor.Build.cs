// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CameraCalibrationEditor : ModuleRules
	{
		public CameraCalibrationEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			// Enable C++ exceptions for try-catch blocks in calibration solver because it relies on OpenCV and it can produce them when calibrations fail.
			bEnableExceptions = true;
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"AppFramework",
					"AssetDefinition",
					"AssetRegistry",
					"AssetTools",
					"CameraCalibrationCore",
					"CinematicCamera",
					"ContentBrowserData",
					"Core",
					"CoreUObject",
	                "CurveEditor",
					"DesktopPlatform",
					"DeveloperSettings",
					"EditorStyle",
					"EditorWidgets",
					"Engine",
					"RenderCore",
					"ImageCore",
					"InputCore",
					"Json",
					"JsonUtilities",
					"LensComponent",
					"MediaAssets",
					"MediaProfile",
					"MediaFrameworkUtilities",
					"OpenCV",
					"OpenCVHelper",
					"PlacementMode",
					"PropertyEditor",
					"SequencerWidgets",
					"Settings",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd",
					"WorkspaceMenuStructure",
				});
		}
	}
}
