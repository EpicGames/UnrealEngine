// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_LensFile.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "AssetEditor/CameraCalibrationToolkit.h"
#include "LensFile.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_LensFile"

TSoftClassPtr<UObject> UAssetDefinition_LensFile::GetAssetClass() const
{
	return ULensFile::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_LensFile::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(LOCTEXT("LensFileAssetCategory", "Virtual Production")) };
	return Categories;
}

FText UAssetDefinition_LensFile::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Lens File");
}

FLinearColor UAssetDefinition_LensFile::GetAssetColor() const
{
	return FLinearColor(0.1f, 1.0f, 0.1f);
}

EAssetCommandResult UAssetDefinition_LensFile::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const TArray<ULensFile*> ObjectsToOpen = OpenArgs.LoadObjects<ULensFile>();
	for (ULensFile* Object : ObjectsToOpen)
	{
		FCameraCalibrationToolkit::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
