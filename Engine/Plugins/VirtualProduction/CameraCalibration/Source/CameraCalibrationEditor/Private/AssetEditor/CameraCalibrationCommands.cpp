// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"



#define LOCTEXT_NAMESPACE "CameraCalibrationCommands"

void FCameraCalibrationCommands::RegisterCommands()
{
	UI_COMMAND(Edit, "Edit", "Edit the current lens file.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowMediaPlaybackControls, "Show Media Playback Controls", "Show the media playback control buttons", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE