// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

#include "UI/CameraCalibrationEditorStyle.h"


class FUICommandInfo;


class FCameraCalibrationCommands : public TCommands<FCameraCalibrationCommands>
{
public:

	FCameraCalibrationCommands()
		: TCommands<FCameraCalibrationCommands>(TEXT("CameraCalibration"), NSLOCTEXT("CameraCalibrationCommands", "CameraCalibrationCommandsEdit", "CameraCalibration"), NAME_None, FCameraCalibrationEditorStyle::Get().GetStyleSetName())
	{}
	
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

	/** Edit the current lens distortion. */
	TSharedPtr<FUICommandInfo> Edit;

	/** Show the media playback control buttons. */
	TSharedPtr<FUICommandInfo> ShowMediaPlaybackControls;

};
