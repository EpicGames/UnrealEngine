// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationCurveEditor.h"


FCameraCalibrationCurveEditor::FCameraCalibrationCurveEditor()
	: FCurveEditor()
{
	//Disable zoom axis snapping behavior. 
	InputSnapEnabledAttribute = false;
}