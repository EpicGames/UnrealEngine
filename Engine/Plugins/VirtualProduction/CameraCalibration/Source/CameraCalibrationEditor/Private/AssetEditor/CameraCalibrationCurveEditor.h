// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"

/**
 * Custom Curve editor child class
 */
class FCameraCalibrationCurveEditor : public FCurveEditor
{
public:
	FCameraCalibrationCurveEditor();
	
public:
	/** Delegate instance when new Data Point added */
	FSimpleDelegate OnAddDataPointDelegate;
};

