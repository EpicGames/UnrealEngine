// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/Texture.h"
#include "LensData.h"

#include "CameraCalibrationEditorCommon.generated.h"


/** Container of distortion data to use instanced customization to show parameter struct defined by the model */
USTRUCT()
struct FDistortionInfoContainer
{
	GENERATED_BODY()

public:

	/** Distortion parameters */
	UPROPERTY(EditAnywhere, Category = "Distortion", meta = (ShowOnlyInnerProperties))
	FDistortionInfo DistortionInfo;
};

/** Holds the last FIZ data that was evaluated */
struct FCachedFIZData
{
	TOptional<float> RawFocus;
	TOptional<float> RawIris;
	TOptional<float> RawZoom;

	TOptional<float> EvaluatedFocus;
	TOptional<float> EvaluatedIris;
	TOptional<float> EvaluatedZoom;
};

