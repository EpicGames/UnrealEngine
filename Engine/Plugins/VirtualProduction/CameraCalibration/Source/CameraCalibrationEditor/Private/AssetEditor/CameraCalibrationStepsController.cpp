// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationStepsController.h"

#include "Camera/CameraActor.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationStep.h"
#include "CameraCalibrationSubsystem.h"
#include "CameraCalibrationToolkit.h"
#include "CameraCalibrationTypes.h"
#include "CineCameraActor.h"
#include "ContentBrowserDataSubsystem.h"
#include "Containers/Ticker.h"
#include "Editor.h"
#include "Engine/UserDefinedEnum.h"
#include "EngineUtils.h"
#include "IContentBrowserDataModule.h"
#include "Input/Events.h"
#include "Engine/Canvas.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "LensComponent.h"
#include "LensFile.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Misc/MessageDialog.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "SCameraCalibrationSteps.h"
#include "TextureResource.h"
#include "TimeSynchronizableMediaSource.h"
#include "UnrealClient.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/SWidget.h"


#define LOCTEXT_NAMESPACE "CameraCalibrationStepsController"

namespace CameraCalibrationStepsController
{
	/** Gets all assets of type TObject from the content browser */
	template<typename TObject>
	TArray<TSoftObjectPtr<TObject>> GetUObjectAssets()
	{
		TArray<TSoftObjectPtr<TObject>> Assets;
	
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		FContentBrowserDataFilter Filter;
		Filter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFiles;
		Filter.ItemCategoryFilter = EContentBrowserItemCategoryFilter::IncludeAssets | EContentBrowserItemCategoryFilter::IncludeClasses | EContentBrowserItemCategoryFilter::IncludeCollections;
		Filter.ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeProject;
		Filter.bRecursivePaths = true;
	
		FContentBrowserDataClassFilter& ClassFilter = Filter.ExtraFilters.FindOrAddFilter<FContentBrowserDataClassFilter>();
		ClassFilter.ClassNamesToInclude.Add(TObject::StaticClass()->GetClassPathName().ToString());
		ClassFilter.bRecursiveClassNamesToInclude = true;
		ClassFilter.bRecursiveClassNamesToExclude = false;

		ContentBrowserData->EnumerateItemsUnderPath(TEXT("/"), Filter, [&Assets](FContentBrowserItemData&& InItemData)
		{
			TSoftObjectPtr<TObject> MediaSourcePtr(FSoftObjectPath(InItemData.GetInternalPath().ToString()));

			if (MediaSourcePtr.IsValid() || MediaSourcePtr.IsPending())
			{
				Assets.Add(MediaSourcePtr);
			}
		
			return true;
		});

		return Assets;
	}
}


FCameraCalibrationStepsController::FCameraCalibrationStepsController(TWeakPtr<FCameraCalibrationToolkit> InCameraCalibrationToolkit, ULensFile* InLensFile)
	: CameraCalibrationToolkit(InCameraCalibrationToolkit)
	, LensFile(TWeakObjectPtr<ULensFile>(InLensFile))
{
	check(CameraCalibrationToolkit.IsValid());
	check(LensFile.IsValid());

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateRaw(this, &FCameraCalibrationStepsController::OnTick), 0.0f);
}

FCameraCalibrationStepsController::~FCameraCalibrationStepsController()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}

	Cleanup();
}

void FCameraCalibrationStepsController::CreateSteps()
{
	// Ask subsystem for the registered calibration steps.

	const UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();

	if (!Subsystem)
	{
		UE_LOGF(LogCameraCalibrationEditor, Error, "Could not find UCameraCalibrationSubsystem");
		return;
	}

	const TArray<FName> StepNames = Subsystem->GetCameraCalibrationSteps();

	// Create the steps

	for (const FName& StepName : StepNames)
	{
		const TSubclassOf<UCameraCalibrationStep> StepClass = Subsystem->GetCameraCalibrationStep(StepName);

		UCameraCalibrationStep* const Step = NewObject<UCameraCalibrationStep>(
			GetTransientPackage(),
			StepClass,
			MakeUniqueObjectName(GetTransientPackage(), StepClass),
			RF_Transactional);

		check(Step);

		Step->Initialize(SharedThis(this));

		CalibrationSteps.Add(TStrongObjectPtr<UCameraCalibrationStep>(Step));
	}

	// Sort them according to prerequisites
	//
	// We iterate from the left and move to the right-most existing prerequisite, leaving a null behind
	// At the end we remove all the nulls that were left behind.

	for (int32 StepIdx = 0; StepIdx < CalibrationSteps.Num(); ++StepIdx)
	{
		int32 InsertIdx = StepIdx;

		for (int32 PrereqIdx = StepIdx+1; PrereqIdx < CalibrationSteps.Num(); ++PrereqIdx)
		{
			if (CalibrationSteps[StepIdx]->DependsOnStep(CalibrationSteps[PrereqIdx].Get()))
			{
				InsertIdx = PrereqIdx + 1;
			}
		}

		if (InsertIdx != StepIdx)
		{
			const TStrongObjectPtr<UCameraCalibrationStep> DependentStep = CalibrationSteps[StepIdx];
			CalibrationSteps.Insert(DependentStep, InsertIdx);

			// Invalidate the pointer left behind. This entry will be removed a bit later.
			CalibrationSteps[StepIdx].Reset();
		}
	}

	// Remove the nulled out ones

	for (int32 StepIdx = 0; StepIdx < CalibrationSteps.Num(); ++StepIdx)
	{
		if (!CalibrationSteps[StepIdx].IsValid())
		{
			CalibrationSteps.RemoveAt(StepIdx);
			StepIdx--;
		}
	}
}

TSharedPtr<SWidget> FCameraCalibrationStepsController::BuildUI()
{
	return SNew(SCameraCalibrationSteps, SharedThis(this));
}

bool FCameraCalibrationStepsController::OnTick(float DeltaTime)
{
	// Update the lens file evaluation inputs
	LensFileEvaluationInputs.bIsValid = false;
	if (const ULensComponent* const LensComponent = FindLensComponent())
	{
		LensFileEvaluationInputs = LensComponent->GetLensFileEvaluationInputs();
	}

	const FIntPoint MediaOverlayResolution = GetMediaOverlayResolution();
	bool bMediaOverlayResolutionChanged = false;
	if (MediaOverlayResolution != LastMediaOverlayResolution)
	{
		LastMediaOverlayResolution = MediaOverlayResolution;
		bMediaOverlayResolutionChanged = true;
	}
	
	// Update the LensFile SimulcamInfo if the Media Dimensions have changed
	if (ULensFile* const LensFilePtr = GetLensFile())
	{
		const FIntPoint CameraFeedResolution = LensFilePtr->CameraFeedInfo.GetDimensions();

		// Camera feed is not allowed to be larger than media resolution, so if this is detected, reset the camera feed dimensions
		const bool bIsCameraFeedLargerThanMedia = CameraFeedResolution.X > MediaOverlayResolution.X || CameraFeedResolution.Y > MediaOverlayResolution.Y;
		if (bIsCameraFeedLargerThanMedia || bMediaOverlayResolutionChanged)
		{
			ResetCameraFeedDimensions();
		}
		
		LensFilePtr->SimulcamInfo.MediaResolution = MediaOverlayResolution;
		LensFilePtr->SimulcamInfo.MediaPlateAspectRatio = (MediaOverlayResolution.Y != 0) ? (float)MediaOverlayResolution.X / (float)MediaOverlayResolution.Y : 0.0f;
		
		if (CineCameraComponent.IsValid())
		{
			const float FilmbackAspectRatio = CineCameraComponent.Pin()->AspectRatio;
			LensFilePtr->SimulcamInfo.CGLayerAspectRatio = FilmbackAspectRatio;
		}
	}

	// Tick each of the calibration steps
	for (TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationSteps)
	{
		if (Step.IsValid())
		{
			Step->Tick(DeltaTime);
		}
	}

	return true;
}

void FCameraCalibrationStepsController::SetCameraFeedDimensionsFromMousePosition(FVector2D MousePosition)
{
	// Compute the size of the camera feed, using the MousePosition as one of its corners.
	// We assume that the camera feed will always be centered in the media. If that assumption proves false, this math should be updated in the future.
	const FIntPoint MediaOverlayResolution = GetMediaOverlayResolution();

	FIntPoint CameraFeedDimensions = FIntPoint(0, 0);
	CameraFeedDimensions.X = FMath::Abs((MediaOverlayResolution.X / 2) - FMath::Floor(MousePosition.X)) * 2;
	CameraFeedDimensions.Y = FMath::Abs((MediaOverlayResolution.Y / 2) - FMath::Floor(MousePosition.Y)) * 2;

	// Early-out if the dimensions are invalid
	if ((CameraFeedDimensions.X == 0) || (CameraFeedDimensions.Y == 0))
	{
		return;
	}

	if (ULensFile* const LensFilePtr = GetLensFile())
	{
		// Compare the aspect ratio of the selected camera to the feed aspect ratio to determine if we should attempt to minimize the aspect ratio difference
		if (CineCameraComponent.IsValid())
		{
			const float CameraFeedAspectRatio = CameraFeedDimensions.X / (float)CameraFeedDimensions.Y;
			const float CameraAspectRatio = CineCameraComponent->AspectRatio;

			// If the two aspect ratios are within the acceptable tolerance, attempt to minimize the aspect ratio difference by adjusting the camera feed dimensions
			constexpr float AspectRatioNudgeTolerance = 0.1f;
			if (FMath::IsNearlyEqual(CameraAspectRatio, CameraFeedAspectRatio, AspectRatioNudgeTolerance))
			{
				MinimizeAspectRatioError(CameraFeedDimensions, CameraAspectRatio);
			}

			// Update the camera feed dimensions and mark as overridden because it was changed as the result of user interaction
			constexpr bool bMarkAsOverridden = true;
			LensFilePtr->CameraFeedInfo.SetDimensions(CameraFeedDimensions, bMarkAsOverridden);
		}
	}
}

void FCameraCalibrationStepsController::SetCameraFeedDimensions(FIntPoint Dimensions, bool bMarkAsOverridden)
{
	if (ULensFile* const LensFilePtr = GetLensFile())
	{
		LensFilePtr->CameraFeedInfo.SetDimensions(Dimensions, bMarkAsOverridden);
	}
}

void FCameraCalibrationStepsController::ResetCameraFeedDimensions()
{
	TStrongObjectPtr<ULensFile> PinnedLensFile = LensFile.Pin();
	if (!PinnedLensFile.IsValid())
	{
		return;
	}
	
	float CameraAspectRatio = 0.0f;
	if (CineCameraComponent.IsValid())
	{
		CameraAspectRatio = CineCameraComponent->AspectRatio;
	}
	else if (Camera.IsValid())
	{
		CameraAspectRatio = Camera->GetCameraComponent()->AspectRatio;
	}

	if (FMath::IsNearlyZero(CameraAspectRatio))
	{
		return;
	}

	// Update the camera feed dimensions to match with the media resolution, corrected for the camera's aspect ratio
	const FIntPoint MediaResolution = GetMediaOverlayResolution();
	const float MediaAspectRatio = MediaResolution.Y != 0 ? MediaResolution.X / (float)MediaResolution.Y : 0.0f;

	FIntPoint CameraFeedDimensions = MediaResolution;
	if (CameraAspectRatio > MediaAspectRatio)
	{
		CameraFeedDimensions.Y = CameraFeedDimensions.X / CameraAspectRatio;
	}
	else
	{
		CameraFeedDimensions.X = CameraFeedDimensions.Y * CameraAspectRatio;
	}
	
	PinnedLensFile->SimulcamInfo.CGLayerAspectRatio = CameraAspectRatio;
	
	constexpr bool bMarkAsOverridden = false;
	PinnedLensFile->CameraFeedInfo.SetDimensions(CameraFeedDimensions, bMarkAsOverridden);
}

void FCameraCalibrationStepsController::MinimizeAspectRatioError(FIntPoint& CameraFeedDimensions, float CameraAspectRatio)
{
	float CameraFeedAspectRatio = (CameraFeedDimensions.Y > 0) ? CameraFeedDimensions.X / (float)CameraFeedDimensions.Y : 0.0f;

	// Track the number of iterations to ensure that the minimization does not continue infinitely
	int32 NumIterations = 0;
	constexpr int32 MaxIterations = 20;

	constexpr float AspectRatioErrorTolerance = 0.001f;
	while ((!FMath::IsNearlyEqual(CameraAspectRatio, CameraFeedAspectRatio, AspectRatioErrorTolerance)) && NumIterations < MaxIterations)
	{
		// The camera feed needs to be wider and/or shorter
		if (CameraFeedAspectRatio < CameraAspectRatio)
		{
			// Use modulus to determine whether to alter the width or height each iteration
			if (NumIterations % 2 == 0)
			{
				CameraFeedDimensions.X += 2;
			}
			else
			{
				CameraFeedDimensions.Y -= 2;
			}
		}
		else // The camera feed needs to be narrower or taller
		{
			// Use modulus to determine whether to change the width or height each iteration
			if (NumIterations % 2 == 0)
			{
				CameraFeedDimensions.X -= 2;
			}
			else
			{
				CameraFeedDimensions.Y += 2;
			}
		}

		CameraFeedAspectRatio = (CameraFeedDimensions.Y > 0) ? CameraFeedDimensions.X / (float)CameraFeedDimensions.Y : 0.0f;

		NumIterations++;
	}
}

void FCameraCalibrationStepsController::Cleanup()
{
	for (TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationSteps)
	{
		if (Step.IsValid())
		{
			Step->Shutdown();
			Step.Reset();
		}
	}

	ClearMedia();

	if (InternalMediaPlayer.IsValid())
	{
		InternalMediaPlayer->Close();
		InternalMediaPlayer.Reset();
	}
	
	Camera.Reset();
	
	IMediaProfileManager::Get().OnMediaProfileChanged().RemoveAll(this);
}

UWorld* FCameraCalibrationStepsController::GetWorld() const
{
	return GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
}

FIntPoint FCameraCalibrationStepsController::GetCameraFeedSize() const
{
	if (ULensFile* LensFilePtr = GetLensFile())
	{
		return LensFilePtr->CameraFeedInfo.GetDimensions();
	}

	return FIntPoint(0, 0);
}

void FCameraCalibrationStepsController::InitializeMediaPlayer()
{
	// Create media player and media texture and assign it to the media input of the media plate layer.
	// Create MediaPlayer

	// Using a strong reference prevents the MediaPlayer from going stale when the level is saved.
	InternalMediaPlayer = TStrongObjectPtr<UMediaPlayer>(NewObject<UMediaPlayer>(
		GetTransientPackage(),
		MakeUniqueObjectName(GetTransientPackage(), UMediaPlayer::StaticClass())
		));

	if (!InternalMediaPlayer.IsValid())
	{
		UE_LOGF(LogCameraCalibrationEditor, Warning, "Failed to create MediaPlayer");
		Cleanup();
		return;
	}

	InternalMediaPlayer->PlayOnOpen = true;
	InternalMediaPlayer->SetLooping(true);
	
	// Create MediaTexture
	MediaTexture = NewObject<UMediaTexture>(GetTransientPackage(), NAME_None, RF_Transient);

	if (MediaTexture.IsValid())
	{
		MediaTexture->AutoClear = true;
		MediaTexture->SetMediaPlayer(InternalMediaPlayer.Get());
		MediaTexture->UpdateResource();
	}

	// Play the media source, preferring time-synchronizable sources.
	if (const UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
	{
		bool bFoundPreferredMediaSource = false;

		for (int32 MediaSourceIdx = 0; MediaSourceIdx < MediaProfile->NumMediaSources(); ++MediaSourceIdx)
		{
			UMediaSource* MediaSource = MediaProfile->GetMediaSource(MediaSourceIdx);

			if (Cast<UTimeSynchronizableMediaSource>(MediaSource))
			{
				MediaProfileSourceIndex = MediaSourceIdx;
				ExternalMediaTexture = MediaProfile->GetPlaybackManager()->OpenSourceFromIndex(MediaProfileSourceIndex, this);
				bFoundPreferredMediaSource = true;

				// Break since we don't need to look for more MediaSources.
				break;
			}
		}

		if (!bFoundPreferredMediaSource)
		{
			for (int32 MediaSourceIdx = 0; MediaSourceIdx < MediaProfile->NumMediaSources(); ++MediaSourceIdx)
			{
				if (UMediaSource* MediaSource = MediaProfile->GetMediaSource(MediaSourceIdx))
				{
					MediaProfileSourceIndex = MediaSourceIdx;
					ExternalMediaTexture = MediaProfile->GetPlaybackManager()->OpenSourceFromIndex(MediaProfileSourceIndex, this);

					// Break since we don't need to look for more MediaSources.
					break;
				}
			}
		}
	}

	const FIntPoint DefaultResolution(1920, 1080);
	if (ULensFile* LensFilePtr = LensFile.Get())
	{
		// Initialize the simulcam info of the LensFile to match the initial render resolution of the comp
		const float AspectRatio = (DefaultResolution.Y != 0) ? DefaultResolution.X / (float)DefaultResolution.Y : 1.0f;
		LensFilePtr->SimulcamInfo.CGLayerAspectRatio = AspectRatio;
		LensFilePtr->SimulcamInfo.MediaPlateAspectRatio = AspectRatio;

		// If the Camera Feed is not specifically overriden by the user, initialize it to the initial dimensions of the comp as well
		// If the camera feed exactly matches the comp's resolution, then no aspect ratio correction is needed
		if (!LensFilePtr->CameraFeedInfo.IsOverridden())
		{
			LensFilePtr->CameraFeedInfo.SetDimensions(DefaultResolution);
		}
	}

	// By default, use a camera with our lens. If not found, use the first camera found in the level
	if (ACameraActor* CameraGuess = FindFirstCameraWithCurrentLens())
	{
		SetCamera(CameraGuess);
	}
	else
	{
		SetCamera(FindDefaultCamera());
	}

	// Create a render target that can be used to extract media texture pixels
	CachedMediaRenderTarget = TStrongObjectPtr<UTextureRenderTarget2D>(NewObject<UTextureRenderTarget2D>(
		GetTransientPackage(),
		MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass())));
	
	CachedMediaRenderTarget->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
	CachedMediaRenderTarget->ClearColor = FLinearColor::Black;
	CachedMediaRenderTarget->bAutoGenerateMips = false;
	CachedMediaRenderTarget->InitAutoFormat(DefaultResolution.X, DefaultResolution.Y);
	CachedMediaRenderTarget->UpdateResourceImmediate(true);
}

UMaterialInterface* FCameraCalibrationStepsController::GetOverlayMaterial(EOverlayPassType OverlayPassType) const
{
	if (OverlayPassType == EOverlayPassType::ToolOverlay)
	{
		return ToolOverlayMaterial.Get();
	}
	else if (OverlayPassType == EOverlayPassType::UserOverlay)
	{
		return UserOverlayMaterial.Get();
	}

	return nullptr;
}

bool FCameraCalibrationStepsController::IsOverlayEnabled(EOverlayPassType OverlayPassType) const
{
	switch (OverlayPassType)
	{
	case EOverlayPassType::ToolOverlay:
		return bToolOverlayEnabled;

	case EOverlayPassType::UserOverlay:
		return bUserOverlayEnabled;
	}
	
	return false;
}

void FCameraCalibrationStepsController::SetOverlayEnabled(const bool bEnabled, EOverlayPassType OverlayPassType)
{
	switch (OverlayPassType)
	{
	case EOverlayPassType::ToolOverlay:
		bToolOverlayEnabled = bEnabled;
		break;

	case EOverlayPassType::UserOverlay:
		bUserOverlayEnabled = bEnabled;
		break;
	}
}

void FCameraCalibrationStepsController::SetOverlayMaterial(UMaterialInterface* InOverlay, bool bShowOverlay, EOverlayPassType OverlayPassType)
{
 	if (OverlayPassType == EOverlayPassType::ToolOverlay)
 	{
		ToolOverlayMaterial = InOverlay;
	}
 	else if (OverlayPassType == EOverlayPassType::UserOverlay)
 	{
		UserOverlayMaterial = InOverlay;
	}

	// If the overlay is non-null, refresh the overlay and enable/disable the overlay as necessary
	if (InOverlay)
	{
		SetOverlayEnabled(bShowOverlay, OverlayPassType);
	}
	else
	{
		// Disable the overlay pass if the input overlay is null
		SetOverlayEnabled(false, OverlayPassType);
	}
}

float FCameraCalibrationStepsController::GetWiperWeight() const
{
	return WiperWeight;
}

void FCameraCalibrationStepsController::SetWiperWeight(float InWeight)
{
	WiperWeight = InWeight;
}

void FCameraCalibrationStepsController::SetCamera(ACameraActor* InCamera)
{
	Camera = InCamera;

	if (!InCamera)
	{
		return;
	}

	TInlineComponentArray<UCineCameraComponent*> CameraComponents;
	InCamera->GetComponents(CameraComponents);

	if (CameraComponents.Num() > 0)
	{
		CineCameraComponent = CameraComponents[0];
	}
}

ACameraActor* FCameraCalibrationStepsController::GetCamera() const
{
	return Camera.Get();
}

float FCameraCalibrationStepsController::GetCameraFeedAspectRatio() const
{
	TStrongObjectPtr<ULensFile> PinnedLensFile = LensFile.Pin();
	if (!PinnedLensFile)
	{
		return 1.0f;
	}

	if (PinnedLensFile->CameraFeedInfo.IsValid())
	{
		const FVector2D CameraFeedDimensions(PinnedLensFile->CameraFeedInfo.GetDimensions());
		return CameraFeedDimensions.X / CameraFeedDimensions.Y;
	}

	return 1.0f;
}

void FCameraCalibrationStepsController::OnSimulcamViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bool bStepHandled = false;

	for (TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationSteps)
	{
		if (Step.IsValid() && Step->IsActive())
		{
			bStepHandled |= Step->OnViewportClicked(MyGeometry, MouseEvent);
			break;
		}
	}

	// If a step handled the event, we're done
	if (bStepHandled)
	{
		return;
	}

	// Toggle video pause with right click.
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TogglePlay();
		return;
	}
}

bool FCameraCalibrationStepsController::OnSimulcamViewportInputKey(const FKey& InKey, const EInputEvent& InEvent)
{
	bool bStepHandled = false;

	for (TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationSteps)
	{
		if (Step.IsValid() && Step->IsActive())
		{
			bStepHandled |= Step->OnViewportInputKey(InKey, InEvent);
			break;
		}
	}

	return bStepHandled;
}

void FCameraCalibrationStepsController::OnSimulcamViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition)
{
	for (TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationSteps)
	{
		if (Step.IsValid() && Step->IsActive())
		{
			Step->OnViewportMarqueeSelect(StartPosition, EndPosition);
			break;
		}
	}
}

FReply FCameraCalibrationStepsController::OnRewindButtonClicked()
{
	// Rewind to the beginning of the media
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->Rewind();
	}

	return FReply::Handled();
}

FReply FCameraCalibrationStepsController::OnReverseButtonClicked()
{
	// Increase the reverse media playback rate
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->SetRate(GetFasterReverseRate());
	}

	return FReply::Handled();
}

FReply FCameraCalibrationStepsController::OnStepBackButtonClicked() 
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float DefaultStepRateInMilliseconds = GetDefault<UCameraCalibrationEditorSettings>()->DefaultMediaStepRateInMilliseconds;
		const bool bForceDefaultStepRate = GetDefault<UCameraCalibrationEditorSettings>()->bForceDefaultMediaStepRate;

		// The media player could return a frame rate of 0 for the current video track
		const float MediaFrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

		// Use the default step rate if the media player returned an invalid frame rate or if the project settings force it
		float MillisecondsPerStep = 0.0f;
		if (FMath::IsNearlyEqual(MediaFrameRate, 0.0f) || bForceDefaultStepRate)
		{
			MillisecondsPerStep = DefaultStepRateInMilliseconds;
		}
		else
		{
			MillisecondsPerStep = 1000.0f / MediaFrameRate;
		}

		// Compute the number of ticks in one step and go backward from the media's current time (clamping to 0)
		const FTimespan TicksInOneStep = ETimespan::TicksPerMillisecond * (MillisecondsPerStep);
		FTimespan PreviousStepTime = MediaPlayer->GetTime() - TicksInOneStep;
		if (PreviousStepTime < FTimespan::Zero())
		{
			PreviousStepTime = FTimespan::Zero();
		}

		MediaPlayer->Seek(PreviousStepTime);
		MediaPlayer->Pause();
	}

	return FReply::Handled();
}

FReply FCameraCalibrationStepsController::OnPlayButtonClicked() 
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->Play();
	}

	return FReply::Handled(); 
}

FReply FCameraCalibrationStepsController::OnPauseButtonClicked() 
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->Pause();
	}

	return FReply::Handled(); 
}

FReply FCameraCalibrationStepsController::OnStepForwardButtonClicked()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		const float DefaultStepRateInMilliseconds = GetDefault<UCameraCalibrationEditorSettings>()->DefaultMediaStepRateInMilliseconds;
		const bool bForceDefaultStepRate = GetDefault<UCameraCalibrationEditorSettings>()->bForceDefaultMediaStepRate;

		// The media player could return a frame rate of 0 for the current video track
		const float MediaFrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);

		// Use the default step rate if the media player returned an invalid frame rate or if the project settings force it
		float MillisecondsPerStep = 0.0f;
		if (FMath::IsNearlyEqual(MediaFrameRate, 0.0f) || bForceDefaultStepRate)
		{
			MillisecondsPerStep = DefaultStepRateInMilliseconds;
		}
		else
		{
			MillisecondsPerStep = 1000.0f / MediaFrameRate;
		}


		// Compute the number of ticks in one step and go forward from the media's current time
		const FTimespan TicksInOneStep = ETimespan::TicksPerMillisecond * (MillisecondsPerStep);
		const FTimespan NextStepTime = MediaPlayer->GetTime() + TicksInOneStep;

		// Ensure that we do not attempt to seek past the end of the media
		const FTimespan Duration = MediaPlayer->GetDuration();
		if ((NextStepTime + TicksInOneStep) < Duration)
		{
			MediaPlayer->Seek(NextStepTime);
			MediaPlayer->Pause();
		}
	}

	return FReply::Handled();
}

FReply FCameraCalibrationStepsController::OnForwardButtonClicked() 
{
	// Increase the forward media playback rate
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->SetRate(GetFasterForwardRate());
	}

	return FReply::Handled(); 
}

bool FCameraCalibrationStepsController::DoesMediaSupportSeeking() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->SupportsSeeking();
	}

	return false;
}

bool FCameraCalibrationStepsController::DoesMediaSupportNextReverseRate() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		constexpr bool Unthinned = false;
		return MediaPlayer->SupportsRate(GetFasterReverseRate(), Unthinned);
	}

	return false;
}

bool FCameraCalibrationStepsController::DoesMediaSupportNextForwardRate() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		constexpr bool Unthinned = false;
		return MediaPlayer->SupportsRate(GetFasterForwardRate(), Unthinned);
	}

	return false;
}

float FCameraCalibrationStepsController::GetFasterReverseRate() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		// Get the current playback rate of the media player
		float Rate = MediaPlayer->GetRate();

		// Reverse the playback direction (if needed) to ensure the rate is going in reverse
		if (Rate > -1.0f)
		{
			return -1.0f;
		}

		// Double the reverse playback rate
		return 2.0f * Rate;
	}

	return -1.0f;
}

float FCameraCalibrationStepsController::GetFasterForwardRate() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		// Get the current playback rate of the media player
		float Rate = MediaPlayer->GetRate();

		// Reverse the playback direction (if needed) to ensure the rate is going forward
		if (Rate < 1.0f)
		{
			Rate = 1.0f;
		}

		// Double the forward playback rate
		return 2.0f * Rate;
	}

	return 1.0f;
}

void FCameraCalibrationStepsController::ToggleShowMediaPlaybackControls()
{
	bShowMediaPlaybackButtons = !bShowMediaPlaybackButtons;
}

bool FCameraCalibrationStepsController::AreMediaPlaybackControlsVisible() const
{
	return bShowMediaPlaybackButtons;
}

ACameraActor* FCameraCalibrationStepsController::FindFirstCameraWithCurrentLens() const
{
	// We iterate over all cameras in the scene and try to find one that is using the current LensFile
	ACineCameraActor* FirstCamera = nullptr;
	for (TActorIterator<ACineCameraActor> CameraItr(GetWorld()); CameraItr; ++CameraItr)
	{
		ACineCameraActor* CameraActor = *CameraItr;

		if (ULensComponent* FoundLensComponent = FindLensComponentOnCamera(CameraActor))
		{
			if (FirstCamera == nullptr)
			{
				FirstCamera = CameraActor;
			}
			else
			{
				FText ErrorMessage = LOCTEXT("MoreThanOneCameraFoundError", "There are multiple cameras in the scene using this LensFile. When the asset editor opens, be sure to select the correct camera if not already selected.");
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
				break;
			}
		}
	}

	return FirstCamera;
}

ACameraActor* FCameraCalibrationStepsController::FindDefaultCamera() const
{
	for (TActorIterator<ACineCameraActor> CamActorIt(GetWorld()); CamActorIt; ++CamActorIt)
	{
		ACineCameraActor* FoundActor = *CamActorIt;
		if (IsValid(FoundActor))
		{
			return FoundActor;
		}
	}

	for (TActorIterator<ACameraActor> CamActorIt(GetWorld()); CamActorIt; ++CamActorIt)
	{
		ACameraActor* FoundActor = *CamActorIt;
		if (IsValid(FoundActor))
		{
			return FoundActor;
		}
	}

	return nullptr;
}

void FCameraCalibrationStepsController::TogglePlay()
{
	//@todo Eventually pause should cache the texture instead of relying on player play/pause support.

	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		if (MediaPlayer->IsPaused())
		{
			MediaPlayer->Play();
		}
		else
		{
			// TODO: Trigger the current step (and ultimately the algo) to cache any data it cares about (like 3D scene data)
			MediaPlayer->Pause();
		}
	}
}

void FCameraCalibrationStepsController::Play()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->Play();
	}
}

void FCameraCalibrationStepsController::Pause()
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		MediaPlayer->Pause();
	}
}

bool FCameraCalibrationStepsController::IsPaused() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->IsPaused();
	}

	return true;
}

FLensFileEvaluationInputs FCameraCalibrationStepsController::GetLensFileEvaluationInputs() const
{
	return LensFileEvaluationInputs;
}

ULensFile* FCameraCalibrationStepsController::GetLensFile() const
{
	if (LensFile.IsValid())
	{
		return LensFile.Get();
	}

	return nullptr;
}

ULensComponent* FCameraCalibrationStepsController::FindLensComponentOnCamera(ACameraActor* CineCamera) const
{
	const ULensFile* OpenLensFile = GetLensFile();
	if (CineCamera && OpenLensFile)
	{
		TInlineComponentArray<ULensComponent*> LensComponents;
		CineCamera->GetComponents(LensComponents);

		for (ULensComponent* LensComponent : LensComponents)
		{
			if (LensComponent->GetLensFile() == OpenLensFile)
			{
				return LensComponent;
			}
		}
	}

	return nullptr;
}

ULensComponent* FCameraCalibrationStepsController::FindLensComponent() const
{
	return FindLensComponentOnCamera(GetCamera());
}

const ULensDistortionModelHandlerBase* FCameraCalibrationStepsController::GetDistortionHandler() const
{
	if (ULensComponent* LensComponent = FindLensComponent())
	{
		return LensComponent->GetLensDistortionHandler();
	}

	return nullptr;
}

bool FCameraCalibrationStepsController::SetMediaSource(UMediaSource* InMediaSource)
{
	ClearMedia();
	
	if (!InternalMediaPlayer.IsValid())
	{
		return false;
	}

	if (!InMediaSource)
	{
		return true;
	}
	
	InternalMediaPlayer->OpenSource(InMediaSource);
	InternalMediaPlayer->Play();
	return false;
}

bool FCameraCalibrationStepsController::SetMediaTexture(UMediaTexture* InMediaTexture)
{
	ClearMedia();
	
	ExternalMediaTexture = InMediaTexture;
	return true;
}

bool FCameraCalibrationStepsController::SetMediaProfileSource(UMediaSource* InMediaProfileSource)
{
	ClearMedia();
	
	if (UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
	{
		MediaProfileSourceIndex = MediaProfile->FindMediaSourceIndex(InMediaProfileSource);
		if (MediaProfileSourceIndex == INDEX_NONE)
		{
			return false;
		}

		ExternalMediaTexture = MediaProfile->GetPlaybackManager()->OpenSourceFromIndex(MediaProfileSourceIndex, this);
		return ExternalMediaTexture.IsValid();
	}

	return false;
}

void FCameraCalibrationStepsController::ClearMedia()
{
	if (MediaProfileSourceIndex != INDEX_NONE)
	{
		if (UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
		{
			UMediaProfilePlaybackManager::FCloseSourceArgs Args;
			Args.Consumer = this;
			
			MediaProfile->GetPlaybackManager()->CloseSourceFromIndex(MediaProfileSourceIndex, Args);
		}
	}
	
	MediaProfileSourceIndex = INDEX_NONE;
	InternalMediaPlayer->Close();
	ExternalMediaTexture = nullptr;
}

void FCameraCalibrationStepsController::GetMediaProfileSources(TArray<TWeakObjectPtr<UMediaSource>>& OutSources) const
{
	const UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!MediaProfile)
	{
		return;
	}

	for (int32 MediaSourceIdx = 0; MediaSourceIdx < MediaProfile->NumMediaSources(); ++MediaSourceIdx)
	{
		if (UMediaSource* MediaSource = MediaProfile->GetMediaSource(MediaSourceIdx))
		{
			// Media player will only play valid media sources, so only return the valid media sources from the profile
			if (MediaSource->Validate())
			{
				OutSources.Add(TWeakObjectPtr<UMediaSource>(MediaSource));
			}
		}
	}
}

TArray<TSoftObjectPtr<UMediaSource>> FCameraCalibrationStepsController::GetMediaSourceAssets() const
{
	return CameraCalibrationStepsController::GetUObjectAssets<UMediaSource>();
}

TArray<TSoftObjectPtr<UMediaTexture>> FCameraCalibrationStepsController::GetMediaTextureAssets() const
{
	return CameraCalibrationStepsController::GetUObjectAssets<UMediaTexture>();
}

/** Gets the current media source url being played. Empty if None */
FString FCameraCalibrationStepsController::GetMediaSourceUrl() const
{
	if (UMediaPlayer* MediaPlayer = GetMediaPlayer())
	{
		return MediaPlayer->GetUrl();
	}

	return TEXT("");
}

UMediaSource* FCameraCalibrationStepsController::GetMediaSource() const
{
	if (!InternalMediaPlayer.IsValid())
	{
		return nullptr;
	}

	return InternalMediaPlayer->GetPlaylist()->Get(InternalMediaPlayer->GetPlaylistIndex());
}

UMediaSource* FCameraCalibrationStepsController::GetMediaProfileSource() const
{
	if (MediaProfileSourceIndex != INDEX_NONE)
	{
		if (UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
		{
			return MediaProfile->GetMediaSource(MediaProfileSourceIndex);
		}
	}

	return nullptr;
}

UMediaTexture* FCameraCalibrationStepsController::GetMediaTexture() const
{
	return ExternalMediaTexture.Get();
}

UMediaPlayer* FCameraCalibrationStepsController::GetMediaPlayer() const
{
	if (ExternalMediaTexture.IsValid())
	{
		return ExternalMediaTexture->GetMediaPlayer();
	}

	if (InternalMediaPlayer.IsValid())
	{
		return InternalMediaPlayer.Get();
	}

	return nullptr;
}

const TConstArrayView<TStrongObjectPtr<UCameraCalibrationStep>> FCameraCalibrationStepsController::GetCalibrationSteps() const
{
	return TConstArrayView<TStrongObjectPtr<UCameraCalibrationStep>>(CalibrationSteps);
}

void FCameraCalibrationStepsController::SelectStep(const FName& Name)
{
	for (TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationSteps)
	{
		if (Step.IsValid())
		{
			if (Name == Step->FriendlyName())
			{
				Step->Activate();

				// Switch the overlay material for the tool overlay pass to the material used by the step being selected (and enable if needed)
				SetOverlayMaterial(Step->GetOverlayMID(), Step->IsOverlayEnabled(), EOverlayPassType::ToolOverlay);
			}
			else
			{
				Step->Deactivate();
			}
		}
	}
}

void FCameraCalibrationStepsController::Initialize()
{
	// Not doing these in the constructor so that SharedThis can be used.

	InitializeMediaPlayer();
	CreateSteps();
	
	IMediaProfileManager::Get().OnMediaProfileChanged().AddSP(this, &FCameraCalibrationStepsController::OnActiveMediaProfileChanged);
}

UMediaTexture* FCameraCalibrationStepsController::GetMediaOverlayTexture() const
{
	if (ExternalMediaTexture.IsValid())
	{
		return ExternalMediaTexture.Get();
	}

	if (InternalMediaPlayer->IsReady())
	{
		return MediaTexture.IsValid() ? MediaTexture.Get() : nullptr;
	}

	return nullptr;
}

FIntPoint FCameraCalibrationStepsController::GetMediaOverlayResolution() const
{
	if (ExternalMediaTexture.IsValid())
	{
		return FIntPoint(ExternalMediaTexture.Get()->GetSurfaceWidth(), ExternalMediaTexture.Get()->GetSurfaceHeight());
	}

	if (InternalMediaPlayer->IsReady())
	{
		return MediaTexture.IsValid() ? FIntPoint(MediaTexture.Get()->GetSurfaceWidth(), MediaTexture.Get()->GetSurfaceHeight()) : FIntPoint::ZeroValue;
	}

	return FIntPoint::ZeroValue;
}

bool FCameraCalibrationStepsController::CalculateNormalizedMouseClickPosition(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FVector2f& OutPosition, ESimulcamViewportPortion ViewportPortion) const
{
	// Reject viewports with no area
	if (FMath::IsNearlyZero(MyGeometry.Size.X) || FMath::IsNearlyZero(MyGeometry.Size.Y))
	{
		return false;
	}

	// About the Mouse Event data:
	// 
	// * MouseEvent.GetScreenSpacePosition(): Position in pixels on the screen (independent of window size of position)
	// * MyGeometry.Size                    : Size of viewport (the one with the media, not the whole window)
	// * MyGeometry.AbsolutePosition        : Position of the top-left corner of viewport within screen
	// * MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()) gives you the pixel coordinates local to the viewport.

	const FVector2f LocalInPixels = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	float XNormalized = LocalInPixels.X / MyGeometry.Size.X;
	float YNormalized = LocalInPixels.Y / MyGeometry.Size.Y;

	if (ViewportPortion == ESimulcamViewportPortion::CameraFeed)
	{
		FIntRect CropRegion;
		FIntPoint CropSize;
		if (!CalculateCameraFeedCropRegion(CropRegion, CropSize))
		{
			return false;
		}
		
		const FIntPoint MediaOverlayResolution = GetMediaOverlayResolution();
		
		// Convert normalized viewport coordinates to media texture coordinates
		const float MediaX = XNormalized * MediaOverlayResolution.X;
		const float MediaY = YNormalized * MediaOverlayResolution.Y;
		
		// Check if the click is within the cropped region
		if (MediaX < CropRegion.Min.X || MediaX > CropRegion.Max.X ||
		    MediaY < CropRegion.Min.Y || MediaY > CropRegion.Max.Y)
		{
			return false;
		}
		
		// Map from cropped region to normalized camera feed coordinates
		XNormalized = (MediaX - CropRegion.Min.X) / CropSize.X;
		YNormalized = (MediaY - CropRegion.Min.Y) / CropSize.Y;

		// Ensure normalized values are within valid range (defensive check)
		if (XNormalized < 0.0f || XNormalized > 1.0f || YNormalized < 0.0f || YNormalized > 1.0f)
		{
			return false;
		}
	}

	// Position 0~1. Origin at top-left corner of the viewport.
	OutPosition = FVector2f(XNormalized, YNormalized);

	return true;
}

bool FCameraCalibrationStepsController::ReadMediaPixels(TArray<FColor>& Pixels, FIntPoint& Size, FText& OutErrorMessage, ESimulcamViewportPortion ViewportPortion) const
{
	// Renders a media texture to a render target (via canvas rendering) and reads those textures to the output pixel array
	// NOTE: UMediaTexture's texture resource is both an FTextureResource and an FRenderTarget, so in theory we could read the media texture directly,
	// but such functionality is not exposed, nor is it likely the pixels are currently configured for CPU read access. As such, we must output to an
	// intermediate render target to access the pixels
	UMediaTexture* MediaOverlayTexture = GetMediaOverlayTexture();
	if (!MediaOverlayTexture)
	{
		OutErrorMessage = LOCTEXT("InvalidMediaTexture", "Invalid MediaTexture");
		return false;
	}

	const FIntPoint MediaTextureSize(MediaOverlayTexture->GetSurfaceWidth(), MediaOverlayTexture->GetSurfaceHeight());
	if (CachedMediaRenderTarget->SizeX != MediaTextureSize.X || CachedMediaRenderTarget->SizeY != MediaTextureSize.Y)
	{
		CachedMediaRenderTarget->ResizeTarget(MediaTextureSize.X, MediaTextureSize.Y);
		FlushRenderingCommands();
	}
	
	UKismetRenderingLibrary::ClearRenderTarget2D(GetWorld(), CachedMediaRenderTarget.Get(), FLinearColor::Black);

	UCanvas* Canvas;
	FVector2D CanvasToRenderTargetSize;
	FDrawToRenderTargetContext RenderTargetContext;
	UKismetRenderingLibrary::BeginDrawCanvasToRenderTarget(GetWorld(), CachedMediaRenderTarget.Get(), /*out*/ Canvas, /*out*/ CanvasToRenderTargetSize, /*out*/ RenderTargetContext);
	{
		Canvas->K2_DrawTexture(MediaOverlayTexture, FVector2D::ZeroVector, MediaTextureSize, FVector2D::ZeroVector);
	}
	UKismetRenderingLibrary::EndDrawCanvasToRenderTarget(GetWorld(), RenderTargetContext);
	
	FTextureRenderTargetResource* MediaRenderTarget = CachedMediaRenderTarget->GameThread_GetRenderTargetResource();
	if (!MediaRenderTarget)
	{
		OutErrorMessage = LOCTEXT("InvalidRenderTargetResource", "Media render target did not have a RenderTarget resource");
		return false;
	}

	// Read the pixels onto CPU
	TArray<FColor> MediaPixels;
	const bool bReadPixels = MediaRenderTarget->ReadPixels(MediaPixels);

	if (!bReadPixels)
	{
		OutErrorMessage = LOCTEXT("ReadPixelsFailed", "ReadPixels from render target failed");
		return false;
	}

	ULensFile* LensFilePtr = GetLensFile();
	if (!LensFilePtr)
	{
		OutErrorMessage = LOCTEXT("InvalidLensFile", "There was no LensFile found.");
		return false;
	}

	if ((ViewportPortion == ESimulcamViewportPortion::CameraFeed) && LensFilePtr->CameraFeedInfo.IsValid())
	{
		Size = LensFilePtr->CameraFeedInfo.GetDimensions();
		
		// Protect against invalid dimensions
		if (Size.X <= 0 || Size.Y <= 0)
		{
			OutErrorMessage = LOCTEXT("InvalidCameraFeedDimensions", "Camera feed dimensions are invalid (zero or negative).");
			return false;
		}
		
		Pixels.SetNumUninitialized(Size.X * Size.Y);

		const FIntPoint MediaResolution = MediaRenderTarget->GetSizeXY();
		
		// Protect against invalid media resolution
		if (MediaResolution.X <= 0 || MediaResolution.Y <= 0)
		{
			OutErrorMessage = LOCTEXT("InvalidMediaResolution", "Media resolution is invalid (zero or negative).");
			return false;
		}
		
		// Use the common function to calculate crop region
		FIntRect CropRegion;
		FIntPoint CropSize;
		if (!CalculateCameraFeedCropRegion(CropRegion, CropSize))
		{
			OutErrorMessage = LOCTEXT("InvalidCropCalculation", "Failed to calculate valid crop region.");
			return false;
		}
		
		// Map each output pixel to the fitted media region
		for (int32 YCoordinate = 0; YCoordinate < Size.Y; YCoordinate++)
		{
			for (int32 XCoordinate = 0; XCoordinate < Size.X; XCoordinate++)
			{
				// Map output coordinates to crop region coordinates
				const int32 ScaledX = CropRegion.Min.X + (XCoordinate * CropSize.X) / Size.X;
				const int32 ScaledY = CropRegion.Min.Y + (YCoordinate * CropSize.Y) / Size.Y;

				Pixels[YCoordinate * Size.X + XCoordinate] = MediaPixels[ScaledY * MediaResolution.X + ScaledX];
			}
		}
	}
	else
	{
		Pixels = MoveTemp(MediaPixels);
		Size = MediaRenderTarget->GetSizeXY();
	}

	check(Pixels.Num() == Size.X * Size.Y);

	return true;
}

void FCameraCalibrationStepsController::OnActiveMediaProfileChanged(UMediaProfile* PreviousMediaProfile, UMediaProfile* NewMediaProfile)
{
	if (MediaProfileSourceIndex != INDEX_NONE)
	{
		if (PreviousMediaProfile)
		{
			UMediaProfilePlaybackManager::FCloseSourceArgs Args;
			Args.Consumer = this;
			
			PreviousMediaProfile->GetPlaybackManager()->CloseSourceFromIndex(MediaProfileSourceIndex, Args);
		}
		MediaProfileSourceIndex = INDEX_NONE;
	}

	ClearMedia();
}

bool FCameraCalibrationStepsController::CalculateCameraFeedCropRegion(FIntRect& OutCropRegion, FIntPoint& OutCropSize) const
{
	ULensFile* LensFilePtr = GetLensFile();
	if (!LensFilePtr || !LensFilePtr->CameraFeedInfo.IsValid())
	{
		return false;
	}

	const FIntPoint CameraFeedDimensions = LensFilePtr->CameraFeedInfo.GetDimensions();
	const FIntPoint MediaResolution = GetMediaOverlayResolution();
	
	// Protect against invalid dimensions
	if (CameraFeedDimensions.X <= 0 || CameraFeedDimensions.Y <= 0 || 
	    MediaResolution.X <= 0 || MediaResolution.Y <= 0)
	{
		return false;
	}
	
	// Calculate the center-crop/fit region
	const float MediaAspect = (float)MediaResolution.X / (float)MediaResolution.Y;
	const float DesiredAspect = (float)CameraFeedDimensions.X / (float)CameraFeedDimensions.Y;
	
	if (MediaAspect > DesiredAspect)
	{
		// Media is wider - fit by height, center horizontally
		OutCropSize.Y = MediaResolution.Y;
		OutCropSize.X = FMath::RoundToInt(MediaResolution.Y * DesiredAspect);
		OutCropRegion.Min.X = (MediaResolution.X - OutCropSize.X) / 2;
		OutCropRegion.Min.Y = 0;
	}
	else
	{
		// Media is taller - fit by width, center vertically
		OutCropSize.X = MediaResolution.X;
		OutCropSize.Y = FMath::RoundToInt(MediaResolution.X / DesiredAspect);
		OutCropRegion.Min.X = 0;
		OutCropRegion.Min.Y = (MediaResolution.Y - OutCropSize.Y) / 2;
	}
	
	OutCropRegion.Max = OutCropRegion.Min + OutCropSize;
	
	// Validate crop size
	return (OutCropSize.X > 0 && OutCropSize.Y > 0);
}

#undef LOCTEXT_NAMESPACE
