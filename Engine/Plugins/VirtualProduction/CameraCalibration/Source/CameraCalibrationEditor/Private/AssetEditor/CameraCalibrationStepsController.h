// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/Ticker.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "Input/Reply.h"
#include "LensFile.h"
#include "UObject/StrongObjectPtr.h"


class ACameraActor;
class ACompositingElement;
class FCameraCalibrationToolkit;
class SWidget;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;
class UMediaProfile;
class UCameraCalibrationStep;
class UCompositingElementMaterialPass;
class ULensComponent;
class ULensDistortionModelHandlerBase;
class ULensFile;

struct FGeometry;
struct FPointerEvent;

/** Enumeration of overlay passes used to indicate which overlay pass to interact with */
enum class EOverlayPassType : uint8
{
	ToolOverlay = 0,
	UserOverlay = 1
};

/** Enumeration specifying the portion of the viewport to consider when performing operations such as normalizing the mouse position or reading pixels from the media render target */
enum class ESimulcamViewportPortion : uint8
{
	FullViewport = 0,
	CameraFeed = 1
};

/**
 * Controller for SCameraCalibrationSteps, where the calibration steps are hosted in.
 */
class FCameraCalibrationStepsController : public TSharedFromThis<FCameraCalibrationStepsController>
{
public:

	FCameraCalibrationStepsController(TWeakPtr<FCameraCalibrationToolkit> InCameraCalibrationToolkit, ULensFile* InLensFile);
	~FCameraCalibrationStepsController();

	/** Initialize resources. */
	void Initialize();

	/** Returns the UI that this object controls */
	TSharedPtr<SWidget> BuildUI();

	/** Creates the media player and media texture used to play the media source */
	void InitializeMediaPlayer();

	/** Gets the texture used to display the media overlay */
	UMediaTexture* GetMediaOverlayTexture() const;

	/** Gets the resolution of the media overlay */
	FIntPoint GetMediaOverlayResolution() const;

	/** Returns the size of the camera feed (which may be smaller than the media overlay or the CG render) */
	FIntPoint GetCameraFeedSize() const;

	/** Returns the aspect ratio of the camera feed */
	float GetCameraFeedAspectRatio() const;

	/** Returns the CG weight that is composited on top of the media */
	float GetWiperWeight() const;

	/** Sets the weight/alpha for that CG that is composited on top of the media. 0 means invisible. */
	void SetWiperWeight(float InWeight);

	/** Sets the camera used for the CG */
	void SetCamera(ACameraActor* InCamera);

	/** Returns the camera used for the CG */
	ACameraActor* GetCamera() const;

	/** Toggles the play/stop state of the media player */
	void TogglePlay();

	/** Returns true if the media player is paused */
	bool IsPaused() const;

	/** Set play on the media player */
	void Play();

	/** Set pause on the media player */
	void Pause();

	/** Returns the latest data used when evaluating the lens */
	FLensFileEvaluationInputs GetLensFileEvaluationInputs() const;

	/** Returns the LensFile that this tool is using */
	ULensFile* GetLensFile() const;

	/** Returns the first LensComponent attached to the CG camera whose LensFile matches the open asset */
	ULensComponent* FindLensComponent() const;

	/** Returns the distortion handler used to distort the CG being displayed in the simulcam viewport */
	const ULensDistortionModelHandlerBase* GetDistortionHandler() const;

	/** Sets the current media source being used in the calibration */
	bool SetMediaSource(UMediaSource* InMediaSource);

	/** Sets the current media to a media texture being used in the calibration */
	bool SetMediaTexture(UMediaTexture* InMediaTexture);

	/** Sets the current media to a media source in the active media profile */
	bool SetMediaProfileSource(UMediaSource* InMediaProfileSource);
	
	/** Clears out any media source or media texture currently being played */
	void ClearMedia();

	/** Gets a list of valid media sources from the current media profile */
	void GetMediaProfileSources(TArray<TWeakObjectPtr<UMediaSource>>& OutSources) const;

	/** Gets all media source assets in the content browser */
	TArray<TSoftObjectPtr<UMediaSource>> GetMediaSourceAssets() const;

	/** Gets all media texture assets in the content browser */
	TArray<TSoftObjectPtr<UMediaTexture>> GetMediaTextureAssets() const;
	
	/** Gets the current media source url being played. Empty if None */
	FString GetMediaSourceUrl() const;

	/** Gets the current media source being played, or nullptr if nothing is being played */
	UMediaSource* GetMediaSource() const;

	/** Gets the media source from the active media profile currently being played, or nullptr if no media profile source is being played */
	UMediaSource* GetMediaProfileSource() const;
	
	/** Gets the current media texture being displayed, or nullptr if the media texture is not being overridden */
	UMediaTexture* GetMediaTexture() const;

	/**
	 * Gets the active media player being used; if an external media texture is being used,
	 * this returns the player that texture is linked to, otherwise, the internal media player is returned
	 */
	UMediaPlayer* GetMediaPlayer() const;
	
	/** Returns the calibration steps */
	const TConstArrayView<TStrongObjectPtr<UCameraCalibrationStep>> GetCalibrationSteps() const;

	/** Returns the calibration steps */
	void SelectStep(const FName& Name);

	/** Calculates the normalized (0~1) coordinates in the simulcam viewport of the given mouse click */
	bool CalculateNormalizedMouseClickPosition(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FVector2f& OutPosition, ESimulcamViewportPortion ViewportPortion = ESimulcamViewportPortion::FullViewport) const;

	/** Finds the world being used by the tool for finding and spawning objects */
	UWorld* GetWorld() const;

	/** Reads the pixels from the open media source */
	bool ReadMediaPixels(TArray<FColor>& Pixels, FIntPoint& Size, FText& OutErrorMessage, ESimulcamViewportPortion ViewportPortion = ESimulcamViewportPortion::FullViewport) const;

	/** Returns true if the overlay transform pass is currently enabled */
	bool IsOverlayEnabled(EOverlayPassType OverlayPass = EOverlayPassType::ToolOverlay) const;

	/** Sets the enabled state of the overlay transform pass */
	void SetOverlayEnabled(const bool bEnabled = true, EOverlayPassType OverlayPass = EOverlayPassType::ToolOverlay);

	/** Returns the overlay material used by the input overlay pass type */
	UMaterialInterface* GetOverlayMaterial(EOverlayPassType OverlayPass) const;
	
	/** Sets the overlay material to be used by the overlay transform pass */
	void SetOverlayMaterial(UMaterialInterface* OverlayMaterial, bool bShowOverlay = true, EOverlayPassType OverlayPass = EOverlayPassType::ToolOverlay);

	/** Use the input mouse position (representing any corner of the camera feed) and the dimensions of the media source to calculate the new size of the camera feed */
	void SetCameraFeedDimensionsFromMousePosition(FVector2D MousePosition);

	/** Set the camera feed dimensions info of the LensFile being edited */
	void SetCameraFeedDimensions(FIntPoint Dimensions, bool bMarkAsOverridden);

	/** Resets the camera feed dimensions by fitting it within the currently playing media source */
	void ResetCameraFeedDimensions();

public:

	/** Called by the UI when the Simulcam Viewport is clicked */
	void OnSimulcamViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Called by the UI when the Simulcam Viewport receives keyboard input */
	bool OnSimulcamViewportInputKey(const FKey& InKey, const EInputEvent& InEvent);

	/** Called by the UI when the Simulcam Viewport receives a marquee select event */
	void OnSimulcamViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition);

	/** Called by the UI when the rewind button is clicked */
	FReply OnRewindButtonClicked();

	/** Called by the UI when the reverse button is clicked */
	FReply OnReverseButtonClicked();

	/** Called by the UI when the step back button is clicked */
	FReply OnStepBackButtonClicked();

	/** Called by the UI when the play button is clicked */
	FReply OnPlayButtonClicked();

	/** Called by the UI when the pause button is clicked */
	FReply OnPauseButtonClicked();

	/** Called by the UI when the step forward button is clicked */
	FReply OnStepForwardButtonClicked();

	/** Called by the UI when the forward button is clicked */
	FReply OnForwardButtonClicked();

	/** Returns true if the media player supports seeking */
	bool DoesMediaSupportSeeking() const;

	/** Returns true if the media player supports a reverse rate that is faster than its current rate */
	bool DoesMediaSupportNextReverseRate() const;

	/** Returns true if the media player supports a forward rate that is faster than its current rate */
	bool DoesMediaSupportNextForwardRate() const;

	/** Computes the reverse playback rate */
	float GetFasterReverseRate() const;

	/** Computes the fast forward playback rate */
	float GetFasterForwardRate() const;

	/** Toggle the setting that controls whether the media playback buttons will be visible in the UI */
	void ToggleShowMediaPlaybackControls();

	/** Returns true if the media playback buttons are visible in the UI  */
	bool AreMediaPlaybackControlsVisible() const;

private:

	/** Returns the first lens component with a matching LensFile found on the input camera, or nullptr if none exists */
	ULensComponent* FindLensComponentOnCamera(ACameraActor* CineCamera) const;

	/** Releases resources used by the tool */
	void Cleanup();

	/** Convenience function that returns the first camera it finds that is using the lens associated with this object. */
	ACameraActor* FindFirstCameraWithCurrentLens() const;

	/** When no camera can be found using the current lens file, this function finds a default camera in the level to use */
	ACameraActor* FindDefaultCamera() const;

	/** Called by the core ticker */
	bool OnTick(float DeltaTime);

	/** Finds and creates the available calibration steps */
	void CreateSteps();

	/** Iteratively attempt to resize the camera feed until its aspect ratio matches the input camera aspect ratio */
	void MinimizeAspectRatioError(FIntPoint& CameraFeedDimensions, float CameraAspectRatio);

	/** Raised when the active media profile is changed */
	void OnActiveMediaProfileChanged(UMediaProfile* PreviousMediaProfile, UMediaProfile* NewMediaProfile);

	/** Calculate the crop region for center-fit sampling of camera feed from media texture */
	bool CalculateCameraFeedCropRegion(FIntRect& OutCropRegion, FIntPoint& OutCropSize) const;

private:

	/** Pointer to the camera calibration toolkit */
	TWeakPtr<FCameraCalibrationToolkit> CameraCalibrationToolkit;

	/** Array of the calibration steps that this controller is managing */
	TArray<TStrongObjectPtr<UCameraCalibrationStep>> CalibrationSteps;

	/** The lens asset */
	TWeakObjectPtr<class ULensFile> LensFile;

	/** The index of the media source in the active media profile being played */
	int32 MediaProfileSourceIndex = INDEX_NONE;

	/** The media texture used by the media plate */
	TWeakObjectPtr<UMediaTexture> MediaTexture;

	/** An external media texture to use by the media plate instead of the internal one */
	TWeakObjectPtr<UMediaTexture> ExternalMediaTexture;
	
	/** The media player that is playing the selected media source */
	TStrongObjectPtr<UMediaPlayer> InternalMediaPlayer;

	/** Used to extract pixel data from the media texture, cached to avoid having to recreate it every time */
	TStrongObjectPtr<UTextureRenderTarget2D> CachedMediaRenderTarget;

	/** The material used to render the overlay for the tool overlay pass */
	TWeakObjectPtr<UMaterialInterface> ToolOverlayMaterial;

	/** Whether the tool overlay is enabled */
	bool bToolOverlayEnabled = false;
	
	/** The material used to render the overlay for the user overlay pass */
	TWeakObjectPtr<UMaterialInterface> UserOverlayMaterial;

	/** Whether the user overlay is enabled */
	bool bUserOverlayEnabled = false;

	/** Wiper weight for overlaying the media texture on the CG render */
	float WiperWeight = 0.5f;

	/** Keeps track of the last read media overlay resolution so that resolution changes can be detected */
	FIntPoint LastMediaOverlayResolution;
	
	/** The currently selected camera */
	TWeakObjectPtr<ACameraActor> Camera;

	/** The currently selected camera's camera component */
	TWeakObjectPtr<UCineCameraComponent> CineCameraComponent;

	/** The delegate for the core ticker callback */
	FTSTicker::FDelegateHandle TickerHandle;

	/** Evaluation Data supplied by the Lens Component for the current frame. Only valid during the current frame. */
	FLensFileEvaluationInputs LensFileEvaluationInputs;

	/** Setting to control whether the media playback buttons are visible */
	bool bShowMediaPlaybackButtons = true;
};
