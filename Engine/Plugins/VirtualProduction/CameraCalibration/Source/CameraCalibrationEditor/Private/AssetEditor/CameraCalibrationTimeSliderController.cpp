// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationTimeSliderController.h"

#include "CameraCalibrationSettings.h"
#include "CameraCalibrationStepsController.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Fonts/FontMeasure.h"
#include "LensFile.h"

/** Utility struct for converting between scrub range space and local/absolute screen space */
struct FCameraCalibrationTimeSliderController::FScrubRangeToScreen
{
	double ViewStart;
	float  PixelsPerInput;

	FScrubRangeToScreen(const TRange<double>& InViewInput, const FVector2D& InWidgetSize )
	{
		const float ViewInputRange = InViewInput.Size<double>();

		ViewStart      = InViewInput.GetLowerBoundValue();
		PixelsPerInput = ViewInputRange > 0 ? ( InWidgetSize.X / ViewInputRange ) : 0;
	}

	/** Local Widget Space -> Curve Input domain. */
	double LocalXToInput(float ScreenX) const
	{
		return PixelsPerInput > 0 ? (ScreenX/PixelsPerInput) + ViewStart : ViewStart;
	}

	/** Curve Input domain -> local Widget Space */
	float InputToLocalX(double Input) const
	{
		return (Input - ViewStart) * PixelsPerInput;
	}
};

FCameraCalibrationTimeSliderController::FCameraCalibrationTimeSliderController(const TSharedRef<FCameraCalibrationStepsController>& InCalibrationStepsController, ULensFile* InLensFile)
	: CalibrationStepsControllerWeakPtr(InCalibrationStepsController)
	, LensFileWeakPtr(InLensFile)
{
	ScrubHandleUpBrush          = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.VanillaScrubHandleUp" ) ); 
	ScrubHandleDownBrush        = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.VanillaScrubHandleDown" ) );

	TimeSliderArgs.DisplayRate = FFrameRate(30, 1);
	TimeSliderArgs.TickResolution =  FFrameRate(30000, 1);
	TimeSliderArgs.ViewRange = FAnimatedRange(0.0, 1.0);
	TimeSliderArgs.ScrubPosition = FFrameTime();
}

int32 FCameraCalibrationTimeSliderController::OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	if (!IsVisible())
	{
		return LayerId;
	}

	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange      = TimeSliderArgs.ViewRange.Get();
	const float    LocalViewRangeMin   = LocalViewRange.GetLowerBoundValue();
	const float    LocalViewRangeMax   = LocalViewRange.GetUpperBoundValue();
	const float    LocalSequenceLength = LocalViewRangeMax-LocalViewRangeMin;

	if ( LocalSequenceLength > 0)
	{		
		FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.Size );

		// draw playback & selection range
		FPaintPlaybackRangeArgs PlaybackRangeArgs(
			bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_L"),
			bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_R"),
			6.f
		);

		// Draw the scrub handle
		const float      HandleStart        = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal()) - 7.0f;
		const float      HandleEnd          = HandleStart + 13.0f;

		const int32 ArrowLayer = LayerId + 2;
		FPaintGeometry MyGeometry =	AllottedGeometry.ToPaintGeometry( FVector2D( HandleEnd - HandleStart, AllottedGeometry.Size.Y ), FSlateLayoutTransform(FVector2D( HandleStart, 0 )) );
		FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();
		{
			ScrubColor.A = ScrubColor.A * 0.75f;
			ScrubColor.B *= 0.1f;
			ScrubColor.G *= 0.2f;
		}

		const FSlateBrush* Brush = (bMirrorLabels ? ScrubHandleUpBrush : ScrubHandleDownBrush);
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			ArrowLayer,
			MyGeometry,
			Brush,
			DrawEffects,
			ScrubColor
		);

		{
			// Draw the current input value next to the scrub handle
			FLinearColor TextColor = FLinearColor::White;
			if (TimeSliderArgs.ScrubPositionParent.Get() != MovieSceneSequenceID::Invalid)
			{
				TextColor = FLinearColor::Yellow;
			}

			FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

			const FString FormattedCategoryString = ScrubInput.GetFormattedCategoryString();
			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FVector2D TextSize = FontMeasureService->Measure(FormattedCategoryString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			constexpr float TextOffsetPx  = 2.f;
			bool  bDrawLeft    = (AllottedGeometry.Size.X - HandleEnd) < (TextSize.X + 14.f) - TextOffsetPx;
			float TextPosition = bDrawLeft ? HandleStart - TextSize.X - TextOffsetPx : HandleEnd + TextOffsetPx;
			FVector2D TextOffset( TextPosition, bMirrorLabels ? AllottedGeometry.Size.Y - TextSize.Y : 0.f );

			FSlateDrawElement::MakeText(
				OutDrawElements,
				LayerId + 1, 
				AllottedGeometry.ToPaintGeometry( TextSize, FSlateLayoutTransform(TextOffset) ), 
				FormattedCategoryString, 
				SmallLayoutFont,
				DrawEffects,
				TextColor 
			);
		}

		return ArrowLayer;
	}

	return LayerId;
}

int32 FCameraCalibrationTimeSliderController::OnPaintViewArea( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args ) const
{
	if (!IsVisible())
	{
		return LayerId;
	}
	
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	if( Args.bDisplayScrubPosition )
	{
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		const FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.Size );
		
		// Draw a line for the scrub position
		const float LinePos = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal());

		TArray<FVector2D> LinePoints;
		{
			LinePoints.AddUninitialized(2);
			LinePoints[0] = FVector2D( 0.0f, 0.0f );
			LinePoints[1] = FVector2D( 0.0f, FMath::FloorToFloat( AllottedGeometry.Size.Y ) );
		}
	
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId+1,
			AllottedGeometry.ToPaintGeometry( FVector2D(1.0f,1.0f), FSlateLayoutTransform(FVector2D(LinePos, 0.0f )) ),
			LinePoints,
			DrawEffects,
			FLinearColor(1.f, 1.f, 1.f, .5f),
			false
		);
	}
	
	return LayerId;
}

void FCameraCalibrationTimeSliderController::SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation)
{
	// Clamp to a minimum size to avoid zero-sized or negative visible ranges
	const double MinVisibleTimeRange = FFrameNumber(1) / GetTickResolution();
	const TRange<double> ExistingViewRange  = TimeSliderArgs.ViewRange.Get();

	if (NewRangeMax == ExistingViewRange.GetUpperBoundValue())
	{
		if (NewRangeMin > (NewRangeMax - MinVisibleTimeRange) )
		{
			NewRangeMin = NewRangeMax - MinVisibleTimeRange;
		}
	}
	else if (NewRangeMax < (NewRangeMin + MinVisibleTimeRange) )
	{
		NewRangeMax = NewRangeMin + MinVisibleTimeRange;
	}

	// Clamp to the clamp range
	const TRange<double> NewRange = TRange<double>(NewRangeMin, NewRangeMax);
	TimeSliderArgs.ViewRange.Set( NewRange );
}

void FCameraCalibrationTimeSliderController::Tick(float DeltaTime)
{	
	ScrubInput.Reset();
	
	if (!SelectedCategory.IsSet())
	{
		return;
	}

	if (!ensure(LensFileWeakPtr.IsValid()))
	{
		return;
	}

	const TSharedPtr<FCameraCalibrationStepsController> CalibrationStepsController = CalibrationStepsControllerWeakPtr.Pin();
	if (!CalibrationStepsController.IsValid())
	{
		return;
	}

	const ELensDataCategory SelectedCategoryValue = SelectedCategory.GetValue();
	const FLensFileEvaluationInputs EvalInputs = CalibrationStepsController->GetLensFileEvaluationInputs();
	if (!EvalInputs.bIsValid)
	{
		return;
	}

	if (SelectedCategoryValue == ELensDataCategory::Focus)
	{
		const float RawFocus = EvalInputs.Focus;
		CommitPositionChange(RawFocus, TEXT("Focus"));
	}
	else if (SelectedCategoryValue == ELensDataCategory::Zoom ||
			SelectedCategoryValue == ELensDataCategory::Distortion ||
			SelectedCategoryValue == ELensDataCategory::STMap ||
			SelectedCategoryValue == ELensDataCategory::ImageCenter ||
			SelectedCategoryValue == ELensDataCategory::NodalOffset)
	{
		if (SelectedFocus.IsSet())
		{
			
			const float RawFocus = EvalInputs.Focus;
			const float RawZoom = EvalInputs.Zoom;
			if (LensFileWeakPtr->GetDataTable(SelectedCategoryValue)->IsFocusBetweenNeighbor(SelectedFocus.GetValue(), RawFocus))
			{
				CommitPositionChange(RawZoom, TEXT("Zoom"));
			}
		}
	}
	else if (SelectedCategory == ELensDataCategory::Iris)
	{
		const float RawIris = EvalInputs.Iris;
		CommitPositionChange(RawIris, TEXT("Iris"));
	}
}

TStatId FCameraCalibrationTimeSliderController::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCameraCalibrationTimeSliderController, STATGROUP_Tickables);
}

void FCameraCalibrationTimeSliderController::CommitPositionChange(const float InValue, const FString& InCategoryString)
{
	ScrubInput = { InValue, InCategoryString };
	TimeSliderArgs.ScrubPosition.Set( InValue * GetTickResolution() );
}

void FCameraCalibrationTimeSliderController::UpdateSelection(const ELensDataCategory InCategory, const TOptional<float> InFloatPoint)
{
	SelectedCategory = InCategory;
	SelectedFocus = InFloatPoint;
}

void FCameraCalibrationTimeSliderController::ResetSelection()
{
	SelectedCategory = TOptional<ELensDataCategory>();
	SelectedFocus = TOptional<float>();
}

bool FCameraCalibrationTimeSliderController::IsVisible() const
{
	const UCameraCalibrationEditorSettings* EditorSettings = GetDefault<UCameraCalibrationEditorSettings>();

	return EditorSettings->bEnableTimeSlider && ScrubInput.IsValid();
}
