// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ITimeSlider.h"
#include "TimeSliderArgs.h"
#include "TickableEditorObject.h"

class FCameraCalibrationStepsController;
class ULensFile;
struct FSlateBrush;
enum class ELensDataCategory : uint8;

/**
 * Controlling the time slider visuals from the given zoom, focus, iris input
 */
class FCameraCalibrationTimeSliderController
	: public ITimeSliderController
	, public TSharedFromThis<FCameraCalibrationTimeSliderController>
	, public FTickableEditorObject

{
public:
	FCameraCalibrationTimeSliderController(const TSharedRef<FCameraCalibrationStepsController>& InCalibrationStepsController, ULensFile* InLensFile);
	
	//~ Begin ITimeSliderController Interface
	virtual int32 OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual int32 OnPaintViewArea( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args ) const override;
	virtual void SetViewRange( double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation ) override;

	virtual FCursorReply OnCursorQuery( const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override { return FCursorReply::Unhandled(); }
	virtual FFrameRate GetDisplayRate() const override {  return TimeSliderArgs.DisplayRate.Get(); }
	virtual FFrameRate GetTickResolution() const override { return TimeSliderArgs.TickResolution.Get(); }
	virtual FFrameTime GetScrubPosition() const override { return TimeSliderArgs.ScrubPosition.Get(); }
	virtual FAnimatedRange GetViewRange() const override { return TimeSliderArgs.ViewRange.Get(); }
	//~ End ITimeSliderController Interface

	//~ Begin ISequencerInputHandler Interface
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return FReply::Unhandled(); }
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return FReply::Unhandled(); }
	virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return FReply::Unhandled(); }
	//~ Begin ISequencerInputHandler Interface

	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return true; }
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

	/**
	 * Handler for category changed
	 * @param InCategory new category
	 * @param InFloatPoint new point value
	 */
	void UpdateSelection(const ELensDataCategory InCategory, const TOptional<float> InFloatPoint);

	/** Resets the selection and prevent the displaying */
	void ResetSelection();

private:
	/** Whether time slider should be visible */
	bool IsVisible() const;

private:
	/** forward declared of scrub range struct */
	struct FScrubRangeToScreen;

	/**
	 * Scrub input using for visual and validation of input data
	 */
	struct FScrubInput
	{
		FScrubInput()
		{}

		FScrubInput(const float InValue, const FString& InCategoryString)
			: Value(InValue)
			, CategoryString(InCategoryString)
		{
			FormattedCategoryString = FString::Printf(TEXT("%s: %s"), *CategoryString, *FString::SanitizeFloat(InValue));
		}

		/** Whether the scrub input is valid */
		bool IsValid() const { return Value.IsSet(); }

		/** Get the formatted input string */
		const FString& GetFormattedCategoryString() const { return FormattedCategoryString; }

		/** Reset the input values */
		void Reset()
		{
			*this = FScrubInput();
		}

		/** Get scrub input value */
		TOptional<float> GetValue() const
		{
			return Value;
		}

	private:
		/** Input value */
		TOptional<float> Value;

		/** Input category string */
		FString CategoryString;

		/** Formatted string */
		FString FormattedCategoryString;
	};

private:
	/**
	 * Time scrub line position handler
	 *
	 * @param InValue Input scrub value
	 * @param InCategoryString Scrub category name
	 */
	void CommitPositionChange(const float InValue, const FString& InCategoryString);

private:
	/** Container for the input time slider delegates and properties */ 
	FTimeSliderArgs TimeSliderArgs;

	/** Brush for drawing an upwards facing scrub handles */
	const FSlateBrush* ScrubHandleUpBrush;
	
	/** Brush for drawing a downwards facing scrub handle */
	const FSlateBrush* ScrubHandleDownBrush;

	/** Holds the weak reference to SCameraCalibrationSteps, where the calibration steps are hosted in. */
	TWeakPtr<FCameraCalibrationStepsController> CalibrationStepsControllerWeakPtr;

	/** Selected Lens category */
	TOptional<ELensDataCategory> SelectedCategory;

	/** Selected focus point */
	TOptional<float> SelectedFocus;

	/** Scrub input struct */
	FScrubInput ScrubInput;

	/** Pointer for asset lens file */
	TWeakObjectPtr<ULensFile> LensFileWeakPtr;
};
