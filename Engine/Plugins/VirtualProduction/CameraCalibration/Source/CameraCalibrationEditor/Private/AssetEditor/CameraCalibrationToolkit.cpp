// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationToolkit.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISettingsModule.h"
#include "CameraCalibrationStepsController.h"
#include "CameraCalibrationCommands.h"
#include "EngineAnalytics.h"
#include "LensFile.h"
#include "PropertyEditorModule.h"
#include "SLensEvaluation.h"
#include "SLensFilePanel.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "CameraCalibrationToolkit"

TWeakPtr<SWindow> FCameraCalibrationToolkit::PopupWindow;

namespace CameraCalibrationToolkitUtils
{
	const FName CameraCalibrationIdentifier(TEXT("CameraCalibrationTools"));
	const FName LensTabId(TEXT("LensFileEditorTab"));
	const FName LensEvaluationTabId(TEXT("LensEvaluationTab"));
	const FName CalibrationStepsTabId(TEXT("CalibrationStepsTab"));
}


TSharedRef<FCameraCalibrationToolkit> FCameraCalibrationToolkit::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULensFile* InLensFile)
{
	TSharedRef<FCameraCalibrationToolkit> NewEditor = MakeShared<FCameraCalibrationToolkit>();
	NewEditor->InitCameraCalibrationTool(Mode, InitToolkitHost, InLensFile);

	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LensModel"), InLensFile->LensInfo.LensModel ? InLensFile->LensInfo.LensModel->GetName() : TEXT("None")));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Usage.LensFile.EditorOpened"), EventAttributes);
	}

	return NewEditor;
}

void FCameraCalibrationToolkit::InitCameraCalibrationTool(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULensFile* InLensFile)
{
	LensFile = InLensFile;

	CalibrationStepsController = MakeShared<FCameraCalibrationStepsController>(SharedThis(this), InLensFile);
	check(CalibrationStepsController.IsValid());

	CalibrationStepsController->Initialize();

	LensEvaluationWidget = SNew(SLensEvaluation, CalibrationStepsController, InLensFile);
	CalibrationStepsTab = CalibrationStepsController->BuildUI();
	LensEditorTab = SNew(SLensFilePanel, LensFile, CalibrationStepsController.ToSharedRef())
		.CachedFIZData(TAttribute<FCachedFIZData>::Create(TAttribute<FCachedFIZData>::FGetter::CreateSP(LensEvaluationWidget.ToSharedRef(), &SLensEvaluation::GetLastEvaluatedData)));

	BindCommands();

	TSharedRef<FTabManager::FLayout> NewLayout = FTabManager::NewLayout("CameraCalibrationToolLayout_v0.9")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.85f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(CameraCalibrationToolkitUtils::CalibrationStepsTabId, ETabState::OpenedTab)
					->AddTab(CameraCalibrationToolkitUtils::LensTabId, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.15f)
				->SetHideTabWell(true)
				->AddTab(CameraCalibrationToolkitUtils::LensEvaluationTabId, ETabState::OpenedTab)
			)
		);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	constexpr bool bToolbarFocusable = false;
	constexpr bool bUseSmallIcons = true;
	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		CameraCalibrationToolkitUtils::CameraCalibrationIdentifier,
		NewLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		InLensFile,
		bToolbarFocusable,
		bUseSmallIcons);

	ExtendMenu();
	ExtendToolBar();
	RegenerateMenusAndToolbars();
}

TSharedPtr<SWindow> FCameraCalibrationToolkit::OpenPopupWindow(const FText& InTitle)
{
	TSharedPtr<SWindow> PopupWindowPin = PopupWindow.Pin();
	if (PopupWindowPin.IsValid())
	{
		PopupWindowPin->BringToFront();
	}
	else
	{
		PopupWindowPin = SNew(SWindow)
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.ClientSize(FVector2D(480, 720))
			.MinWidth(480)
			.MinHeight(720);

		FSlateApplication::Get().AddWindow(PopupWindowPin.ToSharedRef());
	}

	PopupWindow = PopupWindowPin;
	PopupWindowPin->SetTitle(InTitle);

	return PopupWindowPin;

}

void FCameraCalibrationToolkit::DestroyPopupWindow()
{
	TSharedPtr<SWindow> ExistingWindowPin = PopupWindow.Pin();
	if (ExistingWindowPin.IsValid())
	{
		ExistingWindowPin->RequestDestroyWindow();
		ExistingWindowPin = nullptr;
	}
}

void FCameraCalibrationToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CameraCalibrationTools", "Camera Calibration Panel"));

	Super::RegisterTabSpawners(InTabManager);
	
	InTabManager->RegisterTabSpawner(CameraCalibrationToolkitUtils::LensTabId, FOnSpawnTab::CreateSP(this, &FCameraCalibrationToolkit::HandleSpawnLensEditorTab))
		.SetDisplayName(LOCTEXT("LensEditorTab", "Lens File Editor"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings.Small"));

	InTabManager->RegisterTabSpawner(CameraCalibrationToolkitUtils::CalibrationStepsTabId, FOnSpawnTab::CreateSP(this, &FCameraCalibrationToolkit::HandleSpawnNodalOffsetTab))
		.SetDisplayName(LOCTEXT("CalibrationStepsTab", "Calibration Steps"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings.Small"));

	InTabManager->RegisterTabSpawner(CameraCalibrationToolkitUtils::LensEvaluationTabId, FOnSpawnTab::CreateSP(this, &FCameraCalibrationToolkit::HandleSpawnLensEvaluationTab))
		.SetDisplayName(LOCTEXT("LensEvaluationTab", "Lens Evaluation"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings.Small"));
}

void FCameraCalibrationToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(CameraCalibrationToolkitUtils::LensTabId);
	InTabManager->UnregisterTabSpawner(CameraCalibrationToolkitUtils::CalibrationStepsTabId);
	InTabManager->UnregisterTabSpawner(CameraCalibrationToolkitUtils::LensEvaluationTabId);
	Super::UnregisterTabSpawners(InTabManager);
}

bool FCameraCalibrationToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return true;
}

FText FCameraCalibrationToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("PanelToolkitName", "Camera Calibration Tools");
}

FName FCameraCalibrationToolkit::GetToolkitFName() const
{
	static const FName PanelName("CameraCalibrationTools");
	return PanelName;
}

FLinearColor FCameraCalibrationToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(1.0, 0.0f, 0.0f, 1.0f);
}

FString FCameraCalibrationToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("CameraCalibrationTabPrefix", "CameraCalibrationTools").ToString();
}

TSharedRef<SDockTab> FCameraCalibrationToolkit::HandleSpawnNodalOffsetTab(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == CameraCalibrationToolkitUtils::CalibrationStepsTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("CalibrationStepsPanel", "Calibration Steps"))
		.TabColorScale(GetTabColorScale())
		[
			CalibrationStepsTab.ToSharedRef()
		];
}

TSharedRef<SDockTab> FCameraCalibrationToolkit::HandleSpawnLensEditorTab(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == CameraCalibrationToolkitUtils::LensTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("LensFilePanelLabel", "Lens File Panel"))
		.TabColorScale(GetTabColorScale())
		[
			LensEditorTab.ToSharedRef()
		];
}

FCachedFIZData FCameraCalibrationToolkit::GetFIZData() const
{
	if (LensEvaluationWidget)
	{
		return LensEvaluationWidget->GetLastEvaluatedData();
	}

	return FCachedFIZData();
}

void FCameraCalibrationToolkit::BindCommands()
{
	const FCameraCalibrationCommands& Commands = FCameraCalibrationCommands::Get();

	ToolkitCommands->MapAction(
		Commands.ShowMediaPlaybackControls,
		FExecuteAction::CreateRaw(CalibrationStepsController.Get(), &FCameraCalibrationStepsController::ToggleShowMediaPlaybackControls),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(CalibrationStepsController.Get(), &FCameraCalibrationStepsController::AreMediaPlaybackControlsVisible));
}

void FCameraCalibrationToolkit::ExtendToolBar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, const TSharedRef<FUICommandList> ToolkitCommands)
		{
			ToolbarBuilder.BeginSection("PlaybackControls");
			{
				ToolbarBuilder.AddToolBarButton(FCameraCalibrationCommands::Get().ShowMediaPlaybackControls);
			}
			ToolbarBuilder.EndSection();
		}
	};


	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, GetToolkitCommands())
	);

	AddToolbarExtender(ToolbarExtender);
}

void FCameraCalibrationToolkit::ExtendMenu()
{
	MenuExtender = MakeShared<FExtender>();

	struct Local
	{
		static void ExtendMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("CameraCalibrationSettings", LOCTEXT("CameraCalibrationSettings", "Plugin Settings"));
			{
				const FUIAction OpenSettingsAction
				(
					FExecuteAction::CreateLambda([]()
					{
						FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Plugins", "Camera Calibration");
					})
				);
				
				const FUIAction OpenEditorSettingsAction
				(
					FExecuteAction::CreateLambda([]()
					{
						FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "Plugins", "Camera Calibration Editor");
					})
				);
				
				MenuBuilder.AddMenuEntry(
					LOCTEXT("OpenCameraCalibrationSettingsLabel", "Open Settings"),
					LOCTEXT("OpenCameraCalibrationSettingsTooltip", "Open Camera Calibration Settings"),
					FSlateIcon(),
					OpenSettingsAction);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("OpenCameraCalibrationEditorSettingsLabel", "Open Editor Settings"),
					LOCTEXT("OpenCameraCalibrationEditorSettingsTooltip", "Open Camera Calibration Editor Settings"),
					FSlateIcon(),
					OpenEditorSettingsAction);
			}
			MenuBuilder.EndSection();
		}
	};

	MenuExtender->AddMenuExtension(
		"EditHistory",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateStatic(&Local::ExtendMenu)
	);

	AddMenuExtender(MenuExtender);
}

TSharedRef<SDockTab> FCameraCalibrationToolkit::HandleSpawnLensEvaluationTab(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == CameraCalibrationToolkitUtils::LensEvaluationTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("LensEvaluationTabLabel", "Lens Evaluation"))
		.TabColorScale(GetTabColorScale())
		[
			LensEvaluationWidget.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE