// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

#include "CoreMinimal.h"
#include "LensFile.h"
#include "CameraCalibrationEditorCommon.h"


class IToolkitHost;
class SLensFilePanel;
class SNodalOffsetToolPanel;
class SLensEvaluation;
class ULensFile;
class FCameraCalibrationStepsController;
class SWindow;

/** Toolkit to do camera calibration */
class FCameraCalibrationToolkit : public FAssetEditorToolkit
{
	using Super = FAssetEditorToolkit;

public:
	/** Creates an editor for LensFile assets */
	static TSharedRef<FCameraCalibrationToolkit> CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULensFile* InLensFile);

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InLensFile				The LensFile asset to edit
	 */
	void InitCameraCalibrationTool(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULensFile* InLensFile);

	/**
	 * Creates Popup Toolkit Window or bring it to front if window does exists
	 *
	 * @param	InTitle	Window Title
	 */
	static TSharedPtr<SWindow> OpenPopupWindow(const FText& InTitle);

	/** Close the window and release the pointer */
	static void DestroyPopupWindow();

protected:

	//~ Begin IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	//~ End IToolkit interface

private:

	/** Handle spawning the tab that holds the LensFile panel tab. */
	TSharedRef<SDockTab> HandleSpawnLensEditorTab(const FSpawnTabArgs& Args) const;

	/** Handle spawning the tab that holds the NodalOffset panel tab. */
	TSharedRef<SDockTab> HandleSpawnNodalOffsetTab(const FSpawnTabArgs& Args) const;
	
	/** Handle spawning the tab that holds the current evaluated data */
	TSharedRef<SDockTab> HandleSpawnLensEvaluationTab(const FSpawnTabArgs& Args) const;

	FCachedFIZData GetFIZData() const;

	/** Binds the UI commands to delegates. */
	void BindCommands();

	/** Builds the toolbar widget for the camera calibration tools editor. */
	void ExtendToolBar();

	/** Extend Toolkit Menu */
	void ExtendMenu();

private:

	/** Toolkit is adding a reference to the editing object so a standard pointer is ok */
	ULensFile* LensFile = nullptr;

	/** Lens editor tab to do manual adjustments */
	TSharedPtr<SLensFilePanel> LensEditorTab;

	/** Camera calibration tools */
	TSharedPtr<SWidget> CalibrationStepsTab;

	/** Data displaying evaluated data based on evaluation inputs */
	TSharedPtr<SLensEvaluation> LensEvaluationWidget;

	/** Calibration Steps Controller */
	TSharedPtr<FCameraCalibrationStepsController> CalibrationStepsController;

	/** Singleton for the toolkit pop-up window. */
	static TWeakPtr<SWindow> PopupWindow;

	/** Menu extender */
	TSharedPtr<FExtender> MenuExtender;
};