// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDataCurveModel.h"

#include "LensFile.h"

#define LOCTEXT_NAMESPACE "LensDataCurveModel"


ECurveEditorViewID FLensDataCurveModel::ViewId = ECurveEditorViewID::Invalid;
FLensDataCurveModel::FLensDataCurveModel(ULensFile* InOwner)
	: FRichCurveEditorModel(InOwner)
	, LensFile(InOwner)
	, ClampOutputRange(TRange<double>(TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max()))
{
	check(InOwner);

	SupportedViews = ViewId;
}

void FLensDataCurveModel::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	//Don't support adding keys from curve editor by default. Specific models can override
}

void FLensDataCurveModel::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	//Don't support removing keys from curve editor by default. Specific models can override
}

void FLensDataCurveModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	// Re-implementation of FRichCurveEditorModel::SetKeyPosition that only changes the y-axis of the point being edited, as changing the x-axis (focus/zoom) is not supported
	if (IsReadOnly())
	{
		return;
	}

	if (UObject* Owner = GetOwningObject())
	{
		if (IsValid())
		{
			Owner->Modify();

			FRichCurve& RichCurve = GetRichCurve();
			for (int32 Index = 0; Index < InKeys.Num(); ++Index)
			{
				FKeyHandle Handle = InKeys[Index];
				if (RichCurve.IsKeyHandleValid(Handle))
				{
					const TRange<double> OutputRange = ClampOutputRange.Get();
					RichCurve.GetKey(Handle).Value = FMath::Clamp(InKeyPositions[Index].OutputValue, OutputRange.GetLowerBoundValue(), OutputRange.GetUpperBoundValue());
				}
			}
			
			RichCurve.AutoSetTangents();
			FPropertyChangedEvent PropertyChangeStruct(nullptr, ChangeType);
			Owner->PostEditChangeProperty(PropertyChangeStruct);

			CurveModifiedDelegate.Broadcast();
		}
	}
}

bool FLensDataCurveModel::IsValid() const
{
	return bIsCurveValid;
}

FRichCurve& FLensDataCurveModel::GetRichCurve()
{
	return CurrentCurve;
}

const FRichCurve& FLensDataCurveModel::GetReadOnlyRichCurve() const
{
	return CurrentCurve;
}

UObject* FLensDataCurveModel::GetOwningObject() const
{
	return LensFile.Get();
}

FText FLensDataCurveModel::GetKeyLabel() const
{
	return LOCTEXT("XAxisLabel", "Raw Zoom");
}

FText FLensDataCurveModel::GetValueLabel() const
{
	return FText();
}

FText FLensDataCurveModel::GetValueUnitPrefixLabel() const
{
	return FText();
}

FText FLensDataCurveModel::GetValueUnitSuffixLabel() const
{
	return FText();
}

#undef LOCTEXT_NAMESPACE
