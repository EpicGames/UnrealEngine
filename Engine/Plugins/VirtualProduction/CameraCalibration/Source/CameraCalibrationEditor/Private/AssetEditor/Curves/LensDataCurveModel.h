// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CurveModel.h"
#include "Curves/RichCurve.h"
#include "RichCurveEditorModel.h"
#include "UObject/StrongObjectPtr.h"


// Forward Declarations
class ULensFile;


/**
 * Base class to handle displaying curves for various lens data types
 */
class FLensDataCurveModel : public FRichCurveEditorModel
{
public:
	FLensDataCurveModel(ULensFile* InOwner);

	//~ Begin FRichCurveEditorModel
	virtual void AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override;
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override;
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType) override;
	virtual bool IsValid() const override;
	virtual FRichCurve& GetRichCurve() override;
	virtual const FRichCurve& GetReadOnlyRichCurve() const override;
	//~ End FRichCurveEditorModel

	//~ Begin FCurveModel
	virtual UObject* GetOwningObject() const override;
	//~ End FCurveModel

	virtual FText GetKeyLabel() const;
	virtual FText GetValueLabel() const;
	virtual FText GetValueUnitPrefixLabel() const;
	virtual FText GetValueUnitSuffixLabel() const;

public:
	/** View ID that identifies the registered view type */
	static ECurveEditorViewID ViewId;

protected:

	/** LensFile we are operating on */
	TStrongObjectPtr<ULensFile> LensFile;

	/** Active curve pointer */
	FRichCurve CurrentCurve;

	/** An optional clamp on the output values (y-axis) that the curve keys are allowed to have */
	TAttribute<TRange<double>> ClampOutputRange;
	
	/** Whether a valid curve was built from lens data */
	bool bIsCurveValid = false;
};
