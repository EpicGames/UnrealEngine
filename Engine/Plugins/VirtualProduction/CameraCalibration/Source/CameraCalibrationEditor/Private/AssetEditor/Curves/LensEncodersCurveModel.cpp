// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensEncodersCurveModel.h"

#include "LensFile.h"

#define LOCTEXT_NAMESPACE "LensEncodersCurveModel"


FLensEncodersCurveModel::FLensEncodersCurveModel(ULensFile* InOwner, EEncoderType InEncoderType)
	: FLensDataCurveModel(InOwner)
	, EncoderType(InEncoderType)
{
	check(InOwner);

	if (EncoderType == EEncoderType::Focus)
	{
		CurrentCurve = LensFile->EncodersTable.Focus;
	}
	else
	{
		CurrentCurve = LensFile->EncodersTable.Iris;
	}

	bIsCurveValid = true;

	OnCurveModified().AddRaw(this, &FLensEncodersCurveModel::OnCurveModifiedCallback);
}

void FLensEncodersCurveModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType /*= EPropertyChangeType::Unspecified*/)
{
	//Use modifications directly. We copy curve on changes below
	FRichCurveEditorModel::SetKeyAttributes(InKeys, InAttributes, ChangeType);
}

void FLensEncodersCurveModel::OnCurveModifiedCallback() const
{
	//Can definitely get optimized. This is a catch all to keep both curves aligned
	if (EncoderType == EEncoderType::Focus)
	{
		LensFile->EncodersTable.Focus = CurrentCurve;
	}
	else
	{
		LensFile->EncodersTable.Iris = CurrentCurve;
	}
}

FText FLensEncodersCurveModel::GetKeyLabel() const
{
	if (EncoderType == EEncoderType::Focus)
	{
		return LOCTEXT("KeyLabelFocus", "Raw Focus");
	}
	else if (EncoderType == EEncoderType::Iris)
	{
		return LOCTEXT("KeyLabelIris", "Raw Iris");
	}
	return FText();
}

FText FLensEncodersCurveModel::GetValueLabel() const
{
	if (EncoderType == EEncoderType::Focus)
	{
		return LOCTEXT("FocusValueLabel", "(cm)");
	}
	else if (EncoderType == EEncoderType::Iris)
	{
		return LOCTEXT("IrisValueLabel", "(FStop)");
	}
	return FText();
}

FText FLensEncodersCurveModel::GetValueUnitPrefixLabel() const
{
	if (EncoderType == EEncoderType::Iris)
	{
		return LOCTEXT("UnitPrefixLabelIris", "f/");
	}
	return FText();
}

FText FLensEncodersCurveModel::GetValueUnitSuffixLabel() const
{
	if (EncoderType == EEncoderType::Focus)
	{
		return LOCTEXT("UnitSuffixLabelFocus", "cm");
	}
	return FText();
}

#undef LOCTEXT_NAMESPACE