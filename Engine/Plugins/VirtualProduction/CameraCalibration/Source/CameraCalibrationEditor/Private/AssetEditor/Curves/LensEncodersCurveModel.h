// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LensDataCurveModel.h"


// Forward Declarations
class ULensFile;

/** Type of supported encoders */
enum class EEncoderType : uint8
{
	Focus,
	Iris
};

/**
 * Handles curves associated to encoder mapping
 */
class FLensEncodersCurveModel : public FLensDataCurveModel
{
public:

	FLensEncodersCurveModel(ULensFile* InOwner, EEncoderType InType);

	//~ Begin FRichCurveEditorModel
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;
	//~ End FRichCurveEditorModel

	//~ Begin FLensDataCurveModel interface
	virtual FText GetKeyLabel() const override;
	virtual FText GetValueLabel() const override;
	virtual FText GetValueUnitPrefixLabel() const override;
	virtual FText GetValueUnitSuffixLabel() const override;
	//~ End FLensDataCurveModel interface
	
protected:
	void OnCurveModifiedCallback() const;

private:

	/** Type of encoder for which to display curve */
	EEncoderType EncoderType = EEncoderType::Focus;
};
