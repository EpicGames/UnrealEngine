// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensMultiAxisCurveModel.h"

#define LOCTEXT_NAMESPACE "FLensDataMultiAxisCurveModel"

FLensDataMultiAxisCurveModel::FLensDataMultiAxisCurveModel(ULensFile* InOwner, ELensDataCategory InCategory, ELensCurveAxis InCurveAxis, float InCurveValue, int32 InParameterIndex)
	: FLensDataCurveModel(InOwner)
	, Category(InCategory)
	, CurveAxis(InCurveAxis)
	, CurveValue(InCurveValue)
	, ParameterIndex(InParameterIndex)
{
	if (const FBaseLensTable* DataTable = LensFile->GetDataTable(Category))
	{
		if (CurveAxis == ELensCurveAxis::Zoom)
		{
			bIsCurveValid = DataTable->BuildParameterCurveAtFocus(InCurveValue, InParameterIndex, CurrentCurve);
		}
		else if (CurveAxis == ELensCurveAxis::Focus)
		{
			bIsCurveValid = DataTable->BuildParameterCurveAtZoom(InCurveValue, InParameterIndex, CurrentCurve);
		}

		ClampOutputRange = DataTable->GetCurveKeyPositionRange(InParameterIndex);
	}
}

void FLensDataMultiAxisCurveModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	if (FBaseLensTable* DataTable = LensFile->GetDataTable(Category))
	{
		if (!DataTable->CanEditCurveKeyPositions(ParameterIndex))
		{
			return;
		}

		FLensDataCurveModel::SetKeyPositions(InKeys, InKeyPositions, ChangeType);
		
		if (CurveAxis == ELensCurveAxis::Zoom)
		{
			DataTable->SetParameterCurveKeysAtFocus(CurveValue, ParameterIndex, CurrentCurve, InKeys);
		}
		else if (CurveAxis == ELensCurveAxis::Focus)
		{
			DataTable->SetParameterCurveKeysAtZoom(CurveValue, ParameterIndex, CurrentCurve, InKeys);
		}
	}
}

void FLensDataMultiAxisCurveModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
	if (FBaseLensTable* DataTable = LensFile->GetDataTable(Category))
	{
		if (!DataTable->CanEditCurveKeyAttributes(ParameterIndex))
		{
			return;
		}
			
		FLensDataCurveModel::SetKeyAttributes(InKeys, InAttributes, ChangeType);

		if (CurveAxis == ELensCurveAxis::Zoom)
		{
			DataTable->SetParameterCurveKeysAtFocus(CurveValue, ParameterIndex, CurrentCurve, InKeys);
		}
		else if (CurveAxis == ELensCurveAxis::Focus)
		{
			DataTable->SetParameterCurveKeysAtZoom(CurveValue, ParameterIndex, CurrentCurve, InKeys);
		}
	}
}

FText FLensDataMultiAxisCurveModel::GetKeyLabel() const
{
	if (CurveAxis == ELensCurveAxis::Zoom)
	{
		return LOCTEXT("ZoomAxisLabel", "Raw Zoom");
	}
	else
	{
		return LOCTEXT("FocusAxisLabel", "Raw Focus");
	}
}

FText FLensDataMultiAxisCurveModel::GetValueLabel() const
{
	if (const FBaseLensTable* DataTable = LensFile->GetDataTable(Category))
	{
		return DataTable->GetParameterValueLabel(ParameterIndex);
	}

	return FText();
}

FText FLensDataMultiAxisCurveModel::GetValueUnitSuffixLabel() const
{
	if (const FBaseLensTable* DataTable = LensFile->GetDataTable(Category))
	{
		return DataTable->GetParameterValueUnitLabel(ParameterIndex);
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE