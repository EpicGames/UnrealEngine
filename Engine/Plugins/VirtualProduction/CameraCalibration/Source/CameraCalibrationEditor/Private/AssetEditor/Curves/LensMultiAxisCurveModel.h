// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensDataCurveModel.h"
#include "LensFile.h"

enum class ELensCurveAxis : uint8
{
	Zoom,
	Focus
};

/**
 * Curve model for lens data tables that have multiple axes (e.g focus and zoom), which can generate curves for either axis
 */
class FLensDataMultiAxisCurveModel : public FLensDataCurveModel
{
public:
	FLensDataMultiAxisCurveModel(ULensFile* InOwner, ELensDataCategory InCategory, ELensCurveAxis InCurveAxis, float InCurveValue, int32 InParameterIndex);

	//~ Begin FRichCurveEditorModel interface
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType) override;
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType) override;
	virtual FText GetKeyLabel() const override;
	virtual FText GetValueLabel() const override;
	virtual FText GetValueUnitSuffixLabel() const override;
	//~ End FRichCurveEditorModel interface
	
private:
	/** The category of the data table being displayed in the curve */
	ELensDataCategory Category;
	
	/** The axis to use as the x-axis of the curve */
	ELensCurveAxis CurveAxis;

	/** The value to hold fixed on the axis that isn't being plotted */
	float CurveValue = 0.0;

	/** The index of the data table parameter whose values are plotted on the y-axis of the curve */
	int32 ParameterIndex = INDEX_NONE;
};
