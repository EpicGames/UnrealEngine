// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageCenterTool.h"

#include "CameraCalibrationStepsController.h"
#include "CameraCalibrationSubsystem.h"
#include "CameraImageCenterAlgo.h"
#include "Engine/Engine.h"
#include "LensDistortionTool.h"
#include "LensFile.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "SImageCenterToolPanel.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "ImageCenterTool"

void UImageCenterTool::Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController)
{
	CameraCalibrationStepsController = InCameraCalibrationStepController;

	UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	check(Subsystem);

	for (const FName& AlgoName : Subsystem->GetCameraImageCenterAlgos())
	{
		TSubclassOf<UCameraImageCenterAlgo> AlgoClass = Subsystem->GetCameraImageCenterAlgo(AlgoName);
		const UCameraImageCenterAlgo* Algo = CastChecked<UCameraImageCenterAlgo>(AlgoClass->GetDefaultObject());

		// If the algo uses an overlay material, create a new MID to use with that algo
		if (UMaterialInterface* OverlayMaterial = Algo->GetOverlayMaterial())
		{
			AlgoOverlayMIDs.Add(Algo->FriendlyName(), UMaterialInstanceDynamic::Create(OverlayMaterial, GetTransientPackage()));
		}
	}
}

void UImageCenterTool::Shutdown()
{
	if (CurrentAlgo)
	{
		CurrentAlgo->Shutdown();
		CurrentAlgo = nullptr;
	}
}

void UImageCenterTool::Tick(float DeltaTime)
{
	if (CurrentAlgo && CurrentAlgo->IsActive())
	{
		CurrentAlgo->Tick(DeltaTime);
	}
}

bool UImageCenterTool::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (IsActive() && CurrentAlgo && CurrentAlgo->IsActive())
	{
		return CurrentAlgo->OnViewportClicked(MyGeometry, MouseEvent);
	}

	return false;
}

bool UImageCenterTool::OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent)
{
	if (IsActive() && CurrentAlgo && CurrentAlgo->IsActive())
	{
		return CurrentAlgo->OnViewportInputKey(InKey, InEvent);
	}

	return false;
}

TSharedRef<SWidget> UImageCenterTool::BuildUI()
{
	return SNew(SImageCenterToolPanel, this);
}

bool UImageCenterTool::DependsOnStep(UCameraCalibrationStep* Step) const
{
	return Cast<ULensDistortionTool>(Step) != nullptr;
}

void UImageCenterTool::Activate()
{
	bIsActive = true;

	if (CurrentAlgo && !CurrentAlgo->IsActive())
	{
		CurrentAlgo->Activate();
	}
}

void UImageCenterTool::Deactivate()
{
	bIsActive = false;

	if (CurrentAlgo && CurrentAlgo->IsActive())
	{
		CurrentAlgo->Deactivate();
	}
}

bool UImageCenterTool::IsActive() const
{
	return bIsActive;
}

FCameraCalibrationStepsController* UImageCenterTool::GetCameraCalibrationStepsController() const
{
	return CameraCalibrationStepsController.Pin().Get();
}

UCameraImageCenterAlgo* UImageCenterTool::GetAlgo() const
{
	return CurrentAlgo;
}

void UImageCenterTool::SetAlgo(const FName& AlgoName)
{
	// Query subsystem for the selected algorithm class
	UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	check(Subsystem);

	TSubclassOf<UCameraImageCenterAlgo> AlgoClass = Subsystem->GetCameraImageCenterAlgo(AlgoName);

	// If it is the same as the existing one, do nothing.
	if (!CurrentAlgo && !AlgoClass)
	{
		return;
	}
	
	if (CurrentAlgo && (CurrentAlgo->GetClass() == AlgoClass))
	{
		return;
	}

	// Remove old Algo
	if (CurrentAlgo)
	{
		CurrentAlgo->Shutdown();
		CurrentAlgo = nullptr;
	}

	// If AlgoClass is none, we're done here.
	if (!AlgoClass)
	{
		return;
	}

	// Create new algo
	CurrentAlgo = NewObject<UCameraImageCenterAlgo>(
		GetTransientPackage(),
		AlgoClass,
		MakeUniqueObjectName(GetTransientPackage(), AlgoClass), 
		RF_Transactional);

	if (CurrentAlgo)
	{
		CurrentAlgo->Initialize(this);
	}
}

void UImageCenterTool::OnSaveCurrentImageCenter()
{
	if (CurrentAlgo && CurrentAlgo->IsActive() && CurrentAlgo->HasImageCenterChanged())
	{
		CurrentAlgo->OnSavedImageCenter();
	}
}

UMaterialInstanceDynamic* UImageCenterTool::GetOverlayMID() const
{
	if (CurrentAlgo)
	{
		return AlgoOverlayMIDs.FindRef(CurrentAlgo->FriendlyName()).Get();
	}

	return nullptr;
}

bool UImageCenterTool::IsOverlayEnabled() const
{
	if (CurrentAlgo)
	{
		return CurrentAlgo->IsOverlayEnabled();
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
