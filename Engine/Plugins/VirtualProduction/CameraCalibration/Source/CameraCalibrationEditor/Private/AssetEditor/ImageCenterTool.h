// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibrationStep.h"

#include "ImageCenterTool.generated.h"

class UCameraImageCenterAlgo;

/**
 * UImageCenterTool is the controller for the image center panel.
 */
UCLASS()
class UImageCenterTool : public UCameraCalibrationStep
{
	GENERATED_BODY()

public:

	//~ Begin UCameraCalibrationStep interface
	virtual void Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController) override;
	virtual void Shutdown() override;
	virtual void Tick(float DeltaTime) override;
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const  override { return TEXT("Image Center"); };
	virtual bool DependsOnStep(UCameraCalibrationStep* Step) const override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual bool IsActive() const override;
	virtual UMaterialInstanceDynamic* GetOverlayMID() const override;
	virtual bool IsOverlayEnabled() const override;
	virtual FCameraCalibrationStepsController* GetCameraCalibrationStepsController() const override;
	//~ End UCameraCalibrationStep interface

	/** Returns the currently active algorithm */
	UCameraImageCenterAlgo* GetAlgo() const;

	/** Sets the active algorithm by name */
	void SetAlgo(const FName& AlgoName);

public:

	/** Called by the UI when the user wants to save the calibration data that the current algorithm is providing */
	void OnSaveCurrentImageCenter();

private:

	/** Pointer to the calibration steps controller */
	TWeakPtr<FCameraCalibrationStepsController> CameraCalibrationStepsController;

	/** The currently selected algorithm */
	UPROPERTY(Transient)
	TObjectPtr<UCameraImageCenterAlgo> CurrentAlgo;

	/** Map of algo names to overlay MIDs used by those algos */
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UMaterialInstanceDynamic>> AlgoOverlayMIDs;

	/** True if this tool is the active one in the panel */
	bool bIsActive = false;
};
