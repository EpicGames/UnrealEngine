// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDistortionTool.h"

#include "AssetToolsModule.h"
#include "CalibrationPointComponent.h"
#include "Camera/CameraActor.h"
#include "CameraCalibrationCharucoBoard.h"
#include "CameraCalibrationCheckerboard.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationStepsController.h"
#include "CameraCalibrationUtilsPrivate.h"
#include "DesktopPlatformModule.h"
#include "Dom/JsonObject.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IDesktopPlatform.h"
#include "ImageUtils.h"
#include "JsonObjectConverter.h"
#include "LensFile.h"
#include "LensInfoStep.h"
#include "Misc/DateTime.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "OpenCVHelper.h"
#include "ScopedTransaction.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SLensDistortionToolPanel.h"

#define LOCTEXT_NAMESPACE "LensDistortionTool"


// Console variable to control whether to generate object points in local space for Charuco calibration
static TAutoConsoleVariable<bool> CVarCharucoUseLocalSpaceObjectPoints(
	TEXT("CameraCalibration.Charuco.UseLocalSpaceObjectPoints"),
	true,
	TEXT("If true, generate object points in local space for Charuco calibration instead of using world space points.\n")
	TEXT("The solvers don't require object points in a canonical plane anymore, but can yield less accurate results otherwise.\n")
	TEXT("Default: true"),
	ECVF_Default
);


namespace UE::CameraCalibration::Private::LensDistortionTool
{
	static const FString SessionDateTimeField(TEXT("SessionDateTime"));
	static const FString Version(TEXT("Version"));
}

namespace UE::LensDistortionToolAnalytics
{
	void RecordEvent(FLensCaptureSettings CaptureSettings, FLensSolverSettings SolverSettings, int32 DatasetSize)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;

			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Pattern"), *UEnum::GetDisplayValueAsText(CaptureSettings.CalibrationPattern).ToString()));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsCameraTracked"), FString::Printf(TEXT("%d"), CaptureSettings.bIsCameraTracked ? 1 : 0)));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("IsCalibratorTracked"), FString::Printf(TEXT("%d"), CaptureSettings.bIsCalibratorTracked ? 1 : 0)));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("SolveNodalOffset"), FString::Printf(TEXT("%d"), SolverSettings.bSolveNodalOffset ? 1 : 0)));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DatasetSize"), FString::Printf(TEXT("%d"), DatasetSize)));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CameraCalibration.DistortionCalibrationStarted"), EventAttributes);
		}
	}
}

void ULensDistortionTool::Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController)
{
	WeakStepsController = InCameraCalibrationStepController;

	// Discover all available solver classes and select the first one as the solver to use
	TArray<UClass*> DerivedSolverClasses;
	GetDerivedClasses(ULensDistortionSolver::StaticClass(), DerivedSolverClasses);

	if (DerivedSolverClasses.Num() > 0)
	{
		SolverSettings.SolverClass = DerivedSolverClasses[0];
	}

	// Find all actors in the current level that have calibration point components and select the first one as the starting calibrator 
	TArray<AActor*> CalibratorActors;
	UE::CameraCalibration::Private::FindActorsWithCalibrationComponents(CalibratorActors);

	if (CalibratorActors.Num() > 0)
	{
		SetCalibrator(CalibratorActors[0]);
	}

	// Initialize the overlay material and texture
	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin())
	{
		UMaterialInterface* OverlayParent = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibration/Materials/M_Coverage.M_Coverage"))).LoadSynchronous();
		OverlayMID = UMaterialInstanceDynamic::Create(OverlayParent, GetTransientPackage());

		FIntPoint OverlayResolution = StepsController->GetMediaOverlayResolution();
		if (OverlayResolution == FIntPoint::ZeroValue)
		{
			// If the media overlay hasn't been initialized yet, it could return a zero size, in which case we will use a default size to initialize the texture
			static const FIntPoint DefaultSize(1920, 1080);
			OverlayResolution = DefaultSize;
		}
		
		OverlayTexture = UTexture2D::CreateTransient(OverlayResolution.X, OverlayResolution.Y, EPixelFormat::PF_B8G8R8A8);
		UE::CameraCalibration::Private::ClearTexture(OverlayTexture);

		OverlayMID->SetTextureParameterValue(FName(TEXT("CoverageTexture")), OverlayTexture);

		StepsController->SetOverlayMaterial(OverlayMID);
	}
}

void ULensDistortionTool::Shutdown()
{
	if (CalibrationTask.IsValid())
	{
		CancelCalibration();
		CalibrationTask = {};
		DistortionWidget->Shutdown();
	}

	EndCalibrationSession();
}

TSharedRef<SWidget> ULensDistortionTool::BuildUI()
{
	DistortionWidget = SNew(SLensDistortionToolPanel, this, WeakStepsController);
	return DistortionWidget.ToSharedRef();
}

void ULensDistortionTool::Tick(float DeltaTime)
{
	// If the resolution of the media overlay texture has changed, update the coverage texture to be the correct size
	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin())
	{
		const FIntPoint MediaOverlayResolution = StepsController->GetMediaOverlayResolution();
		const FIntPoint TextureResolution = OverlayTexture ? FIntPoint(OverlayTexture->GetSizeX(), OverlayTexture->GetSizeY()) : FIntPoint::ZeroValue;
		if (MediaOverlayResolution != TextureResolution)
		{
			OverlayTexture = UTexture2D::CreateTransient(MediaOverlayResolution.X, MediaOverlayResolution.Y, EPixelFormat::PF_B8G8R8A8);
			RefreshCoverage();
		}
	}

	// A valid task handle implies that there is an asynchronous calibration happening on another thread.
	if (CalibrationTask.IsValid())
	{
		if (CalibrationTask.IsCompleted())
		{
			// Extract the return value from the task and release the task resource
			CalibrationResult = CalibrationTask.GetResult();
			CalibrationTask = {};

			FText TaskCompletionText;
			if (CalibrationResult.ErrorMessage.IsEmpty())
			{
				FNumberFormattingOptions Options;
				Options.MinimumFractionalDigits = 3;
				Options.MaximumFractionalDigits = 3;

				TaskCompletionText = FText::Format(LOCTEXT("CalibrationResultReprojectionError", "Reprojection Error: {0} pixels"), FText::AsNumber(CalibrationResult.ReprojectionError, &Options));
			}
			else
			{
				TaskCompletionText = FText::Format(LOCTEXT("CalibrationResultErrorMessage", "Calibration Error: {0}"), CalibrationResult.ErrorMessage);
			}

			DistortionWidget->UpdateProgressText(TaskCompletionText);
			DistortionWidget->MarkProgressFinished();
		}
		else
		{
			// Update the calibration status in the progress window
			FText StatusText = FText::GetEmpty();
			const bool bIsStatusNew = GetCalibrationStatus(StatusText);

			if (bIsStatusNew)
			{
				DistortionWidget->UpdateProgressText(StatusText);
			}
		}
	}
}

bool ULensDistortionTool::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We only respond to left clicks
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return false;
	}

	return CaptureCalibrationData(MyGeometry, MouseEvent);
}

bool ULensDistortionTool::OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition)
{
	// Marquee select supports ROI for checkerboard and Charuco detection
	if (CaptureSettings.CalibrationPattern != ECalibrationPattern::Checkerboard && 
	    CaptureSettings.CalibrationPattern != ECalibrationPattern::Charuco)
	{
		return false;
	}

	// Normalize the marquee selection to ensure Min <= Max for both axes
	FIntRect MarqueeSelection;
	MarqueeSelection.Min = FIntPoint(
		FMath::Min(FMath::Floor(StartPosition.X), FMath::Floor(EndPosition.X)),
		FMath::Min(FMath::Floor(StartPosition.Y), FMath::Floor(EndPosition.Y))
	);
	MarqueeSelection.Max = FIntPoint(
		FMath::Max(FMath::Floor(StartPosition.X), FMath::Floor(EndPosition.X)),
		FMath::Max(FMath::Floor(StartPosition.Y), FMath::Floor(EndPosition.Y))
	);

	const FGeometry MyGeometryUnused;
	const FPointerEvent MouseEventUnused;

	return CaptureCalibrationData(MyGeometryUnused, MouseEventUnused, MarqueeSelection);
}

bool ULensDistortionTool::CaptureCalibrationData(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FIntRect RegionOfInterest)
{
	// Block user interaction with the simulcam viewport while an async calibration task is executing
	if (CalibrationTask.IsValid())
	{
		return false;
	}

	TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin();
	if (!StepsController)
	{
		return false;
	}

	// Create new row of calibration data
	TSharedPtr<FCalibrationRow> NewRow = MakeShared<FCalibrationRow>();

	// If capturing a single point, add the clicked point to the new row
	if (CaptureSettings.CalibrationPattern == ECalibrationPattern::Points)
	{
		const bool bDetectionResult = DetectPoint(MyGeometry, MouseEvent, NewRow);
		if (!bDetectionResult)
		{
			return true; // Though unsuccessful, the user input was handled
		}
	}
	else
	{
		// If capturing a calibration pattern, read the media texture into CPU memory to send to one of the pattern detection algorithms
		FText OutErrorMessage;

		TArray<FColor> Pixels;
		FIntPoint ImageSize;
		if (!StepsController->ReadMediaPixels(Pixels, ImageSize, OutErrorMessage, ESimulcamViewportPortion::CameraFeed))
		{
			FMessageDialog::Open(EAppMsgType::Ok, OutErrorMessage, LOCTEXT("ErrorReadingMedia", "Media Error"));
			return true; // Though unsuccessful, the user input was handled
		}

		// Detect the selected pattern in the media image
		bool bDetectionResult = false;
		if (CaptureSettings.CalibrationPattern == ECalibrationPattern::Checkerboard)
		{
			// If no ROI was provided, use the full image size
			if (RegionOfInterest.IsEmpty())
			{
				RegionOfInterest = FIntRect(FIntPoint(0), ImageSize);
			}
			bDetectionResult = DetectCheckerboardPattern(Pixels, ImageSize, RegionOfInterest, NewRow);
		}
		else if (CaptureSettings.CalibrationPattern == ECalibrationPattern::Aruco)
		{
			bDetectionResult = DetectArucoPattern(Pixels, ImageSize, NewRow);
		}
		else if (CaptureSettings.CalibrationPattern == ECalibrationPattern::Charuco)
		{
			// If no ROI was provided, use the full image size
			if (RegionOfInterest.IsEmpty())
			{
				RegionOfInterest = FIntRect(FIntPoint(0), ImageSize);
			}
			bDetectionResult = DetectCharucoPattern(Pixels, ImageSize, RegionOfInterest, NewRow);
		}

		if (!bDetectionResult)
		{
			return true; // Though unsuccessful, the user input was handled
		}

		// Save an image view of the captured frame
		FImageView ImageView = FImageView(Pixels.GetData(), ImageSize.X, ImageSize.Y, ERawImageFormat::BGRA8);
		ImageView.CopyTo(NewRow->MediaImage);
	}

	NewRow->Index = AdvanceSessionRowIndex();
	NewRow->Pattern = CaptureSettings.CalibrationPattern;

	NewRow->CameraPose = FTransform::Identity;
	if (const ACameraActor* Camera = StepsController->GetCamera())
	{
		if (const UCameraComponent* CameraComponent = Camera->GetCameraComponent())
		{
			NewRow->CameraPose = CameraComponent->GetComponentToWorld();
		}
	}

	Dataset.CalibrationRows.Add(NewRow);

	// Notify the ListView of the new data
	if (DistortionWidget)
	{
		DistortionWidget->RefreshListView();
	}

	// Export the data for this row to a .json file on disk
	ExportCalibrationRow(NewRow);

	ExportSessionData();

	return true;
}

bool ULensDistortionTool::DetectCheckerboardPattern(TArray<FColor>& Pixels, FIntPoint Size, FIntRect RegionOfInterest, TSharedPtr<FCalibrationRow> OutRow)
{
	const FText ErrorTitle = LOCTEXT("CaptureError", "Capture Error");

	// The selected calibrator must be a checkerboard actor
	ACameraCalibrationCheckerboard* Checkerboard = Cast<ACameraCalibrationCheckerboard>(CaptureSettings.Calibrator.Get());
	if (!Checkerboard)
	{
		const FText ErrorMessage = LOCTEXT("CheckerboardActorRequiredError", "The selected calibrator must be actor must be a Camera Calibration Checkerboard actor.");
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	const FIntPoint CheckerboardDimensions = FIntPoint(Checkerboard->NumCornerCols, Checkerboard->NumCornerRows);
	OutRow->CheckerboardDimensions = CheckerboardDimensions;

	// Launch an async task to perform the opencv checkerboard detection to prevent the game thread from being blocked in the rare cases when detection takes a very long time
	UE::Tasks::TTask<TArray<FVector2f>> DetectionTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Pixels, Size, RegionOfInterest, CheckerboardDimensions]() mutable
	{
		TArray<FVector2f> Corners;
		FOpenCVHelper::IdentifyCheckerboard(Pixels, Size, RegionOfInterest, CheckerboardDimensions, Corners);
		return Corners;
	});

	TArray<FVector2f> DetectedCorners;
	bool bCornersFound = false;

	const FDateTime StartTime = FDateTime::Now();

	const float Timeout = GetDefault<UCameraCalibrationSettings>()->GetCheckerboardDetectionTimeout();

	// If the detection has not completed before a set timeout, abandon this task. The user will be informed of the detection failure.
	while ((FDateTime::Now() - StartTime).GetSeconds() < Timeout)
	{
		if (DetectionTask.IsValid() && DetectionTask.IsCompleted())
		{
			// Extract the return value from the task
			DetectedCorners = DetectionTask.GetResult();
			bCornersFound = DetectedCorners.Num() > 0;
			break;
		}
	}

	// Release the task resource handle
	DetectionTask = {};

	if (!bCornersFound || DetectedCorners.IsEmpty())
	{
		const FText ErrorMessage = FText::Format(LOCTEXT("NoCheckerboardError", "Failed to detect a {0}x{1} checkerboard in the image."), Checkerboard->NumCornerCols, Checkerboard->NumCornerRows);
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	for (const FVector2f& Corner : DetectedCorners)
	{
		OutRow->ImagePoints.Points.Add(FVector2D(Corner));
	}

	// Fill out the checkerboard's 3D points
	const FVector TopLeft = Checkerboard->TopLeft->GetComponentLocation();
	const FVector TopRight = Checkerboard->TopRight->GetComponentLocation();
	const FVector BottomLeft = Checkerboard->BottomLeft->GetComponentLocation();

	const FVector RightVector = TopRight - TopLeft;
	const FVector DownVector = BottomLeft - TopLeft;

	const float HorizontalStep = (Checkerboard->NumCornerCols > 1) ? (1.0f / (Checkerboard->NumCornerCols - 1)) : 0.0f;
	const float VerticalStep = (Checkerboard->NumCornerRows > 1) ? (1.0f / (Checkerboard->NumCornerRows - 1)) : 0.0f;

	for (int32 RowIdx = 0; RowIdx < Checkerboard->NumCornerRows; ++RowIdx)
	{
		for (int32 ColIdx = 0; ColIdx < Checkerboard->NumCornerCols; ++ColIdx)
		{
			const FVector PointLocation = TopLeft + (RightVector * ColIdx * HorizontalStep) + (DownVector * RowIdx * VerticalStep);
			OutRow->ObjectPoints.Points.Add(PointLocation);
		}
	}

	// Update the coverage overlay with the latest checkerboard corners
	TArray<FVector2D> CameraFeedAdjustedCorners = OutRow->ImagePoints.Points;
	const FIntPoint OverlayTextureSize = FIntPoint(OverlayTexture->GetSizeX(), OverlayTexture->GetSizeY());
	RescalePoints(CameraFeedAdjustedCorners, OverlayTextureSize, Size);

	FOpenCVHelper::DrawCheckerboardCorners(CameraFeedAdjustedCorners, OutRow->CheckerboardDimensions, OverlayTexture);

	OutRow->TargetPose.SetLocation(TopLeft);

	FRotator BoardRotation = FRotationMatrix::MakeFromYZ(TopRight - TopLeft, TopLeft - BottomLeft).Rotator();
	OutRow->TargetPose.SetRotation(BoardRotation.Quaternion());

	return true;
}

bool ULensDistortionTool::DetectArucoPattern(TArray<FColor>& Pixels, FIntPoint Size, TSharedPtr<FCalibrationRow> OutRow)
{
	const FText ErrorTitle = LOCTEXT("CaptureError", "Capture Error");

	// Detect the aruco dictionary to use by looking at the names of the calibration points on the selected calibrator actor
	AActor* CalibratorActor = CaptureSettings.Calibrator.Get();
	if (!CalibratorActor)
	{
		const FText ErrorMessage = LOCTEXT("NoCalibratorError", "Please select a valid calibrator actor.");
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	EArucoDictionary Dictionary = UE::CameraCalibration::Private::GetArucoDictionaryForCalibrator(CalibratorActor);
	if (Dictionary == EArucoDictionary::None)
	{
		const FText ErrorMessage = LOCTEXT("NoArucoDictionaryError", "The calibration components of the selected calibrator do not specify a valid Aruco dictionary.");
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	// Identify any aruco markers matching the current dictionary in the media image
	TArray<FArucoMarker> IdentifiedMarkers;
	bool bResult = FOpenCVHelper::IdentifyArucoMarkers(Pixels, Size, Dictionary, IdentifiedMarkers);

	if (!bResult || IdentifiedMarkers.IsEmpty())
	{
		const FText ErrorMessage = LOCTEXT("NoArucoMarkersFoundError", "Failed to detect any aruco markers in the image belonging to the dictionary of the selected calibrator.");
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	// For each identified marker, search the calibration components to find the subpoints matching that marker and the 3D location of each of its corners
	TArray<FArucoCalibrationPoint> ArucoCalibrationPoints;
	ArucoCalibrationPoints.Reserve(IdentifiedMarkers.Num());

	UpdateCalibrationComponents();

	for (const FArucoMarker& Marker : IdentifiedMarkers)
	{
		FArucoCalibrationPoint ArucoCalibrationPoint;
		if (UE::CameraCalibration::Private::FindArucoCalibrationPoint(CalibrationComponents, Dictionary, Marker, ArucoCalibrationPoint))
		{
			ArucoCalibrationPoints.Add(ArucoCalibrationPoint);
		}
	}

	if (ArucoCalibrationPoints.Num() > 0)
	{
		for (const FArucoCalibrationPoint& Marker : ArucoCalibrationPoints)
		{
			for (int32 CornerIndex = 0; CornerIndex < 4; ++CornerIndex)
			{
				OutRow->ObjectPoints.Points.Add(Marker.Corners3D[CornerIndex]);
				OutRow->ImagePoints.Points.Add(FVector2D(Marker.Corners2D[CornerIndex]));
			}
		}

		FArucoCalibrationPoint& FirstAruco = ArucoCalibrationPoints[0];

		FVector TopLeft = FirstAruco.Corners3D[0];
		FVector TopRight = FirstAruco.Corners3D[1];
		FVector BottomLeft = FirstAruco.Corners3D[3];

		OutRow->TargetPose.SetLocation(TopLeft);

		FRotator FirstMarkerRotation = FRotationMatrix::MakeFromYZ(TopRight - TopLeft, TopLeft - BottomLeft).Rotator();
		OutRow->TargetPose.SetRotation(FirstMarkerRotation.Quaternion());
	}

	FOpenCVHelper::DrawArucoMarkers(IdentifiedMarkers, OverlayTexture);

	return true;
}

bool ULensDistortionTool::DetectCharucoPattern(TArray<FColor>& Pixels, FIntPoint Size, FIntRect RegionOfInterest, TSharedPtr<FCalibrationRow> OutRow)
{
	const FText ErrorTitle = LOCTEXT("CaptureError", "Capture Error");

	AActor* CalibratorActor = CaptureSettings.Calibrator.Get();
	if (!CalibratorActor)
	{
		const FText ErrorMessage = LOCTEXT("NoCalibratorError", "Please select a valid calibrator actor.");
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	ACameraCalibrationCharucoBoard* CharucoBoard = Cast<ACameraCalibrationCharucoBoard>(CalibratorActor);
	if (!CharucoBoard)
	{
		const FText ErrorMessage = LOCTEXT("NoCharucoBoardError", "The selected calibrator is not a Charuco board.");
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	// Setup board configuration

	FCharucoBoardConfig BoardConfig;

	BoardConfig.SquaresX = CharucoBoard->NumCornerCols + 1;
	BoardConfig.SquaresY = CharucoBoard->NumCornerRows + 1;
	BoardConfig.SquareSize = CharucoBoard->SquareSideLength;
	BoardConfig.MarkerSize = CharucoBoard->SquareSideLength * CharucoBoard->MarkerSizeRatio;
	BoardConfig.Dictionary = UE::CameraCalibration::Private::ShadowToOpenCV(CharucoBoard->ArucoDictionary);

	// Handle ROI by cropping the image (if needed)

	TArray<FColor> ProcessedPixels;
	FIntPoint ProcessedSize;
	FVector2D ROIOffset(0, 0);
	
	if (!RegionOfInterest.IsEmpty() && RegionOfInterest != FIntRect(FIntPoint(0), Size))
	{
		// Crop the image to the ROI

		const int32 ROIWidth = RegionOfInterest.Width();
		const int32 ROIHeight = RegionOfInterest.Height();

		ProcessedPixels.Reserve(ROIWidth * ROIHeight);
		
		for (int32 Y = RegionOfInterest.Min.Y; Y < RegionOfInterest.Max.Y; ++Y)
		{
			for (int32 X = RegionOfInterest.Min.X; X < RegionOfInterest.Max.X; ++X)
			{
				if (X >= 0 && X < Size.X && Y >= 0 && Y < Size.Y)
				{
					ProcessedPixels.Add(Pixels[Y * Size.X + X]);
				}
				else
				{
					ProcessedPixels.Add(FColor::Black);
				}
			}
		}
		
		ProcessedSize = FIntPoint(ROIWidth, ROIHeight);
		ROIOffset = FVector2D(RegionOfInterest.Min.X, RegionOfInterest.Min.Y);
	}
	else
	{
		// Use the full image
		ProcessedPixels = Pixels;
		ProcessedSize = Size;
	}

	// Detect Charuco corners
	FCharucoCorners DetectedCorners;
	bool bResult = FOpenCVHelper::IdentifyCharucoCorners(ProcessedPixels, ProcessedSize, BoardConfig, DetectedCorners);
	
	// Adjust detected coordinates by ROI offset
	if (bResult && ROIOffset != FVector2D::ZeroVector)
	{
		for (FVector2f& Corner : DetectedCorners.Corners)
		{
			Corner.X += ROIOffset.X;
			Corner.Y += ROIOffset.Y;
		}
		for (FArucoMarker& Marker : DetectedCorners.Markers)
		{
			for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
			{
				Marker.Corners[CornerIdx].X += ROIOffset.X;
				Marker.Corners[CornerIdx].Y += ROIOffset.Y;
			}
		}
	}

	if (!bResult || DetectedCorners.Corners.IsEmpty())
	{
		const FText ErrorMessage = FText::FromString(FString::Printf(TEXT(
			"Failed to detect Charuco corners in the image. "
			"Make sure the board is clearly visible and well-lit. "
			"Expected board: %dx%d squares with ArUco dictionary %s"),
			BoardConfig.SquaresX, BoardConfig.SquaresY,
			*UEnum::GetValueAsString(BoardConfig.Dictionary)
		));

		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);

		return false;
	}

	// Convert detected corners to calibration row format
	//

	// Corners and CornerIds arrays must have the same size
	if (DetectedCorners.Corners.Num() != DetectedCorners.CornerIds.Num())
	{
		const FText ErrorMessage = FText::Format(LOCTEXT("CornerDataMismatchError", 
			"Charuco detection data is corrupted: {0} corners but {1} corner IDs. Data integrity check failed."),
			DetectedCorners.Corners.Num(), DetectedCorners.CornerIds.Num());
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, ErrorMessage, ErrorTitle);
		return false;
	}

	for (int32 CornerIdx = 0; CornerIdx < DetectedCorners.Corners.Num(); ++CornerIdx)
	{
		int32 CornerId = DetectedCorners.CornerIds[CornerIdx];
		FVector2f ImageCorner = DetectedCorners.Corners[CornerIdx];
		
		// Get the 3D position of this corner from the board
		FVector WorldCorner = CharucoBoard->GetCornerPosition(CornerId);
				
		OutRow->ObjectPoints.Points.Add(WorldCorner);
		OutRow->ImagePoints.Points.Add(FVector2D(ImageCorner.X, ImageCorner.Y));
	}

	// Set target pose from the board transform
	OutRow->TargetPose = CharucoBoard->GetActorTransform();

	// Rescale corners to match overlay texture size before drawing
	if (OverlayTexture)
	{
		const FIntPoint OverlayTextureSize = FIntPoint(OverlayTexture->GetSizeX(), OverlayTexture->GetSizeY());
		const FVector2D TopLeftCorner = (OverlayTextureSize - Size) / 2.0f;
		
		// Create a copy of detected corners with rescaled positions
		FCharucoCorners RescaledCorners = DetectedCorners;
		for (FVector2f& Corner : RescaledCorners.Corners)
		{
			Corner.X += TopLeftCorner.X;
			Corner.Y += TopLeftCorner.Y;
		}
		for (FArucoMarker& Marker : RescaledCorners.Markers)
		{
			for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
			{
				Marker.Corners[CornerIdx].X += TopLeftCorner.X;
				Marker.Corners[CornerIdx].Y += TopLeftCorner.Y;
			}
		}
		
		// Draw the rescaled corners on overlay
		FOpenCVHelper::DrawCharucoCorners(RescaledCorners, OverlayTexture);
	}

	return true;
}

bool ULensDistortionTool::DetectPoint(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSharedPtr<FCalibrationRow> OutRow)
{
	TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin();
	if (!StepsController)
	{
		return false;
	}

	// When the user captures the first calibration point, the 3D locations for the remaining points are cached to ensure they all come from the same frame of data
	// The media is also paused to ensure that the user is able to more easily capture the 2D location of each subsequent calibration point
	UpdateCalibrationComponents();
	
	if (CalibrationComponentIndex == 0)
	{
		StepsController->Pause();

		// Cache the camera pose and component locations of the remaining components
		CachedComponentLocations.Reset();
		for (TWeakObjectPtr<UCalibrationPointComponent> Component : CalibrationComponents)
		{
			CachedComponentLocations.Add(Component->GetComponentLocation());
		}
	}

	// Calculate the location where the user clicked in the viewport
	FVector2f NormalizedClickPosition;
	if (!StepsController->CalculateNormalizedMouseClickPosition(MyGeometry, MouseEvent, NormalizedClickPosition, ESimulcamViewportPortion::CameraFeed))
	{
		return false;
	}

	const FIntPoint ImageSize = StepsController->GetCameraFeedSize();
	const FVector2D ImagePoint = FVector2D(NormalizedClickPosition * ImageSize);

	OutRow->ImagePoints.Points.Add(ImagePoint);

	OutRow->ObjectPoints.Points.Add(CachedComponentLocations[CalibrationComponentIndex]);

	// Advance the component index, and if it loops around, resume playing the media (which was paused after capturing the first point)
	SetComponentIndex(CalibrationComponentIndex + 1);

	if (CalibrationComponentIndex == 0)
	{
		StepsController->Play();
	}

	return true;
}

void ULensDistortionTool::CalibrateLens()
{
	TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin();
	if (!StepsController)
	{
		return;
	}

	ULensFile* LensFile = StepsController->GetLensFile();
	if (!LensFile)
	{
		return;
	}

	const FText TitleError = LOCTEXT("CalibrationErrorTitle", "Calibration Error");

	if (Dataset.CalibrationRows.Num() < 1)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("EmptyDatasetError", "The calibration dataset is empty. Please select a valid dataset or capture a new one before calibrating."), TitleError);
		return;
	}

	const float PhysicalSensorWidth = StepsController->GetLensFileEvaluationInputs().Filmback.SensorWidth;
	const float PixelAspect = LensFile->LensInfo.SqueezeFactor;

	const float DesqueezedSensorWidth = PhysicalSensorWidth * PixelAspect;

	if (FMath::IsNearlyZero(DesqueezedSensorWidth))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("InvalidSensorWidthError", "The sensor width and squeeze factor in the camera settings must both be greater than zero. Please enter a valid value."), TitleError);
		return;
	}

	if (!SolverSettings.FocalLengthGuess.IsSet() || FMath::IsNearlyZero(SolverSettings.FocalLengthGuess.GetValue()))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("InvalidFocalLengthError", "Please enter a valid estimate for the focal length of the lens (in mm)."), TitleError);
		return;
	}

	UE::LensDistortionToolAnalytics::RecordEvent(CaptureSettings, SolverSettings, Dataset.CalibrationRows.Num());

	const FIntPoint ImageSize = LensFile->CameraFeedInfo.GetDimensions();

	const double FocalLengthEstimateValue = SolverSettings.FocalLengthGuess.GetValue();
	const double Fx = (FocalLengthEstimateValue / DesqueezedSensorWidth) * ImageSize.X;

	// When operating on a desqueezed image, we expect our pixel aspect to be square, so horizontal and vertical field of view are assumed to be equal (i.e. Fx == Fy)
	FVector2D FocalLength = FVector2D(Fx);
	FVector2D ImageCenter = FVector2D((ImageSize.X - 1) * 0.5, (ImageSize.Y - 1) * 0.5);

	TArray<FObjectPoints> Samples3d;
	Samples3d.Reserve(Dataset.CalibrationRows.Num());

	TArray<FImagePoints> Samples2d;
	Samples2d.Reserve(Dataset.CalibrationRows.Num());

	TArray<FTransform> CameraPoses;
	CameraPoses.Reserve(Dataset.CalibrationRows.Num());

	TArray<FTransform> TargetPoses;
	TargetPoses.Reserve(Dataset.CalibrationRows.Num());

	// Extract the 3D points, 2D points, and camera poses from each row to pass to the solver
	for (const TSharedPtr<FCalibrationRow>& Row : Dataset.CalibrationRows)
	{
		Samples3d.Add(Row->ObjectPoints);
		Samples2d.Add(Row->ImagePoints);
		CameraPoses.Add(Row->CameraPose);
		TargetPoses.Add(Row->TargetPose);
	}

	ECalibrationFlags SolverFlags = ECalibrationFlags::None;
	EnumAddFlags(SolverFlags, ECalibrationFlags::UseIntrinsicGuess);

	if (CaptureSettings.bIsCameraTracked)
	{
		EnumAddFlags(SolverFlags, ECalibrationFlags::UseExtrinsicGuess);
	}
	else if (CaptureSettings.CalibrationPattern == ECalibrationPattern::Checkerboard)
	{
		GenerateDummyCheckerboardPoints(Samples3d, Dataset.CalibrationRows.Num(), Dataset.CalibrationRows[0]->CheckerboardDimensions);
	}
	else if (CaptureSettings.CalibrationPattern == ECalibrationPattern::Charuco)
	{
		// The solvers don't require object points in a canonical plane anymore, but can yield less accurate results otherwise.
		if (CVarCharucoUseLocalSpaceObjectPoints.GetValueOnGameThread())
		{
			GenerateObjectPointsInLocalSpace(Samples3d);
		}
	}

	if (CaptureSettings.bIsCalibratorTracked && CaptureSettings.CalibrationPattern != ECalibrationPattern::Points)
	{
		EnumAddFlags(SolverFlags, ECalibrationFlags::SolveTargetOffset);
	}

	if (CaptureSettings.bIsCameraTracked && CaptureSettings.bIsCalibratorTracked)
	{
		EnumAddFlags(SolverFlags, ECalibrationFlags::GroupCameraPoses);
	}

	if (SolverSettings.bFixFocalLength)
	{
		EnumAddFlags(SolverFlags, ECalibrationFlags::FixFocalLength);
	}

	if (SolverSettings.bFixImageCenter)
	{
		EnumAddFlags(SolverFlags, ECalibrationFlags::FixPrincipalPoint);
	}

	if (SolverSettings.bFixDistortion)
	{
		EnumAddFlags(SolverFlags, ECalibrationFlags::FixDistortion);
	}

	const FLensFileEvaluationInputs LensFileEvalInputs = StepsController->GetLensFileEvaluationInputs();

	FDistortionInfo DistortionGuess;
	LensFile->EvaluateDistortionParameters(LensFileEvalInputs.Focus, LensFileEvalInputs.Zoom, DistortionGuess);

	const TSubclassOf<ULensModel> Model = LensFile->LensInfo.LensModel;

	Solver = NewObject<ULensDistortionSolver>(GetTransientPackage(), SolverSettings.SolverClass);

	CalibrationTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [InSolver = this->Solver, Model, Samples3d, Samples2d, ImageSize, FocalLength, ImageCenter, DistortionGuess, CameraPoses, TargetPoses, PixelAspect, SolverFlags, LensFileEvalInputs]() mutable
		{
			FDistortionCalibrationResult Result;
			
			try
			{
				Result = InSolver->Solve(
					Samples3d,
					Samples2d,
					ImageSize,
					FocalLength,
					ImageCenter,
					DistortionGuess.Parameters,
					CameraPoses,
					TargetPoses,
					Model,
					PixelAspect,
					SolverFlags
				);

				// CalibrateCamera() returns focal length and image center in pixels, but the result is expected to be normalized by the image size
				Result.FocalLength.FxFy = Result.FocalLength.FxFy / ImageSize;
				Result.ImageCenter.PrincipalPoint = Result.ImageCenter.PrincipalPoint / ImageSize;

				// FZ inputs to LUT
				Result.EvaluatedFocus = LensFileEvalInputs.Focus;
				Result.EvaluatedZoom = LensFileEvalInputs.Zoom;
			}
			catch (const std::exception& Exception)
			{
				Result.ErrorMessage = FText::FromString(FString::Printf(TEXT("Calibration solver reported the error: %s"), ANSI_TO_TCHAR(Exception.what())));
				UE_LOGF(LogCameraCalibrationEditor, Error, "LensDistortionTool: Solver crashed with std::exception: %ls", ANSI_TO_TCHAR(Exception.what()));
			}
			catch (...)
			{
				Result.ErrorMessage = FText::FromString(TEXT("Calibration solver returned with an unspecified error"));
				UE_LOGF(LogCameraCalibrationEditor, Error, "LensDistortionTool: Solver crashed with unknown exception");
			}

			return Result;
		});

	DistortionWidget->OpenProgressWindow();

	// All of the UI options should be disabled while the calibration task is running
	DistortionWidget->SetEnabled(false);
}

void ULensDistortionTool::SaveCalibrationResult()
{
	TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin();
	if (!StepsController)
	{
		return;
	}

	ULensFile* LensFile = StepsController->GetLensFile();
	if (!LensFile)
	{
		return;
	}

	// If the calibration result contains the name of an ST Map file on disk instead of a UTexture, then we attempt to import it for the user
	if (!CalibrationResult.STMap.DistortionMap && !CalibrationResult.STMapFullPath.IsEmpty())
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		TArray<FString> TextureFileNames;
		TextureFileNames.Add(CalibrationResult.STMapFullPath);
		TArray<UObject*> ImportedImages = AssetToolsModule.Get().ImportAssets(TextureFileNames, FPaths::ProjectContentDir());

		CalibrationResult.STMap.DistortionMap = (ImportedImages.Num() > 0) ? Cast<UTexture>(ImportedImages[0]) : nullptr;
	}

	// The result may have featured calibrated distortion parameters or an ST Map.
	// If the result contains any distortion parameters, then the results will be written as a distortion point in the Lens File
	// Otherwise, if the result contains a valid ST Map, then it will be added to the Lens File
	if (CalibrationResult.Parameters.Parameters.Num() > 0)
	{
		if (LensFile->DataMode != ELensDataMode::Parameters)
		{
			LensFile->DataMode = ELensDataMode::Parameters;
			UE_LOGF(LogCameraCalibrationEditor, Log, "The LensFile's data mode was set to ST Map, but the latest calibration result returned distortion parameters. Data mode will change to Parameters.");
		}

		FScopedTransaction Transaction(LOCTEXT("SaveCurrentDistortionData", "Save Calibrated Distortion to Lens Asset"));
		LensFile->Modify();

		LensFile->AddDistortionPoint(CalibrationResult.EvaluatedFocus, CalibrationResult.EvaluatedZoom, CalibrationResult.Parameters, CalibrationResult.FocalLength);
		LensFile->AddImageCenterPoint(CalibrationResult.EvaluatedFocus, CalibrationResult.EvaluatedZoom, CalibrationResult.ImageCenter);
	}
	else if (CalibrationResult.STMap.DistortionMap)
	{
		if (LensFile->DataMode != ELensDataMode::STMap)
		{
			LensFile->DataMode = ELensDataMode::STMap;
			UE_LOGF(LogCameraCalibrationEditor, Log, "The LensFile's data mode was set to Parameters, but the latest calibration result returned an ST Map. Data mode will change to ST Map.");
		}

		FScopedTransaction Transaction(LOCTEXT("SaveCurrentDistortionData", "Save Calibrated Distortion to Lens Asset"));
		LensFile->Modify();

		LensFile->AddSTMapPoint(CalibrationResult.EvaluatedFocus, CalibrationResult.EvaluatedZoom, CalibrationResult.STMap);
		LensFile->AddFocalLengthPoint(CalibrationResult.EvaluatedFocus, CalibrationResult.EvaluatedZoom, CalibrationResult.FocalLength);
		LensFile->AddImageCenterPoint(CalibrationResult.EvaluatedFocus, CalibrationResult.EvaluatedZoom, CalibrationResult.ImageCenter);
	}

	// If the calibration result contains a meaningful nodal offset result, add it to the Lens File
	if (SolverSettings.bSolveNodalOffset && CaptureSettings.bIsCalibratorTracked && CaptureSettings.bIsCameraTracked)
	{
		if (!CalibrationResult.NodalOffset.LocationOffset.Equals(FVector::ZeroVector) || !CalibrationResult.NodalOffset.RotationOffset.Equals(FQuat::Identity))
		{
			FScopedTransaction Transaction(LOCTEXT("SaveNodalOffsetResult", "Save Calibrated Nodal Offset to Lens Asset"));
			LensFile->Modify();

			LensFile->AddNodalOffsetPoint(CalibrationResult.EvaluatedFocus, CalibrationResult.EvaluatedZoom, CalibrationResult.NodalOffset);
		}
	}

	ClearCalibrationRows();
}

void ULensDistortionTool::CancelCalibration()
{
	if (Solver)
	{
		Solver->Cancel();
	}
}

bool ULensDistortionTool::GetCalibrationStatus(FText& StatusText) const
{
	if (Solver)
	{
		return Solver->GetStatusText(StatusText);
	}
	return false;
}

void ULensDistortionTool::SetCalibrator(AActor* InCalibrator)
{
	CaptureSettings.Calibrator = InCalibrator;

	if (!InCalibrator)
	{
		return;
	}

	CalibrationComponents.Reset();
	UpdateCalibrationComponents();

	// Initialize the component index used by the single point detection mode
	SetComponentIndex(0);

	CachedComponentLocations.Reserve(CalibrationComponents.Num());
}

void ULensDistortionTool::UpdateCalibrationComponents()
{
	bool bNeedsUpdate = false;

	for (TWeakObjectPtr<UCalibrationPointComponent> Component : CalibrationComponents)
	{
		if (!Component.IsValid())
		{
			bNeedsUpdate = true;
			break;
		}
	}

	if (CalibrationComponents.IsEmpty())
	{
		bNeedsUpdate = true;
	}

	if (bNeedsUpdate && CaptureSettings.Calibrator.IsValid())
	{
		// Find all of the calibration components attached to calibrator actor
		TArray<UCalibrationPointComponent*> CalibrationPoints;
		CaptureSettings.Calibrator->GetComponents(CalibrationPoints);

		// Store weak references to all of the calibration components that have an attached scene component
		CalibrationComponents.Reset();
		for (UCalibrationPointComponent* CalibrationPoint : CalibrationPoints)
		{
			if (CalibrationPoint && CalibrationPoint->GetAttachParent())
			{
				CalibrationComponents.Add(CalibrationPoint);
			}
		}
	}
}

void ULensDistortionTool::ClearCalibrationRows()
{
	Dataset.CalibrationRows.Empty();

	if (DistortionWidget)
	{
		DistortionWidget->RefreshListView();
	}

	// Reset the calibration component index to restart the pattern
	SetComponentIndex(0);

	RefreshCoverage();

	// End the current calibration session (a new one will begin the next time a new row is added)
	EndCalibrationSession();
}

void ULensDistortionTool::SetComponentIndex(int32 Index)
{
	CalibrationComponentIndex = Index;

	// If the Index would loop around, reset it to 0
	if (CalibrationComponentIndex == CalibrationComponents.Num())
	{
		CalibrationComponentIndex = 0;
	}

	UpdateCalibrationComponents();

	if (CalibrationComponents.IsValidIndex(CalibrationComponentIndex))
	{
		CaptureSettings.NextPoint = FText::FromString(CalibrationComponents[CalibrationComponentIndex]->GetName());
	}
}

void ULensDistortionTool::RefreshCoverage()
{
	UE::CameraCalibration::Private::ClearTexture(OverlayTexture);

	for (const TSharedPtr<FCalibrationRow>& Row : Dataset.CalibrationRows)
	{
		if (Row->Pattern == ECalibrationPattern::Checkerboard)
		{
			TArray<FVector2D> CameraFeedAdjustedCorners = Row->ImagePoints.Points;
			const FIntPoint OverlayTextureSize = FIntPoint(OverlayTexture->GetSizeX(), OverlayTexture->GetSizeY());
			const FIntPoint ImageSize = FIntPoint(Row->MediaImage.SizeX, Row->MediaImage.SizeY);
			RescalePoints(CameraFeedAdjustedCorners, OverlayTextureSize, ImageSize);

			FOpenCVHelper::DrawCheckerboardCorners(CameraFeedAdjustedCorners, Row->CheckerboardDimensions, OverlayTexture);
		}
	}

	// The coverage texture may have changed as a result of a change in size or pixel format.
	// Therefore, the material parameter should be updated to ensure it is up to date.
	if (OverlayTexture)
	{
		OverlayMID->SetTextureParameterValue(FName(TEXT("CoverageTexture")), OverlayTexture);
	}
}

void ULensDistortionTool::RescalePoints(TArray<FVector2D>& Points, FIntPoint DebugTextureSize, FIntPoint CameraFeedSize)
{
	// It is possible that the size of the debug texture is different than the size of the camera feed.
	// Therefore, the input points should be shifted so that they appear at the correct location in the debug image.
	const FVector2D TopLeftCorner = (DebugTextureSize - CameraFeedSize) / 2.0f;

	for (FVector2D& Point : Points)
	{
		Point += TopLeftCorner;
	}
}

void ULensDistortionTool::GenerateDummyCheckerboardPoints(TArray<FObjectPoints>& Samples3d, int32 NumImages, FIntPoint CheckerboardDimensions)
{
	// If the camera is not tracked, the distortion solver must initialize the camera pose for each image.
	// However, it may struggle when "real" tracking data is used for the calibrator. So in this case, we replace the tracked calibrator data 
	// with a set of dummy points for the 3D checkerboard corners. The board is assumed to lie in the YZ plane with the TopLeft corner at (0, 0, 0) in world space.

	Samples3d.Empty();
	for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		FObjectPoints Points3d;
		for (int32 RowIdx = 0; RowIdx < CheckerboardDimensions.Y; ++RowIdx)
		{
			for (int32 ColIdx = 0; ColIdx < CheckerboardDimensions.X; ++ColIdx)
			{
				Points3d.Points.Add(FVector(0, ColIdx, -RowIdx));
			}
		}

		Samples3d.Add(Points3d);
	}
}

void ULensDistortionTool::GenerateObjectPointsInLocalSpace(TArray<FObjectPoints>& Samples3d)
{
	Samples3d.Empty();
		
	for (const TSharedPtr<FCalibrationRow>& Row : Dataset.CalibrationRows)
	{
		FObjectPoints Points3d;
		
		// Get the inverse of the calibrator transform that was used for this capture
		const FTransform InverseTargetPose = Row->TargetPose.Inverse();
		
		for (int32 PointIdx = 0; PointIdx < Row->ObjectPoints.Points.Num(); ++PointIdx)
		{
			const FVector& WorldPoint = Row->ObjectPoints.Points[PointIdx];
			
			// Transform back to local space using the stored target pose
			const FVector LocalPoint = InverseTargetPose.TransformPosition(WorldPoint);
			
			Points3d.Points.Add(LocalPoint);
		}
		
		Samples3d.Add(Points3d);
	}
}

bool ULensDistortionTool::DependsOnStep(UCameraCalibrationStep* Step) const
{
	return Cast<ULensInfoStep>(Step) != nullptr;
}

void ULensDistortionTool::Activate()
{
	bIsActive = true;
}

void ULensDistortionTool::Deactivate()
{
	bIsActive = false;
}

bool ULensDistortionTool::IsActive() const
{
	return bIsActive;
}

UMaterialInstanceDynamic* ULensDistortionTool::GetOverlayMID() const
{
	return OverlayMID;
}

bool ULensDistortionTool::IsOverlayEnabled() const
{
	return CaptureSettings.bShowOverlay;
}

void ULensDistortionTool::StartCalibrationSession()
{
	if (!SessionInfo.bIsActive)
	{
		SessionInfo.bIsActive = true;
		SessionInfo.StartTime = FDateTime::Now();
	}
}

void ULensDistortionTool::EndCalibrationSession()
{
	if (SessionInfo.bIsActive)
	{
		SessionInfo.bIsActive = false;
		SessionInfo.RowIndex = -1;
	}
}

uint32 ULensDistortionTool::AdvanceSessionRowIndex()
{
	SessionInfo.RowIndex += 1;
	return SessionInfo.RowIndex;
}

FString ULensDistortionTool::GetSessionSaveDir() const
{
	using namespace UE::CameraCalibration::Private;

	const FString SessionDateString = SessionInfo.StartTime.ToString(TEXT("%Y-%m-%d"));
	const FString SessionTimeString = SessionInfo.StartTime.ToString(TEXT("%H-%M-%S"));
	const FString DatasetDir = SessionTimeString;

	const FString ProjectSaveDir = FPaths::ProjectSavedDir() / TEXT("CameraCalibration") / TEXT("LensDistortion");

	return ProjectSaveDir / SessionDateString / DatasetDir;
}

FString ULensDistortionTool::GetRowFilename(int32 RowIndex) const
{
	const FString RowNumString = TEXT("Row") + FString::FromInt(RowIndex) + TEXT("-");
	return RowNumString;
}

void ULensDistortionTool::DeleteExportedRow(const int32& RowIndex) const
{
	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	// Find all files in the directory of the currently active session
	const FString PathName = GetSessionSaveDir();
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *PathName);

	const FString RowNumString = GetRowFilename(RowIndex);

	// Delete any files containing that row number from the session directory
	for (const FString& File : FoundFiles)
	{
		if (File.Contains(RowNumString))
		{
			const FString FullPath = PathName / File;
			IFileManager::Get().Delete(*FullPath);
			UE_LOGF(LogCameraCalibrationEditor, Verbose, "Lens Distortion Tool removed calibration dataset file: %ls", *FullPath);
		}
	}
}

void ULensDistortionTool::ImportCalibrationDataset()
{
	using namespace UE::CameraCalibration::Private;

	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	// If there is existing calibration data that will be overwritten during import, ask the user to confirm that they want to continue
	if (Dataset.CalibrationRows.Num() > 0)
	{
		const FText ConfirmationMessage = LOCTEXT(
			"ImportDatasetConfirmationMessage",
			"There are existing calibration rows which will be removed during the import process. Do you want to proceed with the import?");

		if (FMessageDialog::Open(EAppMsgType::YesNo, ConfirmationMessage) == EAppReturnType::No)
		{
			return;
		}
	}

	// Open a file dialog to select a .ucamcalib session data file 
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
	const FString Title = TEXT("Import Camera Calibration Dataset");
	const FString DefaultPath = FPaths::ProjectSavedDir() / TEXT("CameraCalibration") / TEXT("LensDistortion");;
	const FString DefaultFile = TEXT("");
	const FString FileTypes = TEXT("Camera Calibration Dataset|*.ucamcalib");
	const uint32 OpenFileFlags = 0;

	// Note, OpenFileFlags is not set to "Multiple" so we only expect one file to be selected
	TArray<FString> SelectedFileNames;
	const bool bFileSelected = DesktopPlatform->OpenFileDialog(ParentWindowHandle, Title, DefaultPath, DefaultFile, FileTypes, OpenFileFlags, SelectedFileNames);

	// Early-out if no calibration file was selected
	if (!bFileSelected || SelectedFileNames.Num() < 1)
	{
		return;
	}

	// Parse the session data filename and the directory from the full path
	const FString SessionFileName = FPaths::GetCleanFilename(SelectedFileNames[0]);
	const FString SelectedDirectory = FPaths::GetPath(SelectedFileNames[0]);

	// Find all json files in the selected directory (this will not include the .ucamcalib session date file)
	TArray<FString> FoundFiles;
	const FString FileExtension = TEXT(".json");
	IFileManager::Get().FindFiles(FoundFiles, *SelectedDirectory, *FileExtension);

	// Early-out if selected directory has no json files to import
	if (FoundFiles.Num() < 1)
	{
		const FText ErrorMessage = LOCTEXT("NoJsonFilesFound", "The selected directory has no .json files to import.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		return;
	}

	// Import the session data
	FDateTime ImportedSessionDateTime = FDateTime::Now();
	EDatasetVersion ImportedDatasetVersion = EDatasetVersion::Invalid;
	{
		const FString SessionFile = SelectedDirectory / SessionFileName;

		// Open the Json file for reading, and initialize a JsonReader to parse the contents
		if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*SessionFile)))
		{
			TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());

			// Deserialize the row data from the Json file into a Json object
			TSharedPtr<FJsonObject> JsonSessionData = MakeShared<FJsonObject>();
			if (FJsonSerializer::Deserialize(JsonReader, JsonSessionData))
			{
				// Import the session date/time so that we can restore the imported session
				FString SessionDateTimeString;
				if (JsonSessionData->TryGetStringField(LensDistortionTool::SessionDateTimeField, SessionDateTimeString))
				{
					ensureMsgf(FDateTime::Parse(SessionDateTimeString, ImportedSessionDateTime), TEXT("Failed to parse imported session date and time"));
				}
				else
				{
					UE_LOGF(LogCameraCalibrationEditor, Verbose, "Lens Distortion Tool failed to deserialize the date/time from the session file: %ls", *SessionFile);
				}

				int32 Version = 0;
				if (JsonSessionData->TryGetNumberField(LensDistortionTool::Version, Version))
				{
					ImportedDatasetVersion = (EDatasetVersion)Version;
				}

				// This version predates the calibration pattern field, but the data can be reconstructed by looking at the "Algo Name" field.
				if (ImportedDatasetVersion == EDatasetVersion::SeparateAlgoClasses)
				{
					FString AlgoString;
					if (JsonSessionData->TryGetStringField(TEXT("AlgoName"), AlgoString))
					{
						if (AlgoString.Contains(TEXT("Checkerboard"), ESearchCase::IgnoreCase))
						{
							CaptureSettings.CalibrationPattern = ECalibrationPattern::Checkerboard;
						}
						else if (AlgoString.Contains(TEXT("Aruco"), ESearchCase::IgnoreCase))
						{
							CaptureSettings.CalibrationPattern = ECalibrationPattern::Aruco;
						}
						else if (AlgoString.Contains(TEXT("Points"), ESearchCase::IgnoreCase))
						{
							CaptureSettings.CalibrationPattern = ECalibrationPattern::Points;
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogCameraCalibrationEditor, Verbose, "Lens Distortion Tool failed to deserialize dataset session file: %ls", *SessionFile);
			}
		}
	}

	Dataset.CalibrationRows.Empty();

	// Initialize a maximum row index which will be used to set the current session row index if the user wants to add additional rows after importing
	int32 MaxRowIndex = -1;

	for (const FString& File : FoundFiles)
	{
		const FString JsonFileName = SelectedDirectory / File;
		const FString ImageFileName = JsonFileName.Replace(TEXT(".json"), TEXT(".png"));

		// Load the PNG image file for this row into an FImage
		FImage RowImage;
		if (IFileManager::Get().FileExists(*ImageFileName))
		{
			FImageUtils::LoadImage(*ImageFileName, RowImage);
		}

		// Open the Json file for reading, and initialize a JsonReader to parse the contents
		if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*JsonFileName)))
		{
			TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());

			// Deserialize the row data from the Json file into a Json object
			TSharedPtr<FJsonObject> JsonRowData = MakeShared<FJsonObject>();
			if (FJsonSerializer::Deserialize(JsonReader, JsonRowData))
			{
				int32 RowNum = ImportCalibrationRow(JsonRowData.ToSharedRef(), RowImage, ImportedDatasetVersion);
				MaxRowIndex = FMath::Max(MaxRowIndex, RowNum);
			}
			else
			{
				UE_LOGF(LogCameraCalibrationEditor, Verbose, "Lens Distortion Tool failed to deserialize the dataset row file: %ls", *JsonFileName);
			}
		}
	}

	// Sort imported calibration rows by row index
	Dataset.CalibrationRows.Sort([](const TSharedPtr<FCalibrationRow>& LHS, const TSharedPtr<FCalibrationRow>& RHS) { return LHS->Index < RHS->Index; });

	// Notify the ListView of the new data
	if (DistortionWidget)
	{
		DistortionWidget->RefreshListView();
	}

	// Redraw the coverage overlay for the imported data
	RefreshCoverage();

	// Set the current session's start date/time and row index to match what was just imported to support adding/deleting rows
	SessionInfo.bIsActive = true;
	SessionInfo.StartTime = ImportedSessionDateTime;
	SessionInfo.RowIndex = MaxRowIndex;
}

void ULensDistortionTool::ExportSessionData()
{
	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	using namespace UE::CameraCalibration::Private;

	TSharedPtr<FJsonObject> JsonSessionData = MakeShared<FJsonObject>();

	int32 DatasetVersion = (int32)EDatasetVersion::CurrentVersion;
	JsonSessionData->SetNumberField(LensDistortionTool::Version, DatasetVersion);

	// Start a calibration session (if one is not currently active)
	StartCalibrationSession();

	// Assemble the path and filename for this row based on the session and row index
	const FString PathName = GetSessionSaveDir();
	const FString FileName = TEXT("SessionData");

	const FString SessionFileName = PathName / FileName + TEXT(".ucamcalib");

	// Delete the existing session data file (if it exists)
	if (IFileManager::Get().FileExists(*SessionFileName))
	{
		IFileManager::Get().Delete(*SessionFileName);
	}

	// Create and open a new Json file for writing, and initialize a JsonWriter to serialize the contents
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*SessionFileName)))
	{
		TSharedRef< TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(FileWriter.Get());

		const FString SessionDateTimeString = SessionInfo.StartTime.ToString(TEXT("%Y-%m-%d")) + TEXT("-") + SessionInfo.StartTime.ToString(TEXT("%H-%M-%S"));

		JsonSessionData->SetStringField(LensDistortionTool::SessionDateTimeField, SessionDateTimeString);

		// Write the Json row data out and save the file
		FJsonSerializer::Serialize(JsonSessionData.ToSharedRef(), JsonWriter);
		FileWriter->Close();

		UE_LOGF(LogCameraCalibrationEditor, Verbose, "Lens Distortion Tool wrote to dataset session file: %ls", *SessionFileName);
	}
}

void ULensDistortionTool::ExportCalibrationRow(TSharedPtr<FCalibrationRow> Row)
{
	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	if (const TSharedPtr<FJsonObject>& RowObject = FJsonObjectConverter::UStructToJsonObject<FCalibrationRow>(Row.ToSharedRef().Get()))
	{
		// Start a calibration session (if one is not currently active)
		StartCalibrationSession();

		// Assemble the path and filename for this row based on the session and row index
		const FString PathName = GetSessionSaveDir();
		const FString FileName = GetRowFilename(Row->Index) + FDateTime::Now().ToString(TEXT("%H-%M-%S"));

		const FString JsonFileName = PathName / FileName + TEXT(".json");
		const FString ImageFileName = PathName / FileName + TEXT(".png");

		// Create and open a new Json file for writing, and initialize a JsonWriter to serialize the contents
		if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*JsonFileName)))
		{
			TSharedRef< TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(FileWriter.Get());

			// Write the Json row data out and save the file
			FJsonSerializer::Serialize(RowObject.ToSharedRef(), JsonWriter);
			FileWriter->Close();

			UE_LOGF(LogCameraCalibrationEditor, Verbose, "Lens Distortion Tool wrote to dataset row file: %ls", *JsonFileName);
		}

		// If the row has an image to export, save it out to a file
		FImageView ImageView = Row->MediaImage;
		if (ImageView.RawData != nullptr)
		{
			FImageUtils::SaveImageByExtension(*ImageFileName, ImageView);
		}
	}
}

int32 ULensDistortionTool::ImportCalibrationRow(const TSharedRef<FJsonObject>& CalibrationRowObject, const FImage& RowImage, EDatasetVersion DatasetVersion)
{
	// Create a new row to populate with data from the Json object
	TSharedPtr<FCalibrationRow> NewRow = MakeShared<FCalibrationRow>();

	if (!RowImage.RawData.IsEmpty())
	{
		NewRow->MediaImage = RowImage;
	}

	if (DatasetVersion == EDatasetVersion::SeparateAlgoClasses)
	{
		NewRow->Pattern = CaptureSettings.CalibrationPattern;

		CalibrationRowObject->TryGetNumberField(TEXT("index"), NewRow->Index);

		if (NewRow->Pattern == ECalibrationPattern::Checkerboard)
		{
			const TArray<TSharedPtr<FJsonValue>>* Points3DArray;
			if (CalibrationRowObject->TryGetArrayField(TEXT("points3d"), Points3DArray))
			{
				for (TSharedPtr<FJsonValue> PointValue : *Points3DArray)
				{
					TSharedPtr<FJsonObject> PointObject = PointValue->AsObject();
					FVector NewPoint;
					PointObject->TryGetNumberField(TEXT("x"), NewPoint.X);
					PointObject->TryGetNumberField(TEXT("y"), NewPoint.Y);
					PointObject->TryGetNumberField(TEXT("z"), NewPoint.Z);

					NewRow->ObjectPoints.Points.Add(NewPoint);
				}
			}

			const TArray<TSharedPtr<FJsonValue>>* Points2DArray;
			if (CalibrationRowObject->TryGetArrayField(TEXT("points2d"), Points2DArray))
			{
				for (TSharedPtr<FJsonValue> PointValue : *Points2DArray)
				{
					TSharedPtr<FJsonObject> PointObject = PointValue->AsObject();
					FVector2D NewPoint;
					PointObject->TryGetNumberField(TEXT("x"), NewPoint.X);
					PointObject->TryGetNumberField(TEXT("y"), NewPoint.Y);

					NewRow->ImagePoints.Points.Add(NewPoint);
				}
			}

			CalibrationRowObject->TryGetNumberField(TEXT("numCornerCols"), NewRow->CheckerboardDimensions.X);
			CalibrationRowObject->TryGetNumberField(TEXT("numCornerRows"), NewRow->CheckerboardDimensions.Y);

			if (!NewRow->ObjectPoints.Points.IsEmpty() && NewRow->ObjectPoints.Points.Num() == (NewRow->CheckerboardDimensions.X * NewRow->CheckerboardDimensions.Y))
			{
				FVector TopLeft = NewRow->ObjectPoints.Points[0];
				FVector TopRight = NewRow->ObjectPoints.Points[NewRow->CheckerboardDimensions.Y - 1];
				FVector BottomLeft = NewRow->ObjectPoints.Points[NewRow->CheckerboardDimensions.X * (NewRow->CheckerboardDimensions.Y - 1)];

				NewRow->TargetPose.SetLocation(TopLeft);

				FRotator BoardRotation = FRotationMatrix::MakeFromYZ(TopRight - TopLeft, TopLeft - BottomLeft).Rotator();
				NewRow->TargetPose.SetRotation(BoardRotation.Quaternion());
			}
		}
		else if (NewRow->Pattern == ECalibrationPattern::Aruco)
		{
			const TArray<TSharedPtr<FJsonValue>>* ArucoPointArray;
			if (CalibrationRowObject->TryGetArrayField(TEXT("arucoPoints"), ArucoPointArray))
			{
				for (TSharedPtr<FJsonValue> ArucoPointValue : *ArucoPointArray)
				{
					TSharedPtr<FJsonObject> ArucoPointObject = ArucoPointValue->AsObject();

					const TArray<TSharedPtr<FJsonValue>>* Corners3DArray;
					if (ArucoPointObject->TryGetArrayField(TEXT("corners3D"), Corners3DArray))
					{
						for (TSharedPtr<FJsonValue> CornerValue : *Corners3DArray)
						{
							TSharedPtr<FJsonObject> CornerObject = CornerValue->AsObject();

							FVector Corner;
							CornerObject->TryGetNumberField(TEXT("x"), Corner.X);
							CornerObject->TryGetNumberField(TEXT("y"), Corner.Y);
							CornerObject->TryGetNumberField(TEXT("z"), Corner.Z);

							NewRow->ObjectPoints.Points.Add(Corner);
						}
					}

					const TArray<TSharedPtr<FJsonValue>>* Corners2DArray;
					if (ArucoPointObject->TryGetArrayField(TEXT("corners2D"), Corners2DArray))
					{
						for (TSharedPtr<FJsonValue> CornerValue : *Corners2DArray)
						{
							TSharedPtr<FJsonObject> CornerObject = CornerValue->AsObject();

							FVector2D Corner;
							CornerObject->TryGetNumberField(TEXT("x"), Corner.X);
							CornerObject->TryGetNumberField(TEXT("y"), Corner.Y);

							NewRow->ImagePoints.Points.Add(Corner);
						}
					}

					if (NewRow->ObjectPoints.Points.Num() >= 4)
					{
						FVector TopLeft = NewRow->ObjectPoints.Points[0];
						FVector TopRight = NewRow->ObjectPoints.Points[1];
						FVector BottomLeft = NewRow->ObjectPoints.Points[3];

						NewRow->TargetPose.SetLocation(TopLeft);

						FRotator FirstMarkerRotation = FRotationMatrix::MakeFromYZ(TopRight - TopLeft, TopLeft - BottomLeft).Rotator();
						NewRow->TargetPose.SetRotation(FirstMarkerRotation.Quaternion());
					}
				}
			}
		}
		else if (NewRow->Pattern == ECalibrationPattern::Points)
		{
			const TSharedPtr<FJsonObject>* PointDataJsonObject;
			if (CalibrationRowObject->TryGetObjectField(TEXT("calibratorPointData"), PointDataJsonObject))
			{
				const TSharedPtr<FJsonObject>* Point3DJsonObject;
				if (PointDataJsonObject->Get()->TryGetObjectField(TEXT("point3d"), Point3DJsonObject))
				{
					FVector NewPoint;
					Point3DJsonObject->Get()->TryGetNumberField(TEXT("x"), NewPoint.X);
					Point3DJsonObject->Get()->TryGetNumberField(TEXT("y"), NewPoint.Y);
					Point3DJsonObject->Get()->TryGetNumberField(TEXT("z"), NewPoint.Z);

					NewRow->ObjectPoints.Points.Add(NewPoint);
				}

				const TSharedPtr<FJsonObject>* Point2DJsonObject;
				if (PointDataJsonObject->Get()->TryGetObjectField(TEXT("point2d"), Point2DJsonObject))
				{
					FVector2D NewPoint;
					Point2DJsonObject->Get()->TryGetNumberField(TEXT("x"), NewPoint.X);
					Point2DJsonObject->Get()->TryGetNumberField(TEXT("y"), NewPoint.Y);

					NewRow->ImagePoints.Points.Add(NewPoint);
				}
			}
		}

		const TSharedPtr<FJsonObject>* CameraDataJsonObject;
		if (CalibrationRowObject->TryGetObjectField(TEXT("cameraData"), CameraDataJsonObject))
		{
			const TSharedPtr<FJsonObject>* CameraPoseJsonObject;
			if (CameraDataJsonObject->Get()->TryGetObjectField(TEXT("pose"), CameraPoseJsonObject))
			{
				const TSharedPtr<FJsonObject>* RotationJsonObject;
				if (CameraPoseJsonObject->Get()->TryGetObjectField(TEXT("rotation"), RotationJsonObject))
				{
					FQuat Rotation;
					RotationJsonObject->Get()->TryGetNumberField(TEXT("x"), Rotation.X);
					RotationJsonObject->Get()->TryGetNumberField(TEXT("y"), Rotation.Y);
					RotationJsonObject->Get()->TryGetNumberField(TEXT("z"), Rotation.Z);
					RotationJsonObject->Get()->TryGetNumberField(TEXT("w"), Rotation.W);
					NewRow->CameraPose.SetRotation(Rotation);
				}

				const TSharedPtr<FJsonObject>* TranslationJsonObject;
				if (CameraPoseJsonObject->Get()->TryGetObjectField(TEXT("translation"), TranslationJsonObject))
				{
					FVector Translation;
					TranslationJsonObject->Get()->TryGetNumberField(TEXT("x"), Translation.X);
					TranslationJsonObject->Get()->TryGetNumberField(TEXT("y"), Translation.Y);
					TranslationJsonObject->Get()->TryGetNumberField(TEXT("z"), Translation.Z);
					NewRow->CameraPose.SetTranslation(Translation);
				}
			}
		}

		Dataset.CalibrationRows.Add(NewRow);
	}
	else if (DatasetVersion == EDatasetVersion::CombinedAlgoClasses)
	{
		// We enforce strict mode to ensure that every field in the UStruct of row data is present in the imported json.
		// If any fields are missing, it is likely the row will be invalid, which will lead to errors in the calibration.
		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		constexpr bool bStrictMode = true;
		if (FJsonObjectConverter::JsonObjectToUStruct<FCalibrationRow>(CalibrationRowObject, NewRow.Get(), CheckFlags, SkipFlags, bStrictMode))
		{
			CaptureSettings.CalibrationPattern = NewRow->Pattern;
			Dataset.CalibrationRows.Add(NewRow);
		}
		else
		{
			UE_LOGF(LogCameraCalibrationEditor, Warning, "Failed to import calibration row because at least one field could not be deserialized from the json file.");
		}
	}

	return NewRow->Index;
}

#undef LOCTEXT_NAMESPACE
