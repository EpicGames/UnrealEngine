// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibrationStep.h"

#include "Calibrators/CameraCalibrationSolver.h"
#include "CameraCalibrationTypes.h"
#include "Engine/Texture2D.h"
#include "ImageCore.h"

#include "LensDistortionTool.generated.h"

struct FGeometry;
struct FPointerEvent;

class FJsonObject;
class SLensDistortionToolPanel;
class UCalibrationPointComponent;

typedef UE::Tasks::TTask<FDistortionCalibrationResult> FDistortionCalibrationTask;

/** List of supported calibration patterns */
UENUM()
enum class ECalibrationPattern : uint8
{
	Checkerboard,
	Aruco,
	Points,
	Charuco,
};

/** Version info to support backwards compatibility of dataset importing */
enum class EDatasetVersion : uint8
{
	Invalid = 0,
	SeparateAlgoClasses = 1,
	CombinedAlgoClasses = 2,

	// Add new versions above
	CurrentVersion = CombinedAlgoClasses
};

/** The data associated with a single captured calibration pattern/point */
USTRUCT()
struct FCalibrationRow
{
	GENERATED_BODY()

	/** Index to display in list view */
	UPROPERTY()
	int32 Index = -1;

	/** Set of captured 3D points for the calibrator in world space */
	UPROPERTY()
	FObjectPoints ObjectPoints;

	/** Set of captured 2D pixel locations where the calibrator was detected in the image */
	UPROPERTY()
	FImagePoints ImagePoints;

	/** Pose of the camera actor when this data for this row was captured */
	UPROPERTY()
	FTransform CameraPose = FTransform::Identity;

	/** Pose of the calibrator actor when this data for this row was captured */
	UPROPERTY()
	FTransform TargetPose = FTransform::Identity;

	/** The calibration pattern used to capture the data for this row */
	UPROPERTY()
	ECalibrationPattern Pattern = ECalibrationPattern::Checkerboard;

	/** Dimensions of the detected checkerboard pattern (only valid if calibration pattern was ECalibrationPattern::Checkerboard) */
	UPROPERTY()
	FIntPoint CheckerboardDimensions = FIntPoint(0, 0);

	/** Stored frame from the media source associated with the data for this row */
	FImage MediaImage;
};

/** An array of captured rows with calibration data */
USTRUCT()
struct FCalibrationDataset
{
	GENERATED_BODY()

	TArray<TSharedPtr<FCalibrationRow>> CalibrationRows;
};

/** Settings that control how data is captured in the tool */
USTRUCT()
struct FLensCaptureSettings
{
	GENERATED_BODY()

	/** The pattern to detect in the media image. CharUco is recommended when available. */
	UPROPERTY(EditAnywhere, Category = "Capture Settings")
	ECalibrationPattern CalibrationPattern = ECalibrationPattern::Checkerboard;

	/** An actor with Calibration Point Components that represents the virtual version of a real calibration target */
	UPROPERTY(EditAnywhere, Category = "Capture Settings")
	TWeakObjectPtr<AActor> Calibrator;

	/** Set to true if the calibration target being used is tracked */
	UPROPERTY(EditAnywhere, Category = "Capture Settings")
	bool bIsCalibratorTracked = false;

	/** Set to true if the camera being used is tracked */
	UPROPERTY(EditAnywhere, Category = "Capture Settings")
	bool bIsCameraTracked = false;

	/** Display a debug overlay over the simulcam viewport showing the detected patterns that have been captured */
	UPROPERTY(EditAnywhere, Category = "Capture Settings", meta = (EditCondition = "CalibrationPattern != ECalibrationPattern::Points", EditConditionHides))
	bool bShowOverlay = true;

	/** The name of the next calibration point to locate in the image */
	UPROPERTY(VisibleAnywhere, Category = "Capture Settings", meta = (EditCondition = "CalibrationPattern == ECalibrationPattern::Points", EditConditionHides))
	FText NextPoint;
};

/** Settings that control what and how the solver will compute the calibrated lens data */
USTRUCT()
struct FLensSolverSettings
{
	GENERATED_BODY()

	/** Select the solver to use when solving for calibrated lens data */
	UPROPERTY(EditAnywhere, NoClear, Category = "Solver Settings", meta = (HideViewOptions, ShowDisplayNames))
	TSubclassOf<ULensDistortionSolver> SolverClass = ULensDistortionSolverOpenCV::StaticClass();

	/** 
	 * If true, the solver will calibrate for the nodal offset in addition to lens distortion. 
	 * This property will be read-only if either IsCalibratorTracked or IsCameraTracked is unchecked because both are required to compute the nodal offset.
	 */
	UPROPERTY(EditAnywhere, Category = "Solver Settings")
	bool bSolveNodalOffset = false;

	/** An estimate for the focal length of the lens */
	UPROPERTY(EditAnywhere, Category = "Solver Settings")
	TOptional<double> FocalLengthGuess;

	/** Set to true to prevent the solver from optimizing the focal length. The focal length guess will be used. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Solver Settings", meta = (DisplayName = "Fix Focal Length During Optimization"))
	bool bFixFocalLength = false;

	/** Set to true to prevent the solver from optimizing the image center. The image center will be assumed to be in the exact center. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Solver Settings", meta = (DisplayName = "Fix Image Center During Optimization"))
	bool bFixImageCenter = false;

	/** Set to true to prevent the solver from optimizing the distortion parameters. The current calibrated distortion parameters will be used. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Solver Settings", meta = (DisplayName = "Fix Distortion During Optimization"))
	bool bFixDistortion = false;
};

/** Data associated with a lens distortion calibration session */
struct FLensDistortionSessionInfo
{
	/** The date/time when the current calibration session started */
	FDateTime StartTime;

	/** The index of the next row in the current calibration session */
	int32 RowIndex = -1;

	/** True if a calibration session is currently in progress */
	bool bIsActive = false;
};

/**
 * The primary tool used in the LensFile asset editor to capture and solve for calibrated lens data
 */
UCLASS()
class ULensDistortionTool : public UCameraCalibrationStep
{
	GENERATED_BODY()

public:
	friend class SLensDistortionToolPanel;
	
	//~ Begin UCameraCalibrationStep interface
	virtual void Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController) override;
	virtual void Shutdown() override;
	virtual void Tick(float DeltaTime) override;
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const  override { return TEXT("Lens Distortion"); };
	virtual bool DependsOnStep(UCameraCalibrationStep* Step) const override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual bool IsActive() const override;
	virtual UMaterialInstanceDynamic* GetOverlayMID() const override;
	virtual bool IsOverlayEnabled() const override;
	//~ End UCameraCalibrationStep interface

private:
	/** Initiate a new calibration session (if one is not already active) */
	void StartCalibrationSession();

	/** End the current calibration session */
	void EndCalibrationSession();

	/** Deletes the .json file with the input row index that was previously exported for this session. */
	void DeleteExportedRow(const int32& RowIndex) const;

	/** Increments the session index and returns its new value */
	uint32 AdvanceSessionRowIndex();

	/** Called by the UI when the user wants to save the calibration data */
	void CalibrateLens();

	/** Save the results from a finished distortion calibration to the Lens File */
	void SaveCalibrationResult();

	/** Cancel an in-progress async calibration task */
	void CancelCalibration();

	/** Get the latest status from the lens distortion calibration. Returns true if the status has changed. */
	bool GetCalibrationStatus(FText& StatusText) const;

	/** Capture a calibration pattern (or raw points) from the media image and collect additional data needed for a full calibration row */
	bool CaptureCalibrationData(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FIntRect RegionOfInterest = FIntRect());

	/** Detects a checkerboard pattern in the input array of pixels (supplied by the media image) and fills out the 2D and 3D points in the output calibration row */
	bool DetectCheckerboardPattern(TArray<FColor>& Pixels, FIntPoint Size, FIntRect RegionOfInterest, TSharedPtr<FCalibrationRow> OutRow);

	/** Detects an aruco pattern in the input array of pixels (supplied by the media image) and fills out the 2D and 3D points in the output calibration row */
	bool DetectArucoPattern(TArray<FColor>& Pixels, FIntPoint Size, TSharedPtr<FCalibrationRow> OutRow);

	/** Detects a charuco pattern in the input array of pixels (supplied by the media image) and fills out the 2D and 3D points in the output calibration row */
	bool DetectCharucoPattern(TArray<FColor>& Pixels, FIntPoint Size, FIntRect RegionOfInterest, TSharedPtr<FCalibrationRow> OutRow);

	/** Captures the mouse position and the 3D world location of the current calibration component and and fills out the output calibration row */
	bool DetectPoint(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSharedPtr<FCalibrationRow> OutRow);

	/** Import a saved calibration data set from disk */
	void ImportCalibrationDataset();

	/** Sets the calibrator object to be used. Updates the selection picker. */
	void SetCalibrator(AActor* InCalibrator);

	/** Caches the calibration point components of the current calibrator actor, and updates the list if those components become invalid */
	void UpdateCalibrationComponents();

	/** Clears the list of calibration rows */
	void ClearCalibrationRows();

	/** Updates the calibration point component index (used when the calbration pattern is ECalibrationPattern::Points) */
	void SetComponentIndex(int32 Index);

	/** Clears and redraws the overlay texture with the data from the captured calibration rows */
	void RefreshCoverage();

	/** Adjust the 2D location of every input pixel to account for differences between the size of the captured image and the overlay texture */
	void RescalePoints(TArray<FVector2D>& Points, FIntPoint DebugTextureSize, FIntPoint CameraFeedSize);

	/** Replace tracked checkerboard calibration points with a set of dummy 3D points */
	void GenerateDummyCheckerboardPoints(TArray<FObjectPoints>& Samples3d, int32 NumImages, FIntPoint CheckerboardDimensions);
	
	/** Generates the object points in local space. Available in case the solver requires it. */
	void GenerateObjectPointsInLocalSpace(TArray<FObjectPoints>& Samples3d);

	/** Get the directory for the current session */
	FString GetSessionSaveDir() const;

	/** Get the filename of a row with the input index */
	FString GetRowFilename(int32 RowIndex) const;

	/**
	 * Import a JsonObject of calibration data that represents a single calibration row.
	 * Returns the row index of the imported row.
	 */
	int32 ImportCalibrationRow(const TSharedRef<FJsonObject>& CalibrationRowObject, const FImage& RowImage, EDatasetVersion DatasetVersion);

	/** Export global session data to a .json file */
	void ExportSessionData();

	/** Export the row data to a json file */
	void ExportCalibrationRow(TSharedPtr<FCalibrationRow> Row);

private:
	/** UI Widget for this tool */
	TSharedPtr<SLensDistortionToolPanel> DistortionWidget;

	FLensSolverSettings SolverSettings;
	FLensCaptureSettings CaptureSettings;

	/** Collection of calibration rows containing the data needed to run a lens calibration */
	FCalibrationDataset Dataset;

	/** The solver object that runs the lens calibration on another thread. The ptr is stored to access the solver to get status updates and/or cancel the calibration from the game thread */
	UPROPERTY(Transient)
	TObjectPtr<ULensDistortionSolver> Solver;

	/** An asynchronous task handle. When valid, the tool will poll its state to determine when the task has completed, and then extract the calibration result from this task handle. */
	FDistortionCalibrationTask CalibrationTask;

	/** The result from the most recently completed distortion calibration. It is saved as a member variable because the result is not saved immediately, but only if the user confirms that it should be saved. */
	FDistortionCalibrationResult CalibrationResult;

	/** Container for the set of calibrator components selected in the component combobox */
	TArray<TWeakObjectPtr<UCalibrationPointComponent>> CalibrationComponents;

	/** 
	 * The 3D world locations of the calibration point components should all come from the same frame.
	 * Therefore, the locations are all saved at once when the first calibration point is selected (only applicable to ECalibrationPattern::Points)
	 */
	TArray<FVector> CachedComponentLocations;

	/** Index of the next calibration point component to capture (only applicable to ECalibrationPattern::Points) */
	int32 CalibrationComponentIndex = -1;

	/** Capture session info, used to track whether newly captured data should be added to an existing dataset */
	FLensDistortionSessionInfo SessionInfo;

	/** True if this tool is the active tab in the UI */
	bool bIsActive = false;

	/** Material and Texture that draw the coverage overlay on top of the simulcam viewport during capture */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> OverlayMID;

	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> OverlayTexture;

	/** Weak ptr to the steps controller that created this step */
	TWeakPtr<FCameraCalibrationStepsController> WeakStepsController;
};
