// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensInfoStep.h"

#include "Camera/CameraActor.h"
#include "CameraCalibrationStepsController.h"
#include "IStructureDetailsView.h"
#include "LensFile.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SResetToDefaultMenu.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LensInfoStep"

void ULensInfoStep::Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController)
{
	CameraCalibrationStepsController = InCameraCalibrationStepController;

	const ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();
	OriginalLensInfo = LensFile->LensInfo;
}

TSharedRef<SWidget> ULensInfoStep::BuildUI()
{
	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();

	FStructureDetailsViewArgs LensInfoStructDetailsView;
	FDetailsViewArgs DetailArgs;
	DetailArgs.bAllowSearch = false;
	DetailArgs.bShowScrollBar = true;

	TSharedRef<FStructOnScope> LensInfoStructOnScope = MakeShared<FStructOnScope>(FLensInfo::StaticStruct(), reinterpret_cast<uint8*>(&LensFile->LensInfo));
	TSharedPtr<IStructureDetailsView> LensInfoStructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, LensInfoStructDetailsView, LensInfoStructOnScope);

	LensInfoStructureDetailsView->GetDetailsView()->OnFinishedChangingProperties().AddUObject(this, &ULensInfoStep::OnLensInfoChanged);

	/** Camera Feed Info Details View */
	FStructureDetailsViewArgs CameraFeedInfoStructDetailsView;

	TSharedRef<FStructOnScope> CameraFeedInfoStructOnScope = MakeShared<FStructOnScope>(FCameraFeedInfo::StaticStruct(), reinterpret_cast<uint8*>(&LensFile->CameraFeedInfo));
	TSharedPtr<IStructureDetailsView> CameraFeedInfoStructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, CameraFeedInfoStructDetailsView, CameraFeedInfoStructOnScope);

	CameraFeedInfoStructureDetailsView->GetDetailsView()->OnFinishedChangingProperties().AddUObject(this, &ULensInfoStep::OnCameraFeedInfoChanged);

	/** Simulcam Info Details View */
	FStructureDetailsViewArgs SimulcamInfoStructDetailsView;

	TSharedRef<FStructOnScope> SimulcamInfoStructOnScope = MakeShared<FStructOnScope>(FSimulcamInfo::StaticStruct(), reinterpret_cast<uint8*>(&LensFile->SimulcamInfo));
	TSharedPtr<IStructureDetailsView> SimulcamInfoStructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, SimulcamInfoStructDetailsView, SimulcamInfoStructOnScope);

	SimulcamInfoStructureDetailsView->GetDetailsView()->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([]() { return false; }));

	TSharedPtr<SWidget> StepWidget = 
		SNew(SVerticalBox)
		+ SVerticalBox::Slot() // Lens information structure
		.AutoHeight()
		[ 
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[ LensInfoStructureDetailsView->GetWidget().ToSharedRef() ]
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SResetToDefaultMenu)
				.OnResetToDefault(FSimpleDelegate::CreateUObject(this, &ULensInfoStep::ResetToDefault))
				.DiffersFromDefault(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateUObject(this, &ULensInfoStep::DiffersFromDefault)))
			]
		]

		+ SVerticalBox::Slot() // Text Warning that the camera info does not match the cine camera's settings
		.AutoHeight()
		.Padding(0, 10)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CameraSettingMismatch", "These filmback settings do not match the settings of the CineCamera.\nIf they are expected to match, set the FilmbackOverride setting on the LensComponent to \"LensFile\"."))
			.ColorAndOpacity(FSlateColor(FLinearColor::Yellow))
			.Justification(ETextJustify::Center)
			.AutoWrapText(true)
			.Visibility_UObject(this, &ULensInfoStep::GetFilmbackWarningVisibility)
		]

		+ SVerticalBox::Slot() // Camera Feed information structure
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[CameraFeedInfoStructureDetailsView->GetWidget().ToSharedRef() ]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SResetToDefaultMenu)
				.OnResetToDefault(FSimpleDelegate::CreateUObject(this, &ULensInfoStep::ResetCameraFeedInfoToDefault))
				.DiffersFromDefault(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateUObject(this, &ULensInfoStep::CameraFeedInfoDiffersFromDefault)))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(4.0f, 4.0f, 4.0f, 4.0f))
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ControllerWarningToolTip", "The aspect ratios of the camera feed and the CG camera do not match."))
			.ColorAndOpacity(FSlateColor(FLinearColor::Yellow))
			.Visibility_UObject(this, &ULensInfoStep::HandleAspectRatioWarningVisibility)
		]

		+ SVerticalBox::Slot() // Simulcam information structure
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[ SimulcamInfoStructureDetailsView->GetWidget().ToSharedRef() ]
		];
	

	return StepWidget.ToSharedRef();
}

bool ULensInfoStep::DependsOnStep(UCameraCalibrationStep* Step) const
{
	return false;
}

void ULensInfoStep::Activate()
{
	// Nothing to do if it is already active.
	if (bIsActive)
	{
		return;
	}

	ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();
	if(LensFile)
	{
		// Initialize lens info to current value of the LensFile
		CachedLensInfo = LensFile->LensInfo;
	}

	bIsActive = true;
}

void ULensInfoStep::Deactivate()
{
	bIsActive = false;
}

void ULensInfoStep::OnLensInfoChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// Ignore temporary interaction (dragging sliders, etc.)
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet)
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.Property->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensInfo, LensModel))
	{
		if (ULensFile* const LensFile = CameraCalibrationStepsController.Pin()->GetLensFile())
		{
			LensFile->OnLensFileModelChanged().Broadcast(LensFile->LensInfo.LensModel);
		}
	}

	SaveLensInformation();
}

void ULensInfoStep::SaveLensInformation()
{
	if (!CameraCalibrationStepsController.IsValid())
	{
		return;
	}

	ULensFile* const LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();
	if (!LensFile)
	{
		return;
	}

	// Validate sensor dimensions
	constexpr float MinimumSize = 1.0f; // Limit sensor dimension to 1mm
	if(LensFile->LensInfo.SensorDimensions.X < MinimumSize || LensFile->LensInfo.SensorDimensions.Y < MinimumSize)
	{
		const FText ErrorMessage = LOCTEXT("InvalidSensorDimensions", "Invalid sensor dimensions. Can't have dimensions smaller than 1mm");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		
		// Reset sensor dimensions if the new value was invalid
		LensFile->LensInfo.SensorDimensions = CachedLensInfo.SensorDimensions;
		return;	
	}

	// Before beginning the transaction, briefly restore the LensInfo to its previously saved state
	const FLensInfo NewLensInfo = LensFile->LensInfo;
	LensFile->LensInfo = CachedLensInfo;
	{
		FScopedTransaction Transaction(LOCTEXT("ModifyingLensInfo", "Modifying LensFile information"));
		this->Modify();
		LensFile->Modify();

		// After beginning the transaction, reapply the modified LensInfo to the LensFile
		LensFile->LensInfo = NewLensInfo;

		// If lens model has changed and distortion table has data, notify the user that calibration data will be lost
		const bool bHasModelChanged = CachedLensInfo.LensModel != LensFile->LensInfo.LensModel;
		if (bHasModelChanged)
		{
			if (LensFile->HasSamples(ELensDataCategory::Distortion)
				|| LensFile->HasSamples(ELensDataCategory::ImageCenter)
				|| LensFile->HasSamples(ELensDataCategory::Zoom))
			{
				const FText Message = LOCTEXT("DataChangeDataLoss", "Lens model change detected. Distortion, ImageCenter and FocalLength data samples will be lost. Do you wish to continue?");
				if (FMessageDialog::Open(EAppMsgType::OkCancel, Message) != EAppReturnType::Ok)
				{
					Transaction.Cancel();
					LensFile->LensInfo.LensModel = CachedLensInfo.LensModel;
					return;
				}
			}

			// Clear table when disruptive change is detected and user proceeds
			LensFile->ClearData(ELensDataCategory::Distortion);
			LensFile->ClearData(ELensDataCategory::ImageCenter);
			LensFile->ClearData(ELensDataCategory::Zoom);
		}

		// Cache the newly modified LensInfo as the most recently saved state
		CachedLensInfo = LensFile->LensInfo;
	}
}

bool ULensInfoStep::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// User interaction is Ctrl + Left Mouse Button
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && (MouseEvent.GetModifierKeys().IsControlDown()))
	{
		if (TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin())
		{
			const FVector2D LocalInPixels = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			StepsController->SetCameraFeedDimensionsFromMousePosition(LocalInPixels);
			return true;
		}
	}

	return false;
}

void ULensInfoStep::OnCameraFeedInfoChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// Ignore temporary interaction (dragging sliders, etc.)
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		if (TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin())
		{
			if (ULensFile* LensFile = StepsController->GetLensFile())
			{
				FIntPoint CameraFeedDimensions = LensFile->CameraFeedInfo.GetDimensions();
				const FIntPoint OverlayResolution = StepsController->GetMediaOverlayResolution();
				CameraFeedDimensions.X = FMath::Clamp(CameraFeedDimensions.X, 0, OverlayResolution.X);
				CameraFeedDimensions.Y = FMath::Clamp(CameraFeedDimensions.Y, 0, OverlayResolution.Y);

				StepsController->SetCameraFeedDimensions(CameraFeedDimensions, true);
			}
		}
	}
}

void ULensInfoStep::ResetCameraFeedInfoToDefault()
{
	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin())
	{
		StepsController->ResetCameraFeedDimensions();
	}
}

bool ULensInfoStep::CameraFeedInfoDiffersFromDefault() const
{
	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin())
	{
		if (ULensFile* LensFile = StepsController->GetLensFile())
		{
			return LensFile->CameraFeedInfo.IsOverridden();
		}
	}
	return false;
}

EVisibility ULensInfoStep::HandleAspectRatioWarningVisibility() const
{
	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin())
	{
		if (ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile())
		{
			// Do not display the warning if the camera feed info is invalid
			if (!LensFile->CameraFeedInfo.IsValid())
			{
				return EVisibility::Collapsed;
			}

			const float CameraFeedAspectRatio = LensFile->CameraFeedInfo.GetAspectRatio();
			const float CGCameraAspectRatio = LensFile->SimulcamInfo.CGLayerAspectRatio;

			// Display the warning if the difference between the two aspect ratios is higher than the acceptable tolerance
			constexpr float AspectRatioErrorTolerance = 0.01f;
			if (!FMath::IsNearlyEqual(CameraFeedAspectRatio, CGCameraAspectRatio, AspectRatioErrorTolerance))
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility ULensInfoStep::GetFilmbackWarningVisibility() const
{
	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin())
	{
		ULensFile* LensFile = StepsController->GetLensFile();
		UCineCameraComponent* CineCameraComponent = nullptr;

		if (ACameraActor* Camera = StepsController->GetCamera())
		{
			TInlineComponentArray<UCineCameraComponent*> CameraComponents;
			Camera->GetComponents(CameraComponents);

			if (CameraComponents.Num() > 0)
			{
				CineCameraComponent = CameraComponents[0];
			}
		}

		if (LensFile && CineCameraComponent)
		{
			const FVector2f LensFileFilmback = FVector2f(LensFile->LensInfo.SensorDimensions.X, LensFile->LensInfo.SensorDimensions.Y);
			if (!FMath::IsNearlyEqual(LensFileFilmback.X, CineCameraComponent->Filmback.SensorWidth) ||
				!FMath::IsNearlyEqual(LensFileFilmback.Y, CineCameraComponent->Filmback.SensorHeight) ||
				!FMath::IsNearlyEqual(LensFile->LensInfo.SqueezeFactor, CineCameraComponent->LensSettings.SqueezeFactor))
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

bool ULensInfoStep::IsActive() const
{
	return bIsActive;
}

void ULensInfoStep::ResetToDefault()
{
	ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();
	LensFile->LensInfo = OriginalLensInfo;
	SaveLensInformation();
}

bool ULensInfoStep::DiffersFromDefault() const
{
	const ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();
	return !(OriginalLensInfo == LensFile->LensInfo);
}



#undef LOCTEXT_NAMESPACE
