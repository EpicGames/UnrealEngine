// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibrationStep.h"

#include "LensData.h"
#include "UObject/StructOnScope.h"

#include "LensInfoStep.generated.h"

/**
 * ULensInfoStep is used as the initial step to provide information about the lens you are going to calibrate
 */
UCLASS()
class ULensInfoStep : public UCameraCalibrationStep
{
	GENERATED_BODY()

public:

	//~ Begin UCameraCalibrationStep interface
	virtual void Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController) override;
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const  override { return TEXT("Lens Information"); };
	virtual bool DependsOnStep(UCameraCalibrationStep* Step) const override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual bool IsActive() const override;
	//~ End UCameraCalibrationStep interface

public:

	/** Triggered when user edits any of the LensInfo properties */
	void OnLensInfoChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Triggered when user edits any of the CameraFeedInfo properties */
	void OnCameraFeedInfoChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Validates the new LensInfo settings and records a transaction for them */
	void SaveLensInformation();

	/** Reapply the original LensInfo to the LensFile */
	void ResetToDefault();

	/** Returns true if our LensInfo differs from the original info when the tool was opened */
	bool DiffersFromDefault() const;

	/** Reset the camera feed dimensions to their default value (recomputed by the Steps Controller) */
	void ResetCameraFeedInfoToDefault();

	/** Returns true if the camera feed info has been overriden by the user */
	bool CameraFeedInfoDiffersFromDefault() const;

private:
	/** Determines the visibility of the aspect ratio warning by comparing the camera feed aspect ratio with that of the camera actor */
	EVisibility HandleAspectRatioWarningVisibility() const;

	/** Determines the visibility of the camera settings warning by comparing the filmback and squeeze factor in the LensFile with that of the camera actor */
	EVisibility GetFilmbackWarningVisibility() const;

private:

	/** Pointer to the calibration steps controller */
	TWeakPtr<FCameraCalibrationStepsController> CameraCalibrationStepsController;

	/** True if this tool is the active one in the panel */
	bool bIsActive = false;

	/** Original LensInfo, used to reset the details back to default */
	FLensInfo OriginalLensInfo;

	/** Cached previous LensInfo, used to trigger correct transactions to allow for undo operations */
	UPROPERTY(Transient)
	FLensInfo CachedLensInfo;
};
