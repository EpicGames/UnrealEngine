// Copyright Epic Games, Inc. All Rights Reserved.

#include "NodalOffsetTool.h"

#include "CalibrationPointComponent.h"
#include "Camera/CameraActor.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationStepsController.h"
#include "CameraCalibrationSubsystem.h"
#include "CameraCalibrationTypes.h"
#include "DesktopPlatformModule.h"
#include "Dom/JsonObject.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IDesktopPlatform.h"
#include "ImageCenterTool.h"
#include "ImageUtils.h"
#include "LensComponent.h"
#include "Misc/DateTime.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "SNodalOffsetToolPanel.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "NodalOffsetTool"

namespace UE::CameraCalibration::Private::NodalOffsetTool
{
	static const FString SessionDateTimeField(TEXT("SessionDateTime"));
	static const FString AlgoNameField(TEXT("AlgoName"));
}

void UNodalOffsetTool::Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController)
{
	CameraCalibrationStepsController = InCameraCalibrationStepController;

	UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	check(Subsystem);

	for (const FName& AlgoName : Subsystem->GetCameraNodalOffsetAlgos())
	{
		TSubclassOf<UCameraNodalOffsetAlgo> AlgoClass = Subsystem->GetCameraNodalOffsetAlgo(AlgoName);
		const UCameraNodalOffsetAlgo* Algo = CastChecked<UCameraNodalOffsetAlgo>(AlgoClass->GetDefaultObject());

		// If the algo uses an overlay material, create a new MID to use with that algo
		if (UMaterialInterface* OverlayMaterial = Algo->GetOverlayMaterial())
		{
			AlgoOverlayMIDs.Add(Algo->FriendlyName(), UMaterialInstanceDynamic::Create(OverlayMaterial, GetTransientPackage()));
		}
	}
}

void UNodalOffsetTool::Shutdown()
{
	if (NodalOffsetAlgo)
	{
		NodalOffsetAlgo->Shutdown();
		NodalOffsetAlgo = nullptr;

		EndCalibrationSession();
	}
}

void UNodalOffsetTool::Tick(float DeltaTime)
{
	if (NodalOffsetAlgo)
	{
		NodalOffsetAlgo->Tick(DeltaTime);
	}
}

bool UNodalOffsetTool::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bIsActive)
	{
		return false;
	}

	if (!NodalOffsetAlgo)
	{
		return false;
	}

	return NodalOffsetAlgo->OnViewportClicked(MyGeometry, MouseEvent);
}

bool UNodalOffsetTool::OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition)
{
	if (!bIsActive || !NodalOffsetAlgo)
	{
		return false;
	}

	// Each algo has its own reaction to the marquee selection.
	return NodalOffsetAlgo->OnViewportMarqueeSelect(StartPosition, EndPosition);
}

bool UNodalOffsetTool::OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent)
{
	if (!bIsActive)
	{
		return false;
	}

	if (!NodalOffsetAlgo)
	{
		return false;
	}

	return NodalOffsetAlgo->OnViewportInputKey(InKey, InEvent);
}

TSharedRef<SWidget> UNodalOffsetTool::BuildUI()
{
	return SNew(SNodalOffsetToolPanel, this);
}

bool UNodalOffsetTool::DependsOnStep(UCameraCalibrationStep* Step) const
{
	return !!Cast<UImageCenterTool>(Step);
}

void UNodalOffsetTool::Activate()
{
	// Nothing to do if it is already active.
	if (bIsActive)
	{
		return;
	}

	bIsActive = true;
}

void UNodalOffsetTool::Deactivate()
{
	bIsActive = false;
}

void UNodalOffsetTool::SetNodalOffsetAlgo(const FName& AlgoName)
{
	// Ask subsystem for the selected nodal offset algo class

	UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	check(Subsystem);

	TSubclassOf<UCameraNodalOffsetAlgo> AlgoClass = Subsystem->GetCameraNodalOffsetAlgo(AlgoName);
		
	// If it is the same as the existing one, do nothing.
	if (!NodalOffsetAlgo && !AlgoClass)
	{
		return;
	}
	else if (NodalOffsetAlgo && (NodalOffsetAlgo->GetClass() == AlgoClass))
	{
		return;
	}

	// Remove old Algo
	if (NodalOffsetAlgo)
	{
		NodalOffsetAlgo->Shutdown();
		NodalOffsetAlgo = nullptr;

		EndCalibrationSession();
	}

	// If AlgoClass is none, we're done here.
	if (!AlgoClass)
	{
		return;
	}

	// Create new algo
	NodalOffsetAlgo = NewObject<UCameraNodalOffsetAlgo>(
		GetTransientPackage(),
		AlgoClass,
		MakeUniqueObjectName(GetTransientPackage(), AlgoClass),
		RF_Transactional);

	if (NodalOffsetAlgo)
	{
		NodalOffsetAlgo->Initialize(this);
	}

	// Set the tool overlay pass' material to the MID associate with the current algo
	if (CameraCalibrationStepsController.IsValid())
	{
		TSharedPtr<FCameraCalibrationStepsController> StepsController = CameraCalibrationStepsController.Pin();
		StepsController->SetOverlayEnabled(false);

		if (UMaterialInstanceDynamic* OverlayMID = GetOverlayMID())
		{
			StepsController->SetOverlayMaterial(OverlayMID);
		}
	}
}

UCameraNodalOffsetAlgo* UNodalOffsetTool::GetNodalOffsetAlgo() const
{
	return NodalOffsetAlgo;
}

void UNodalOffsetTool::OnSaveCurrentNodalOffset()
{
	if (!CameraCalibrationStepsController.IsValid())
	{
		return;
	}

	const FText TitleInfo = LOCTEXT("NodalOffsetInfo", "Nodal Offset Calibration Info");
	const FText TitleError = LOCTEXT("NodalOffsetError", "Nodal Offset Calibration Error");

	UCameraNodalOffsetAlgo* Algo = GetNodalOffsetAlgo();

	if (!Algo)
	{
		FText ErrorMessage = LOCTEXT("NoAlgoFound", "No algo found");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);
		return;
	}

	FText ErrorMessage;

	float Focus = 0.0f;
	float Zoom = 0.0f;
	FNodalPointOffset NodalOffset;
	float ReprojectionError;

	if (!Algo->GetNodalOffset(NodalOffset, Focus, Zoom, ReprojectionError, ErrorMessage))
	{
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);
		return;
	}

	// Show reprojection error
	{
		FFormatOrderedArguments Arguments;
		Arguments.Add(FText::FromString(FString::Printf(TEXT("%.2f"), ReprojectionError)));

		const FText Message = FText::Format(LOCTEXT("ReprojectionError", "RMS Reprojection Error: {0} pixels"), Arguments);

		// Allow the user to cancel adding to the LUT if the reprojection error is unacceptable.
		if (FMessageDialog::Open(EAppMsgType::OkCancel, Message, TitleInfo) != EAppReturnType::Ok)
		{
			return;
		}
	}

	ULensFile* LensFile = CameraCalibrationStepsController.Pin()->GetLensFile();

	if (!LensFile)
	{
		ErrorMessage = LOCTEXT("NoLensFile", "No Lens File");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SaveCurrentNodalOffset", "Save Current Nodal Offset"));
	LensFile->Modify();
	LensFile->AddNodalOffsetPoint(Focus, Zoom, NodalOffset);

	// Force the lens component to apply nodal offset so that we can see the effect right away
	TInlineComponentArray<ULensComponent*> LensComponents;
	CameraCalibrationStepsController.Pin()->GetCamera()->GetComponents(LensComponents);

	for (ULensComponent* LensComponent : LensComponents)
	{
		if (LensFile == LensComponent->GetLensFile())
		{
			LensComponent->SetApplyNodalOffsetOnTick(true);
			break;
		}
	}

	Algo->OnSavedNodalOffset();
}

FCameraCalibrationStepsController* UNodalOffsetTool::GetCameraCalibrationStepsController() const
{
	if (!CameraCalibrationStepsController.IsValid())
	{
		return nullptr;
	}

	return CameraCalibrationStepsController.Pin().Get();
}

bool UNodalOffsetTool::IsActive() const
{
	return bIsActive;
}

UMaterialInstanceDynamic* UNodalOffsetTool::GetOverlayMID() const
{
	return AlgoOverlayMIDs.FindRef(NodalOffsetAlgo->FriendlyName()).Get();
}

bool UNodalOffsetTool::IsOverlayEnabled() const
{
	return NodalOffsetAlgo->IsOverlayEnabled();
}

void UNodalOffsetTool::StartCalibrationSession()
{
	if (!SessionInfo.bIsActive)
	{
		SessionInfo.bIsActive = true;
		SessionInfo.StartTime = FDateTime::Now();
	}
}

void UNodalOffsetTool::EndCalibrationSession()
{
	if (SessionInfo.bIsActive)
	{
		SessionInfo.bIsActive = false;
		SessionInfo.RowIndex = -1;
	}
}

uint32 UNodalOffsetTool::AdvanceSessionRowIndex()
{
	SessionInfo.RowIndex += 1;
	return SessionInfo.RowIndex;
}

FString UNodalOffsetTool::GetSessionSaveDir() const
{
	using namespace UE::CameraCalibration::Private;

	const FString SessionDateString = SessionInfo.StartTime.ToString(TEXT("%Y-%m-%d"));
	const FString SessionTimeString = SessionInfo.StartTime.ToString(TEXT("%H-%M-%S"));
	const FString DatasetPrefix = TEXT("Dataset-") + NodalOffsetAlgo->ShortName().ToString() + TEXT("Algorithm-");
	const FString DatasetDir = DatasetPrefix + SessionTimeString;

	const FString ProjectSaveDir = FPaths::ProjectSavedDir() / TEXT("CameraCalibration") / TEXT("NodalOffset");

	return ProjectSaveDir / SessionDateString / DatasetDir;
}

FString UNodalOffsetTool::GetRowFilename(int32 RowIndex) const
{
	const FString RowNumString = TEXT("Row") + FString::FromInt(RowIndex) + TEXT("-");
	return RowNumString;
}

void UNodalOffsetTool::DeleteExportedRow(const int32& RowIndex) const
{
	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	// Find all files in the directory of the currently active session
	const FString PathName = GetSessionSaveDir();
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *PathName);

	const FString RowNumString = GetRowFilename(RowIndex);

	// Delete any files containing that row number from the session directory
	for (const FString& File : FoundFiles)
	{
		if (File.Contains(RowNumString))
		{
			const FString FullPath = PathName / File;
			IFileManager::Get().Delete(*FullPath);
			UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool removed calibration dataset file: %ls", *FullPath);
		}
	}
}

void UNodalOffsetTool::ExportCalibrationRow(int32 RowIndex, const TSharedRef<FJsonObject>& RowObject, const FImageView& RowImage)
{
	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	// Start a calibration session (if one is not currently active)
	StartCalibrationSession();

	// Assemble the path and filename for this row based on the session and row index
	const FString PathName = GetSessionSaveDir();
	const FString FileName = GetRowFilename(RowIndex) + FDateTime::Now().ToString(TEXT("%H-%M-%S"));

	const FString JsonFileName = PathName / FileName + TEXT(".json");
	const FString ImageFileName = PathName / FileName + TEXT(".png");

	// Create and open a new Json file for writing, and initialize a JsonWriter to serialize the contents
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*JsonFileName)))
	{
		TSharedRef< TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(FileWriter.Get());

		// Write the Json row data out and save the file
		FJsonSerializer::Serialize(RowObject, JsonWriter);
		FileWriter->Close();

		UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool wrote to dataset row file: %ls", *JsonFileName);
	}

	// If the row has an image to export, save it out to a file
	if (RowImage.RawData != nullptr)
	{
		FImageUtils::SaveImageByExtension(*ImageFileName, RowImage);
	}
}

void UNodalOffsetTool::ExportSessionData(const TSharedRef<FJsonObject>& SessionDataObject)
{
	using namespace UE::CameraCalibration::Private;

	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	// Start a calibration session (if one is not currently active)
	StartCalibrationSession();

	// Assemble the path and filename for this row based on the session and row index
	const FString PathName = GetSessionSaveDir();
	const FString FileName = TEXT("SessionData");

	const FString SessionFileName = PathName / FileName + TEXT(".ucamcalib");

	// Delete the existing session data file (if it exists)
	if (IFileManager::Get().FileExists(*SessionFileName))
	{
		IFileManager::Get().Delete(*SessionFileName);
	}

	// Create and open a new Json file for writing, and initialize a JsonWriter to serialize the contents
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*SessionFileName)))
	{
		TSharedRef< TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(FileWriter.Get());

		const FString SessionDateTimeString = SessionInfo.StartTime.ToString(TEXT("%Y-%m-%d")) + TEXT("-") + SessionInfo.StartTime.ToString(TEXT("%H-%M-%S"));

		SessionDataObject->SetStringField(NodalOffsetTool::SessionDateTimeField, SessionDateTimeString);
		SessionDataObject->SetStringField(NodalOffsetTool::AlgoNameField, NodalOffsetAlgo->FriendlyName().ToString());

		// Write the Json row data out and save the file
		FJsonSerializer::Serialize(SessionDataObject, JsonWriter);
		FileWriter->Close();

		UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool wrote to dataset session file: %ls", *SessionFileName);
	}
}

void UNodalOffsetTool::ImportCalibrationDataset()
{
	using namespace UE::CameraCalibration::Private;

	if (!GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
	{
		return;
	}

	// If there is existing calibration data that will be overwritten during import, ask the user to confirm that they want to continue
	if (NodalOffsetAlgo->HasCalibrationData())
	{
		const FText ConfirmationMessage = LOCTEXT(
			"ImportDatasetConfirmationMessage",
			"There are existing calibration rows which will be removed during the import process. Do you want to proceed with the import?");

		if (FMessageDialog::Open(EAppMsgType::YesNo, ConfirmationMessage) == EAppReturnType::No)
		{
			return;
		}
	}

	// Open a file dialog to select a .ucamcalib session data file 
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
	const FString Title = TEXT("Import Camera Calibration Dataset");
	const FString DefaultPath = FPaths::ProjectSavedDir() / TEXT("CameraCalibration") / TEXT("NodalOffset");
	const FString DefaultFile = TEXT("");
	const FString FileTypes = TEXT("Camera Calibration Dataset|*.ucamcalib");
	const uint32 OpenFileFlags = 0;

	// Note, OpenFileFlags is not set to "Multiple" so we only expect one file to be selected
	TArray<FString> SelectedFileNames;
	const bool bFileSelected = DesktopPlatform->OpenFileDialog(ParentWindowHandle, Title, DefaultPath, DefaultFile, FileTypes, OpenFileFlags, SelectedFileNames);

	// Early-out if no calibration file was selected
	if (!bFileSelected || SelectedFileNames.Num() < 1)
	{
		return;
	}

	// Parse the session data filename and the directory from the full path
	const FString SessionFileName = FPaths::GetCleanFilename(SelectedFileNames[0]);
	const FString SelectedDirectory = FPaths::GetPath(SelectedFileNames[0]);

	// Find all json files in the selected directory (this will not include the .ucamcalib session date file)
	TArray<FString> FoundFiles;
	const FString FileExtension = TEXT(".json");
	IFileManager::Get().FindFiles(FoundFiles, *SelectedDirectory, *FileExtension);

	// Early-out if selected directory has no json files to import
	if (FoundFiles.Num() < 1)
	{
		const FText ErrorMessage = LOCTEXT("NoJsonFilesFound", "The selected directory has no .json files to import.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		return;
	}

	// Import the session data
	FDateTime ImportedSessionDateTime = FDateTime::Now();
	{
		const FString SessionFile = SelectedDirectory / SessionFileName;

		// Open the Json file for reading, and initialize a JsonReader to parse the contents
		if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*SessionFile)))
		{
			TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());

			// Deserialize the row data from the Json file into a Json object
			TSharedPtr<FJsonObject> JsonSessionData = MakeShared<FJsonObject>();
			if (FJsonSerializer::Deserialize(JsonReader, JsonSessionData))
			{
				// Import the session date/time so that we can restore the imported session
				FString SessionDateTimeString;
				if (JsonSessionData->TryGetStringField(NodalOffsetTool::SessionDateTimeField, SessionDateTimeString))
				{
					ensureMsgf(FDateTime::Parse(SessionDateTimeString, ImportedSessionDateTime), TEXT("Failed to parse imported session date and time"));
				}
				else
				{
					UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool failed to deserialize the date/time from the session file: %ls", *SessionFile);
				}

				// Import the algo name so we can set the appropriate algo before importing the row data
				FString AlgoString;
				if (JsonSessionData->TryGetStringField(NodalOffsetTool::AlgoNameField, AlgoString))
				{
					// Ensure that the algo name matches one of the algos for this tool
					const FName AlgoName = FName(*AlgoString);
					if (UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
					{
						if (TSubclassOf<UCameraNodalOffsetAlgo> AlgoClass = Subsystem->GetCameraNodalOffsetAlgo(AlgoName))
						{
							SetNodalOffsetAlgo(AlgoName);
						}
						else
						{
							const FText ErrorMessage = LOCTEXT("UnknownCalibrationAlgo", "The selected dataset does not represent a nodal offset calibration. Choose a different .ucamcalib dataset file from a nodal offset calibration.");
							FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
							return;
						}
					}
				}
				else
				{
					UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool failed to deserialize the algo name from the session file: %ls", *SessionFile);
				}

				NodalOffsetAlgo->ImportSessionData(JsonSessionData.ToSharedRef());
			}
			else
			{
				UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool failed to deserialize dataset session file: %ls", *SessionFile);
			}
		}
	}

	NodalOffsetAlgo->PreImportCalibrationData();

	// Initialize a maximum row index which will be used to set the current session row index if the user wants to add additional rows after importing
	int32 MaxRowIndex = -1;

	for (const FString& File : FoundFiles)
	{
		const FString JsonFileName = SelectedDirectory / File;
		const FString ImageFileName = JsonFileName.Replace(TEXT(".json"), TEXT(".png"));

		// Load the PNG image file for this row into an FImage
		FImage RowImage;
		if (IFileManager::Get().FileExists(*ImageFileName))
		{
			FImageUtils::LoadImage(*ImageFileName, RowImage);
		}

		// Open the Json file for reading, and initialize a JsonReader to parse the contents
		if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*JsonFileName)))
		{
			TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());

			// Deserialize the row data from the Json file into a Json object
			TSharedPtr<FJsonObject> JsonRowData = MakeShared<FJsonObject>();
			if (FJsonSerializer::Deserialize(JsonReader, JsonRowData))
			{
				int32 RowNum = NodalOffsetAlgo->ImportCalibrationRow(JsonRowData.ToSharedRef(), RowImage);
				MaxRowIndex = FMath::Max(MaxRowIndex, RowNum);
			}
			else
			{
				UE_LOGF(LogCameraCalibrationEditor, Verbose, "Nodal Offset Tool failed to deserialize the dataset row file: %ls", *JsonFileName);
			}
		}
	}

	NodalOffsetAlgo->PostImportCalibrationData();

	// Set the current session's start date/time and row index to match what was just imported to support adding/deleting rows
	SessionInfo.bIsActive = true;
	SessionInfo.StartTime = ImportedSessionDateTime;
	SessionInfo.RowIndex = MaxRowIndex;
}

#undef LOCTEXT_NAMESPACE
