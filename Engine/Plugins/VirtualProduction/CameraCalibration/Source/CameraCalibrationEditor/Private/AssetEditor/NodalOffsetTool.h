// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CameraCalibrationStep.h"
#include "Engine/World.h"
#include "ImageCore.h"

#include "NodalOffsetTool.generated.h"

struct FGeometry;
struct FPointerEvent;

class FJsonObject;
class UCameraNodalOffsetAlgo;

/** Data associated with a nodal offset calibration session */
struct FNodalOffsetSessionInfo
{
	/** The date/time when the current calibration session started */
	FDateTime StartTime;

	/** The index of the next row in the current calibration session */
	int32 RowIndex = -1;

	/** True if a calibration session is currently in progress */
	bool bIsActive = false;
};

/**
 * FNodalOffsetTool is the controller for the nodal offset tool panel.
 * It has the logic to bridge user input like selection of nodal offset algorithm or CG camera
 * with the actions that follow. It houses convenience functions used to generate the data
 * of what is presented to the user, and holds pointers to the relevant objects and structures.
 */
UCLASS()
class UNodalOffsetTool : public UCameraCalibrationStep
{
	GENERATED_BODY()

public:

	//~ Begin UCameraCalibrationStep interface
	virtual void Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController) override;
	virtual void Shutdown() override;
	virtual void Tick(float DeltaTime) override;
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition) override;
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const  override { return TEXT("Nodal Offset"); };
	virtual bool DependsOnStep(UCameraCalibrationStep* Step) const override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual FCameraCalibrationStepsController* GetCameraCalibrationStepsController() const override;
	virtual bool IsActive() const override;
	virtual UMaterialInstanceDynamic* GetOverlayMID() const override;
	virtual bool IsOverlayEnabled() const override;
	//~ End UCameraCalibrationStep interface

	/** Selects the nodal offset algorithm by name */
	void SetNodalOffsetAlgo(const FName& AlgoName);

	/** Returns the currently selected nodal offset algorithm */
	UCameraNodalOffsetAlgo* GetNodalOffsetAlgo() const;

	/** Called by the UI when the user wants to save the nodal offset that the current algorithm is providing */
	void OnSaveCurrentNodalOffset();

	/** Initiate a new calibration session (if one is not already active) */
	void StartCalibrationSession();

	/** End the current calibration session */
	void EndCalibrationSession();

	/** Increments the session index and returns its new value */
	uint32 AdvanceSessionRowIndex();

	/** Intended to be called by one of the algo objects. Exports the data needed by the algo for the current session to a .json file on disk. */
	void ExportSessionData(const TSharedRef<FJsonObject>& SessionDataObject);

	/** Intended to be called by one of the algo objects. Exports the data associated with a single calibration row to a .json file on disk. */
	void ExportCalibrationRow(int32 RowIndex, const TSharedRef<FJsonObject>& RowObject, const FImageView& RowImage = FImageView());

	/** Intended to be called by one of the algo objects. Deletes the .json file with the input row index that was previously exported for this session. */
	void DeleteExportedRow(const int32& RowIndex) const;

	/** Import a saved calibration data set from disk */
	void ImportCalibrationDataset();

private:
	/** Get the directory for the current session */
	FString GetSessionSaveDir() const;

	/** Get the filename of a row with the input index */
	FString GetRowFilename(int32 RowIndex) const;

public:
	/** Stores info about the current calibration session of this tool */
	FNodalOffsetSessionInfo SessionInfo;

private:

	/** Pointer to the calibration steps controller */
	TWeakPtr<FCameraCalibrationStepsController> CameraCalibrationStepsController;

	/** The currently selected nodal offset algorithm */
	UPROPERTY()
	TObjectPtr<UCameraNodalOffsetAlgo> NodalOffsetAlgo;

	/** Map of algo names to overlay MIDs used by those algos */
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UMaterialInstanceDynamic>> AlgoOverlayMIDs;

	/** True if this tool is the active one in the panel */
	bool bIsActive = false;
};
