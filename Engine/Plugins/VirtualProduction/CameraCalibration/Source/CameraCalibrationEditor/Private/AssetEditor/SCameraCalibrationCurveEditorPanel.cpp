// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCameraCalibrationCurveEditorPanel.h"

#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "ITimeSlider.h"
#include "SCurveKeyDetailPanel.h"
#include "SGridLineSpacingList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "CameraCalibrationCurveEditorPanel"

void SCameraCalibrationCurveEditorPanel::Construct(const FArguments&, TSharedRef<FCurveEditor> InCurveEditor,
                                                   TWeakPtr<ITimeSliderController> InTimeSliderControllerWeakPtr)
{	
	SCurveEditorPanel::Construct(
		SCurveEditorPanel::FArguments()
			.ExternalTimeSliderController(InTimeSliderControllerWeakPtr.Pin()),
			InCurveEditor);

	// Cache the curve editor model
	CurveEditorWeakPtr = InCurveEditor;

	// Disable the Key Details all the time
	const TSharedPtr<SCurveKeyDetailPanel> KeyDetailsViewPtr = GetKeyDetailsView();
	check(KeyDetailsViewPtr.IsValid());
	KeyDetailsViewPtr->SetEnabled(false);
}

TSharedRef<SWidget> SCameraCalibrationCurveEditorPanel::MakeViewModeMenu()
{
	// This builds the dropdown menu when looking at the Curve View Modes
	FMenuBuilder MenuBuilder(true, GetCommands());

	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetViewModeAbsolute);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetViewModeStacked);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetViewModeNormalized);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SCameraCalibrationCurveEditorPanel::MakeCurveEditorCurveViewOptionsMenu()
{
	const TSharedPtr<FCurveEditor> CurveEditorPtr = CurveEditorWeakPtr.Pin();
	check(CurveEditorPtr.IsValid());
	
	// This builds the dropdown menu when looking at the Curve View Options combobox.
	FMenuBuilder MenuBuilder(true, CurveEditorPtr->GetCommands());

	MenuBuilder.BeginSection("TangentVisibility", LOCTEXT("CurveEditorMenuTangentVisibilityHeader", "Tangent Visibility"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetAllTangentsVisibility);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetSelectedKeysTangentVisibility);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().SetNoTangentsVisibility);
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleAutoFrameCurveEditor);
	MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleShowCurveEditorCurveToolTips);

	MenuBuilder.BeginSection("Organize", LOCTEXT("CurveEditorMenuOrganizeHeader", "Organize"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleExpandCollapseNodes);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ToggleExpandCollapseNodesAndDescendants);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SCameraCalibrationCurveEditorPanel::MakeGridSpacingMenu()
{
	const TSharedPtr<FCurveEditor> CurveEditorPtr = CurveEditorWeakPtr.Pin();
	check(CurveEditorPtr.IsValid());
	
	TArray<SGridLineSpacingList::FNamedValue> SpacingAmounts;
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(0.1f, LOCTEXT("OneTenth", "0.1"), LOCTEXT("Description_OneTenth", "Set grid spacing to 1/10th")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(0.5f, LOCTEXT("OneHalf", "0.5"), LOCTEXT("Description_OneHalf", "Set grid spacing to 1/2")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(1.0f, LOCTEXT("One", "1"), LOCTEXT("Description_One", "Set grid spacing to 1")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(2.0f, LOCTEXT("Two", "2"), LOCTEXT("Description_Two", "Set grid spacing to 2")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(5.0f, LOCTEXT("Five", "5"), LOCTEXT("Description_Five", "Set grid spacing to 5")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(10.0f, LOCTEXT("Ten", "10"), LOCTEXT("Description_Ten", "Set grid spacing to 10")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(50.0f, LOCTEXT("Fifty", "50"), LOCTEXT("Description_50", "Set grid spacing to 50")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(100.0f, LOCTEXT("OneHundred", "100"), LOCTEXT("Description_OneHundred", "Set grid spacing to 100")));
	SpacingAmounts.Add(SGridLineSpacingList::FNamedValue(TOptional<float>(), LOCTEXT("Automatic", "Automatic"), LOCTEXT("Description_Automatic", "Set grid spacing to automatic")));

	TSharedRef<SWidget> OutputSnapWidget =
		SNew(SGridLineSpacingList)
		.DropDownValues(SpacingAmounts)
		.MinDesiredValueWidth(60)
		.Value_Lambda([CurveEditorPtr]() -> TOptional<float> { return CurveEditorPtr->FixedGridSpacingAttribute.Get(); })
		.OnValueChanged_Lambda([CurveEditorPtr](TOptional<float> InNewOutputSnap) { CurveEditorPtr->FixedGridSpacingAttribute = InNewOutputSnap; })
		.HeaderText(LOCTEXT("CurveEditorMenuGridSpacingHeader", "Grid Spacing"));

	return OutputSnapWidget;
}

TSharedPtr<FExtender> SCameraCalibrationCurveEditorPanel::GetToolbarExtender()
{
	// We're going to create a new Extender and add the main Curve Editor icons to it.
	// We combine this with the extender provided by the Curve Editor Module as that extender has been extended by tools
	ICurveEditorModule& CurveEditorModule = FModuleManager::Get().LoadModuleChecked<ICurveEditorModule>("CurveEditor");
	//We can create our own toolbar extender for our usage of curve editing.
	//For now, don't add any of the external ones 
	TArray<TSharedPtr<FExtender>> ToolbarExtenders;
	// for (ICurveEditorModule::FCurveEditorMenuExtender& ExtenderCallback : CurveEditorModule.GetAllToolBarMenuExtenders())
	// {
	//  	ToolbarExtenders.Add(ExtenderCallback.Execute(GetCommands().ToSharedRef()));
	// }
	TSharedPtr<FExtender> Extender = FExtender::Combine(ToolbarExtenders);

	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolBarBuilder, TSharedRef<SCurveKeyDetailPanel> InKeyDetailsPanel, TSharedRef<SCameraCalibrationCurveEditorPanel> InEditorPanel)
		{
			ToolBarBuilder.BeginSection("View");
			ToolBarBuilder.BeginStyleOverride("AssetEditorToolbar");
			{
				// Dropdown Menu for choosing your viewing mode
				TAttribute<FSlateIcon> ViewModeIcon;
				ViewModeIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([InEditorPanel] {
					switch (InEditorPanel->GetViewMode())
					{
					case ECurveEditorViewID::Absolute:
						return FCurveEditorCommands::Get().SetViewModeAbsolute->GetIcon();
					case ECurveEditorViewID::Stacked:
						return FCurveEditorCommands::Get().SetViewModeStacked->GetIcon();
					case ECurveEditorViewID::Normalized:
						return FCurveEditorCommands::Get().SetViewModeNormalized->GetIcon();
					default: // EKeyGroupMode::None
						return FCurveEditorCommands::Get().SetAxisSnappingNone->GetIcon();
					}
				}));

				ToolBarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateSP(InEditorPanel, &SCameraCalibrationCurveEditorPanel::MakeViewModeMenu),
					LOCTEXT("ViewModeDropdown", "Curve View Modes"),
					LOCTEXT("ViewModeDropdownToolTip", "Choose the viewing mode for the curves."),
					ViewModeIcon);
			}
			ToolBarBuilder.EndSection();

			ToolBarBuilder.BeginSection("Framing");
			{
				// Framing
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ZoomToFit);
			}
			ToolBarBuilder.EndSection();

			ToolBarBuilder.BeginSection("Visibility");
			{
				// Curve Visibility
				ToolBarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateSP(InEditorPanel, &SCameraCalibrationCurveEditorPanel::MakeCurveEditorCurveViewOptionsMenu),
					LOCTEXT("CurveEditorCurveOptions", "Curves Options"),
					LOCTEXT("CurveEditorCurveOptionsToolTip", "Curve Options"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.VisibilityOptions"));

			}
			ToolBarBuilder.EndSection();

			ToolBarBuilder.BeginSection("Key Details");
			{
				ToolBarBuilder.AddWidget(InKeyDetailsPanel);
			}
			ToolBarBuilder.EndSection();

			ToolBarBuilder.BeginSection("Tools");
			{
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().DeactivateCurrentTool);
			}
			ToolBarBuilder.EndSection();
	
			ToolBarBuilder.BeginSection("Adjustment");
			{
				// Toggle Button for Value Snapping
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ToggleOutputSnapping);
				
				// Dropdown Menu to choose the snapping scale.
				ToolBarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateSP(InEditorPanel, &SCameraCalibrationCurveEditorPanel::MakeGridSpacingMenu),
					LOCTEXT("GridSnappingOptions", "Grid Snapping"),
					LOCTEXT("GridSnappingOptionsToolTip", "Choose the spacing between horizontal grid lines."),
					TAttribute<FSlateIcon>(),
					true);
			}
			ToolBarBuilder.EndSection();
			ToolBarBuilder.BeginSection("Tangents");
			{
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().InterpolationCubicAuto);
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().InterpolationCubicUser);
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().InterpolationCubicBreak);
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().InterpolationLinear);
				ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().InterpolationConstant);
			}
			ToolBarBuilder.EndSection();
		}
	};

	const TSharedPtr<SCurveKeyDetailPanel> KeyDetailsViewPtr = GetKeyDetailsView();
	check(KeyDetailsViewPtr.IsValid());

	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, KeyDetailsViewPtr.ToSharedRef(), SharedThis(this))
	);

	return Extender;
}

#undef LOCTEXT_NAMESPACE