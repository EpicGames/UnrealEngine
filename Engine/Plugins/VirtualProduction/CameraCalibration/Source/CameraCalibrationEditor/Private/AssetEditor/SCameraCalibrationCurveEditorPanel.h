// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCurveEditorPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

class FCurveEditor;
class FCameraCalibrationStepsController;
class ITimeSliderController;

/**
 * Camera Calibration Curve editor widget that reflects the state of an FCurveEditor
 */
class SCameraCalibrationCurveEditorPanel : public SCurveEditorPanel
{
public:
	SLATE_BEGIN_ARGS( SCameraCalibrationCurveEditorPanel ) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor, TWeakPtr<ITimeSliderController> InTimeSliderControllerWeakPtr);
	
	/**
	 * Copied from SCurveEditorPanel in order to have custom order and custom buttons in toolbar
	 */
	TSharedPtr<FExtender> GetToolbarExtender();

private:
	/** Generate the View Mode dropdown toolbar widget */
	TSharedRef<SWidget> MakeViewModeMenu();

	/** Generate the Curve View Options dropdown toolbar widget */
	TSharedRef<SWidget> MakeCurveEditorCurveViewOptionsMenu();

	/** Generate the Time Snap dropdown toolbar widget */
	TSharedRef<SWidget> MakeTimeSnapMenu();

	/** Generate the Grid Spacing dropdown toolbar widget */
	TSharedRef<SWidget> MakeGridSpacingMenu();

private:
	/** The curve editor pointer */
	TWeakPtr<FCurveEditor> CurveEditorWeakPtr;
};