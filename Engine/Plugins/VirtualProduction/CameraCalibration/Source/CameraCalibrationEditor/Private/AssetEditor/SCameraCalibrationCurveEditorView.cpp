// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCameraCalibrationCurveEditorView.h"

#include "CameraCalibrationCurveEditor.h"
#include "CurveEditorCommands.h"
#include "CurveEditorTypes.h"
#include "Curves/LensDataCurveModel.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LensFile.h"
#include "Misc/Optional.h"
#include "SCurveEditorPanel.h"

#define LOCTEXT_NAMESPACE "SCameraCalibrationCurveEditorView"


/**
 * Mostly copy of CurveEditorContextMenu.h
 * Building custom context menu with custom Camera Calibration buttons and handlers
 */
struct FCurveEditorContextMenu
{
	static void BuildMenu(FMenuBuilder& MenuBuilder, const TSharedRef<FCameraCalibrationCurveEditor>& CurveEditor, TOptional<FCurveModelID> HoveredCurveID)
	{
		int32 NumSelectedKeys = CurveEditor->GetSelection().Count();

		TSharedPtr<FCameraCalibrationCurveEditor> LocalCurveEditor = CurveEditor;

		const FCurveModel* HoveredCurve = HoveredCurveID.IsSet() ? CurveEditor->FindCurve(HoveredCurveID.GetValue()) : nullptr;

		// Add curve button action
		const FUIAction AddButtonAction
		(
			FExecuteAction::CreateLambda([LocalCurveEditor]()
			{
				LocalCurveEditor->OnAddDataPointDelegate.ExecuteIfBound();
			})
		);

		// We prioritize key selections over curve selections to reduce the pixel-perfectness needed
		// to edit the keys (which is more common than curves). Right clicking on a key or an empty space
		// should show the key menu, otherwise we show the curve menu (ie: right clicking on a curve, not 
		// directly over a key).
		if (NumSelectedKeys > 0 && !HoveredCurveID.IsSet())
		{
			MenuBuilder.BeginSection("CurveEditorKeySection", FText::Format(LOCTEXT("CurveEditorKeySection", "{0} Selected {0}|plural(one=Key,other=Keys)"), NumSelectedKeys));
			{
				bool bIsReadOnly = false;
				if (HoveredCurve)
				{
					bIsReadOnly = HoveredCurve->IsReadOnly();
				}

				if (!bIsReadOnly)
				{
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().FlattenTangents);
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().StraightenTangents);

					MenuBuilder.AddMenuSeparator();

					// Tangent Types
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicAuto);
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicUser);
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicBreak);
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationLinear);
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationConstant);

					MenuBuilder.AddMenuSeparator();
				}

				// View
				MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ZoomToFit);
			}
			MenuBuilder.EndSection();
		}
		else
		{
			auto CreateUnselectedKeysMenu = [&MenuBuilder, &AddButtonAction](FName InExtensionHook, const TAttribute< FText >& InHeadingText, const bool bInIsCurveReadOnly)
			{
				MenuBuilder.BeginSection(InExtensionHook, InHeadingText);
				if (!bInIsCurveReadOnly)
				{
					// Add Curve Point
					MenuBuilder.AddMenuEntry(
						LOCTEXT("AddDataPointLabel", "Add Data Point"),
						LOCTEXT("AddDataPointTooltip", "Add a new key to all curves at the current time"),
						FSlateIcon(),
						AddButtonAction);

					MenuBuilder.AddMenuSeparator();

					// View
					MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().ZoomToFit);
				}

				MenuBuilder.EndSection();
			};
			
			if (HoveredCurve)
			{
				CreateUnselectedKeysMenu(
					TEXT("CurveEditorCurveSection"),
					FText::Format(LOCTEXT("CurveNameFormat", "Curve '{0}'"), HoveredCurve->GetLongDisplayName()),
					HoveredCurve->IsReadOnly());
			}
			else
			{
				// Test if at least one curve is editable
				bool bIsReadOnly = true;
				TSet<FCurveModelID> CurvesToAddTo;
				for(const FCurveModelID& CurveModelID : CurveEditor->GetEditedCurves())
				{
					if (const FCurveModel* CurveModel = CurveEditor->FindCurve(CurveModelID))
					{
						if (!CurveModel->IsReadOnly())
						{
							bIsReadOnly = false;
							break;
						}
					}
				}

				CreateUnselectedKeysMenu(
					TEXT("CurveEditorAllCurveSections"),
					LOCTEXT("CurveEditorAllCurveSections", "All Curves"),
					bIsReadOnly);
			}
		}
	}
};

void SCameraCalibrationCurveEditorView::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	FArguments ParentArguments;
	ParentArguments.AutoSize(false);
	
	SCurveEditorViewAbsolute::Construct(ParentArguments, InCurveEditor);

	// Build the custom tooltip widget, displaying the curve's title, current key, and current value
	CameraCalibrationCurveToolTipWidget =
		SNew(SToolTip)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SCameraCalibrationCurveEditorView::GetToolTipTitleText)
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
				.ColorAndOpacity(FLinearColor::Black)
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SCameraCalibrationCurveEditorView::GetToolTipKeyText)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SCameraCalibrationCurveEditorView::GetToolTipValueText)
				.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
				.ColorAndOpacity(FLinearColor::Black)
			]
		];

	// This replaces the tooltip widget created by the base interactive curve editor view
	SetToolTip(CameraCalibrationCurveToolTipWidget);
}

FText SCameraCalibrationCurveEditorView::GetToolTipTitleText() const
{
	return CachedToolTipTitleText.IsSet() ? CachedToolTipTitleText.GetValue() : FText();
}

FText SCameraCalibrationCurveEditorView::GetToolTipKeyText() const
{
	return CachedToolTipKeyText.IsSet() ? CachedToolTipKeyText.GetValue() : FText();
}

FText SCameraCalibrationCurveEditorView::GetToolTipValueText() const
{
	return CachedToolTipValueText.IsSet() ? CachedToolTipValueText.GetValue() : FText();
}

FReply SCameraCalibrationCurveEditorView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = Super::OnMouseButtonUp(MyGeometry, MouseEvent);

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// The interactive curve editor view implementation of OnMouseButtonUp will only create a context menu if the mouse is not dragging.
		// Rather than duplicate all of the logic to track the dragging state in this widget, we can check if a context menu currently exists,
		// (implying that the mouse is not dragging) and if it does exist, this widget will override that context menu with its own.
		// Note: In the future, we need to expose the context menu from the base curve editor view to support customization and avoid creating two menus.
		if (FSlateApplication::Get().AnyMenusVisible())
		{
			CreateContextMenu(MyGeometry, MouseEvent);
			return FReply::Handled();
		}
	}

	return Reply;
}

FReply SCameraCalibrationCurveEditorView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Don't handle updating if we have a context menu open.
	if (ActiveContextMenu.Pin())
	{
		return FReply::Unhandled();
	}
	
	// Update the tooltip visibility and text if the user is hovered over a curve
	TOptional<FCurveModelID> OptionalCurveID = GetHoveredCurve();
	if (OptionalCurveID.IsSet())
	{
		CameraCalibrationCurveToolTipWidget->SetVisibility(EVisibility::Visible);

		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		const FCurveModelID HoveredCurveID = OptionalCurveID.GetValue();
		const FCurveModel* const HoveredCurve = CurveEditor->FindCurve(HoveredCurveID);

		if (HoveredCurve && Cast<ULensFile>(HoveredCurve->GetOwningObject()))
		{
			const FLensDataCurveModel* const LensDataCurveModel = static_cast<const FLensDataCurveModel* const>(HoveredCurve);

			const FVector2D MousePixel = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

			const FCurveEditorScreenSpace CurveSpace = GetCurveSpace(HoveredCurveID);
			const double MouseTime = CurveSpace.ScreenToSeconds(MousePixel.X) - LensDataCurveModel->GetInputDisplayOffset();
			const double EvaluatedTime = CurveEditor->GetCurveSnapMetrics(HoveredCurveID).SnapInputSeconds(MouseTime);

			double EvaluatedValue = 0.0;
			LensDataCurveModel->Evaluate(EvaluatedTime, EvaluatedValue);

			CachedToolTipTitleText = FText::Format(LOCTEXT("CurveEditorTooltipName", "{0} Curve"), LensDataCurveModel->GetLongDisplayName());
			CachedToolTipKeyText = FText::Format(LOCTEXT("CurveEditorTime", "{0}: {1}"), LensDataCurveModel->GetKeyLabel(), EvaluatedTime);
			CachedToolTipValueText = FText::Format(LOCTEXT("CurveEditorValue", "Evaluated Value: {0}{1} {2}"), LensDataCurveModel->GetValueUnitPrefixLabel(), EvaluatedValue, LensDataCurveModel->GetValueUnitSuffixLabel());
 		}
	}
	else
	{
		// Hide the tooltip when not hovering over a curve
		CameraCalibrationCurveToolTipWidget->SetVisibility(EVisibility::Collapsed);

		CachedToolTipTitleText.Reset();
		CachedToolTipKeyText.Reset();
		CachedToolTipValueText.Reset();
	}

	return Super::OnMouseMove(MyGeometry, MouseEvent);
}

bool SCameraCalibrationCurveEditorView::IsTimeSnapEnabled() const
{
	//We never want anything playing automatically change X (zoom / time) values for us
	return false;
}

void SCameraCalibrationCurveEditorView::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	Super::PaintView(Args, AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, bParentEnabled);

	TArray<FCurveModelID> CurveIDs;
	const int NumCurveKeys = CurveInfoByID.GetKeys(CurveIDs);

	if (NumCurveKeys > 0)
	{
		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		const FCurveModel* const Curve = CurveEditor->FindCurve(CurveIDs[0]);

		if (Curve && Cast<ULensFile>(Curve->GetOwningObject()))
		{
			const FLensDataCurveModel* const LensDataCurveModel = static_cast<const FLensDataCurveModel* const>(Curve);

			// Draw an axis descriptor for the current curve editor view
			const FText AxisDescriptor = FText::Format(LOCTEXT("CurveEditorAxisDescriptor", "X-Axis: {0}   Y-Axis: {1} {2}"), LensDataCurveModel->GetKeyLabel(), LensDataCurveModel->GetLongDisplayName(), LensDataCurveModel->GetValueLabel());

			const FSlateFontInfo FontInfo = FAppStyle::Get().GetFontStyle("CurveEd.LabelFont");
			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

			const int32 LabelLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::Labels;

			const FVector2D ViewSize = GetViewSpace().GetPhysicalSize();
			const FVector2D LabelSize = FontMeasure->Measure(AxisDescriptor, FontInfo);
			const FVector2D LabelOffset = FVector2D(20.0f, 5.0f);
			const FVector2D LabelPosition = ViewSize - LabelSize - LabelOffset;
			const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(LabelPosition));

			FSlateDrawElement::MakeText(OutDrawElements, LabelLayerId + 1, LabelGeometry, AxisDescriptor, FontInfo, DrawEffects, FLinearColor::White);
		}
	}
}

void SCameraCalibrationCurveEditorView::CreateContextMenu(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	const TSharedPtr<SCurveEditorPanel> EditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditor || !EditorPanel)
	{
		return;
	}

	// Cast to Camera Calibration Curve Editor
	const TSharedPtr<FCameraCalibrationCurveEditor> CameraCalibrationCurveEditor = StaticCastSharedPtr<FCameraCalibrationCurveEditor>(CurveEditor);

	constexpr  bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, EditorPanel->GetCommands());

	FCurveEditorContextMenu::BuildMenu(MenuBuilder, CameraCalibrationCurveEditor.ToSharedRef(), GetHoveredCurve());

	// Push the context menu
	const FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
	ActiveContextMenu = FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(),
	                                                      FSlateApplication::Get().GetCursorPos(),
	                                                      FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

#undef LOCTEXT_NAMESPACE
