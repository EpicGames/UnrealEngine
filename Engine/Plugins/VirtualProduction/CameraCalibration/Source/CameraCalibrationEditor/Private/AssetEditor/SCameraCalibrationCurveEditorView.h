// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/SCurveEditorViewAbsolute.h"
#include "Widgets/SToolTip.h"

class IMenu;

/**
 * A Camera Calibration curve view supporting custom context menu
 */
class SCameraCalibrationCurveEditorView : public SCurveEditorViewAbsolute
{
	using Super = SCurveEditorViewAbsolute;
	
public:
	void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);

	//~ Begin SInteractiveCurveEditorView interface
	virtual bool IsTimeSnapEnabled() const override;
	//~ End SInteractiveCurveEditorView interface

protected:
	//~ Begin SCurveEditorView Interface0
	virtual void PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	//~ Begin SCurveEditorView Interface

private:
	//~ Begin SInteractiveCurveEditorView interface
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SInteractiveCurveEditorView interface

	/** Create context menu for Editor View widget */
	void CreateContextMenu(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Get the title text to use in the curve tooltip */
	FText GetToolTipTitleText() const;

	/** Get the key text to use in the curve tooltip */
	FText GetToolTipKeyText() const;

	/** Get the value text to use in the curve tooltip */
	FText GetToolTipValueText() const;

private:
	/** Track if we have a context menu active. Used to suppress hover updates as it causes flickers in the CanExecute bindings. */
	TWeakPtr<IMenu> ActiveContextMenu;

	/** Customized tooltip widget for use when hovering over camera calibration curves */
	TSharedPtr<SToolTip> CameraCalibrationCurveToolTipWidget;

	/** Title text to use in the curve tooltip */
	TOptional<FText> CachedToolTipTitleText;

	/** Key text to use in the curve tooltip */
	TOptional<FText> CachedToolTipKeyText;

	/** Value text to use in the curve tooltip */
	TOptional<FText> CachedToolTipValueText;
};
