// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCameraCalibrationLinkedPointsDialog.h"

#include "Editor.h"
#include "LensFile.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SCameraCalibrationLinkedPointsDialog"

class FLinkedTreeItem : public TSharedFromThis<FLinkedTreeItem>
{
protected:
	typedef SCameraCalibrationLinkedPointsDialog::FLinkedItem FLinkedItem;
	
public:
	FLinkedTreeItem(const FLinkedItem& InItem, bool bInIsSelected)
		: Item(InItem)
		, bIsSelected(bInIsSelected)
	{ }
	
	virtual ~FLinkedTreeItem() = default;
	
	/** Generate the row widget */
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) = 0;
	
	/** Get children of this item */
	const TArray<TSharedPtr<FLinkedTreeItem>>& GetChildren() const { return Children; }

	void AddChild(const TSharedPtr<FLinkedTreeItem>& InChild) { Children.Add(InChild); }
	
	/** Gets whether this item is selected in the tree view */
	bool IsSelected() const { return bIsSelected; }

	/** Gets the linked item this tree item represents */
	const FLinkedItem& GetItem() const { return Item; }

	/** If this tree item is selected, add its linked item to the specified array, and recurse through its children */
	void AddItemToSelectedLinkedItems(TArray<FLinkedItem>& SelectedLinkedItems)
	{
		for (const TSharedPtr<FLinkedTreeItem>& ChildItem : Children)
		{
			ChildItem->AddItemToSelectedLinkedItems(SelectedLinkedItems);
		}

		if (bIsSelected)
		{
			SelectedLinkedItems.Add(Item);
		}
	}

protected:
	ECheckBoxState GetIsSelectedCheckState() const { return bIsSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }
	virtual void OnCheckStateChanged(ECheckBoxState NewState) { bIsSelected = NewState == ECheckBoxState::Checked; }
	
protected:
	/** The linked item this tree item represents */
	FLinkedItem Item;

	/** Parent of this item */
	TWeakPtr<FLinkedTreeItem> Parent = nullptr;
	
	/** Children of this item */
	TArray<TSharedPtr<FLinkedTreeItem>> Children;

	/** Indicates if this item is selected in the tree view */
	bool bIsSelected = false;

	friend class FLinkedFocusTreeItem;
};

class FLinkedFocusTreeItem : public FLinkedTreeItem
{
public:
	FLinkedFocusTreeItem(const FLinkedItem& InItem, bool bInIsSelected)
		: FLinkedTreeItem(InItem, bInIsSelected)
	{ }

	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) override
	{
		const FName PointName = FBaseLensTable::GetFriendlyPointName(Item.Category);
		const FText FocusLabel = FText::Format(LOCTEXT("FocusLabel", "{0}. Focus: {1}"), FText::FromName(PointName), Item.Focus);
		
		return SNew(STableRow<TSharedPtr<FLinkedTreeItem>>, InOwnerTable)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FLinkedFocusTreeItem::GetIsSelectedCheckState)
				.OnCheckStateChanged(this, &FLinkedFocusTreeItem::OnCheckStateChanged)
			]
			
			+SHorizontalBox::Slot()
			.Padding(5.0f, 5.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(FocusLabel)
			]
		];
	}

protected:
	virtual void OnCheckStateChanged(ECheckBoxState NewState) override
	{
		FLinkedTreeItem::OnCheckStateChanged(NewState);

		// When a focus item is selected, all of its child zoom items are also selected
		for (const TSharedPtr<FLinkedTreeItem>& Child : Children)
		{
			Child->bIsSelected = bIsSelected;
		}
	}
};

class FLinkedZoomTreeItem : public FLinkedTreeItem
{
public:
	FLinkedZoomTreeItem(const FLinkedItem& InItem, bool bInIsSelected, const TSharedPtr<FLinkedTreeItem>& InParent)
		: FLinkedTreeItem(InItem, bInIsSelected)
	{
		Parent = InParent;
	} 

	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) override
	{
		const FName PointName = FBaseLensTable::GetFriendlyPointName(Item.Category);
		const FText ZoomLabel = Parent.IsValid() ?
			FText::Format(LOCTEXT("ZoomLabel", "Zoom: {0}"), Item.Zoom.GetValue()) :
			FText::Format(LOCTEXT("StandaloneZoomLabel", "{0}. Focus: {1}, Zoom: {2}"), FText::FromName(PointName), Item.Focus, Item.Zoom.GetValue());
		
		return SNew(STableRow<TSharedPtr<FLinkedTreeItem>>, InOwnerTable)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FLinkedZoomTreeItem::GetIsSelectedCheckState)
				.OnCheckStateChanged(this, &FLinkedZoomTreeItem::OnCheckStateChanged)
			]
			
			+SHorizontalBox::Slot()
			.Padding(5.0f, 5.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(ZoomLabel)
			]
		];
	}
};

void SCameraCalibrationLinkedPointsDialog::Construct(const FArguments& InArgs , ULensFile* InLensFile, const FLinkedItem& InInitialItem)
{
	WeakLensFile = InLensFile;
	InitialItem = InInitialItem;
	LinkedItemMode = InArgs._LinkedItemMode;
	OnApplyLinkedAction = InArgs._OnApplyLinkedAction;

	ChildSlot
	[
		SNew(SVerticalBox)
		
        +SVerticalBox::Slot()
        .Padding(5.0f, 5.0f)
        .AutoHeight()
        [
        	SNew(SBorder)
        	.HAlign(HAlign_Center)
        	.VAlign(VAlign_Center)
        	.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
        	[
        		SNew(STextBlock).Text(InArgs._DialogText)
        	]
        ]
        
        +SVerticalBox::Slot()
        .Padding(5.0f, 5.0f)
        .FillHeight(1.f)
        [
        	SAssignNew(LinkedItemsTree, STreeView<TSharedPtr<FLinkedTreeItem>>)
			.TreeItemsSource(&LinkedItems)
			.OnGenerateRow_Lambda([](TSharedPtr<FLinkedTreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable) { return InItem->MakeTreeRowWidget(OwnerTable); })
			.OnGetChildren_Lambda([](TSharedPtr<FLinkedTreeItem> InItem, TArray<TSharedPtr<FLinkedTreeItem>>& OutNodes) { OutNodes = InItem->GetChildren(); })
			.ClearSelectionOnClick(false)
        ]

        +SVerticalBox::Slot()
        .Padding(5.0f, 5.0f)
        .AutoHeight()
        [
        	InArgs._Content.Widget
        ]
        
        +SVerticalBox::Slot()
        .Padding(5.0f, 5.0f)
        .AutoHeight()
        [
        	SNew(SBorder)
        	.HAlign(HAlign_Fill)
        	.VAlign(VAlign_Fill)
        	.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
        	[
        		SNew(SHorizontalBox)
        		
				+SHorizontalBox::Slot()
				[
					SNew(SButton)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SCameraCalibrationLinkedPointsDialog::OnAcceptButtonClicked)
					.HAlign(HAlign_Center)
					.Text(InArgs._AcceptButtonText)
				]
				
				+SHorizontalBox::Slot()
				[
					SNew(SButton)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SCameraCalibrationLinkedPointsDialog::OnCancelButtonClicked)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("CancelLabel", "Cancel"))
				]
        	]
        ]
	];
	
	UpdateLinkedItems();
}

void SCameraCalibrationLinkedPointsDialog::OpenWindow(const FText& WindowTitle, const TSharedRef<SCameraCalibrationLinkedPointsDialog>& DialogBox)
{
	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(570, 420));

	DialogBox->WindowWeakPtr = ModalWindow;
	ModalWindow->SetContent(DialogBox);

	GEditor->EditorAddModalWindow(ModalWindow);
}

void SCameraCalibrationLinkedPointsDialog::UpdateLinkedItems()
{
	LinkedItems.Empty();

	if (!WeakLensFile.IsValid())
	{
		LinkedItemsTree->RequestTreeRefresh();
		return;
	}
	
	const FBaseLensTable* const BaseDataTable = WeakLensFile.Get()->GetDataTable(InitialItem.Category);
	if (!ensure(BaseDataTable))
	{
		LinkedItemsTree->RequestTreeRefresh();
		return;	
	}
	
	if (LinkedItemMode == ELinkedItemMode::Zoom && InitialItem.Zoom.IsSet())
	{
		// Only display the linked zoom values and not the focus values

		// Add the initial item to the tree view
		constexpr bool bIsSelected = true;
		LinkedItems.Add(MakeShared<FLinkedZoomTreeItem>(InitialItem, bIsSelected, nullptr));

		BaseDataTable->ForEachLinkedFocusPoint([this, InitialZoom = InitialItem.Zoom.GetValue()](const FBaseFocusPoint& InFocusPoint, ELensDataCategory InCategory, FLinkPointMetadata LinkPointMeta)
		{
			for (int32 Index = 0; Index < InFocusPoint.GetNumPoints(); ++Index)
			{
				const float Zoom = InFocusPoint.GetZoom(Index);
				if (!FMath::IsNearlyEqual(Zoom ,InitialZoom, WeakLensFile->InputTolerance))
				{
					continue;
				}
				
				FLinkedItem LinkedZoomItem(InCategory, InFocusPoint.GetFocus(), Zoom);
				LinkedItems.Add(MakeShared<FLinkedZoomTreeItem>(LinkedZoomItem, LinkPointMeta.bRemoveByDefault, nullptr));
			}
		}, InitialItem.Focus, WeakLensFile->InputTolerance);
	}
	else
	{
		// Add the initial item to the tree view
		BaseDataTable->ForEachFocusPoint([this, Category = InitialItem.Category](const FBaseFocusPoint& InFocusPoint)
		{
			//Add entry for focus
			FLinkedItem FocusLinkedItem(Category, InFocusPoint.GetFocus());
			
			constexpr bool bIsSelected = true;
			const TSharedPtr<FLinkedFocusTreeItem> FocusTreeItem = MakeShared<FLinkedFocusTreeItem>(FocusLinkedItem, bIsSelected);
			LinkedItems.Add(FocusTreeItem);

			LinkedItemsTree->SetItemExpansion(FocusTreeItem, true);

			if (LinkedItemMode == ELinkedItemMode::Both)
			{
				for (int32 Index = 0; Index < InFocusPoint.GetNumPoints(); ++Index)
				{
					//Add zoom points for this focus
					FLinkedItem ZoomLinkedItem(Category, InFocusPoint.GetFocus(), InFocusPoint.GetZoom(Index));
					FocusTreeItem->AddChild(MakeShared<FLinkedZoomTreeItem>(ZoomLinkedItem, bIsSelected, FocusTreeItem));
				}
			}
		}, InitialItem.Focus);

		BaseDataTable->ForEachLinkedFocusPoint([this](const FBaseFocusPoint& InFocusPoint, ELensDataCategory InCategory, FLinkPointMetadata LinkPointMeta)
		{
			FLinkedItem FocusLinkedItem(InCategory, InFocusPoint.GetFocus());
			const TSharedPtr<FLinkedFocusTreeItem> FocusTreeItem = MakeShared<FLinkedFocusTreeItem>(FocusLinkedItem, LinkPointMeta.bRemoveByDefault);
			
			LinkedItems.Add(FocusTreeItem);
			LinkedItemsTree->SetItemExpansion(FocusTreeItem, true);

			if (LinkedItemMode == ELinkedItemMode::Both)
			{
				for(int32 Index = 0; Index < InFocusPoint.GetNumPoints(); ++Index)
				{
					//Add zoom points for this focus
					FLinkedItem ZoomLinkedItem(InCategory, InFocusPoint.GetFocus(), InFocusPoint.GetZoom(Index));
					FocusTreeItem->AddChild(MakeShared<FLinkedZoomTreeItem>(ZoomLinkedItem, LinkPointMeta.bRemoveByDefault, FocusTreeItem));
				}
			}
		}, InitialItem.Focus, WeakLensFile->InputTolerance);
	}

	LinkedItemsTree->RequestTreeRefresh();
}

FReply SCameraCalibrationLinkedPointsDialog::OnAcceptButtonClicked()
{
	TArray<FLinkedItem> SelectedLinkedItems;

	for (const TSharedPtr<FLinkedTreeItem>& TreeItem : LinkedItems)
	{
		TreeItem->AddItemToSelectedLinkedItems(SelectedLinkedItems);
	}

	OnApplyLinkedAction.ExecuteIfBound(SelectedLinkedItems);

	if (WindowWeakPtr.IsValid())
	{
		WindowWeakPtr.Pin()->RequestDestroyWindow();
	}
	
	return FReply::Handled();
}

FReply SCameraCalibrationLinkedPointsDialog::OnCancelButtonClicked()
{
	if (WindowWeakPtr.IsValid())
	{
		WindowWeakPtr.Pin()->RequestDestroyWindow();
	}
	
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
