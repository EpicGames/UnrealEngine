// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LensFile.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FLinkedTreeItem;
class ULensFile;
enum class ELensDataCategory : uint8;

/**
 * A generic dialog box that displays all points linked to a specific point, allowing the user to
 * apply a single operation to multiple linked points
 */
class SCameraCalibrationLinkedPointsDialog : public SCompoundWidget
{
public:
	/** An item within a lens data table that is linked to the current item */
	struct FLinkedItem
	{
		FLinkedItem(ELensDataCategory InCategory, float InFocus)
			: Category(InCategory)
			, Focus(InFocus)
		{ }

		FLinkedItem(ELensDataCategory InCategory, float InFocus, float InZoom)
			: Category(InCategory)
			, Focus(InFocus)
			, Zoom(InZoom)
		{ }
		
		ELensDataCategory Category = (ELensDataCategory)0;
		float Focus = 0.0f;
		TOptional<float> Zoom = TOptional<float>();
	};

	/** Indicates the mode of the dialog box */
	enum class ELinkedItemMode : uint8
	{
		/** Only linked focus items will be displayed and selected from */
		Focus,

		/** Only linked zoom items will be displayed and selected from */
		Zoom,

		/** Both focus and zoom items will be displayed, and their selection state is coupled */
		Both
	};
	
	DECLARE_DELEGATE_OneParam(FOnApplyLinkedAction, const TArray<FLinkedItem>& /**ItemsToApplyActionTo*/);
	
	SLATE_BEGIN_ARGS(SCameraCalibrationLinkedPointsDialog)
		: _AcceptButtonText(NSLOCTEXT("SCameraCalibrationLinkedPointsDialog", "AcceptLabel", "Accept"))
	{ }
		SLATE_ARGUMENT(ELinkedItemMode, LinkedItemMode)
		SLATE_ATTRIBUTE(FText, DialogText)
		SLATE_ATTRIBUTE(FText, AcceptButtonText)
		SLATE_EVENT(FOnApplyLinkedAction, OnApplyLinkedAction)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, ULensFile* InLensFile, const FLinkedItem& InInitialItem);

	/**
	 * Open a new linked points window and block until the window is closed
	 * @param WindowTitle The title of the window
	 * @param DialogBox The dialog widget to display in the window
	 */
	static void OpenWindow(const FText& WindowTitle, const TSharedRef<SCameraCalibrationLinkedPointsDialog>& DialogBox);

private:
	/** Rebuilds the list of linked items and refreshes the tree view */
	void UpdateLinkedItems();
	
	/** Raised when the accept button is clicked */
	FReply OnAcceptButtonClicked();

	/** Raised when the cancel button is clicked */
	FReply OnCancelButtonClicked();
	
private:
	/** Modal window pointer */
	TWeakPtr<SWindow> WindowWeakPtr;

	/** LensFile we're editing */
	TWeakObjectPtr<ULensFile> WeakLensFile = nullptr;

	/** The item linked items are being found for */
	FLinkedItem InitialItem = FLinkedItem((ELensDataCategory)0, 0.0f);
	
	/** A list of items that are linked */
	TArray<TSharedPtr<FLinkedTreeItem>> LinkedItems;

	/** Tree widget to display the linked items */
	TSharedPtr<STreeView<TSharedPtr<FLinkedTreeItem>>> LinkedItemsTree;

	/** The mode of the dialog box */
	ELinkedItemMode LinkedItemMode = ELinkedItemMode::Both;

	/** Delegate to raise when the accept button is pressed and an action needs to be applied to each selected linked item */
	FOnApplyLinkedAction OnApplyLinkedAction;
};
