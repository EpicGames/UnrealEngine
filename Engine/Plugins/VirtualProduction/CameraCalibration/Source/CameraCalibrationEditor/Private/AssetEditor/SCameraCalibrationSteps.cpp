// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCameraCalibrationSteps.h"

#include "AssetRegistry/AssetData.h"
#include "Camera/CameraActor.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationStepsController.h"
#include "CameraCalibrationSubsystem.h"
#include "Engine/Engine.h"
#include "Styling/AppStyle.h"
#include "Engine/Selection.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IContentBrowserSingleton.h"
#include "LensFile.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "PropertyCustomizationHelpers.h"
#include "SSimulcamViewport.h"
#include "Styling/CoreStyle.h"
#include "Profile/MediaProfile.h"
#include "Styling/SlateStyle.h"
#include "UI/CameraCalibrationEditorStyle.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "CameraCalibrationSteps"

void SCameraCalibrationSteps::AddReferencedObjects(FReferenceCollector& Collector)
{
 	if (CurrentOverlayMID)
 	{
		Collector.AddReferencedObject(CurrentOverlayMID);
	}

	for (TPair<FName, TObjectPtr<UMaterialInstanceDynamic>>& Pair : OverlayMIDs)
	{
		if (Pair.Value)
		{
			Collector.AddReferencedObject(Pair.Value);
		}
	}
}

void SCameraCalibrationSteps::Construct(const FArguments& InArgs, TWeakPtr<FCameraCalibrationStepsController> InCalibrationStepsController)
{
	CalibrationStepsController = InCalibrationStepsController;
	check(CalibrationStepsController.IsValid());

	// Create and populate the step switcher with the UI for all the calibration steps
	{
		StepWidgetSwitcher = SNew(SWidgetSwitcher);

		for (const TStrongObjectPtr<UCameraCalibrationStep>& Step: CalibrationStepsController.Pin()->GetCalibrationSteps())
		{
			StepWidgetSwitcher->AddSlot()
				[Step->BuildUI()];
		}
	}
	
	// Make media playback buttons
	TSharedRef<SWidget> RewindButton = SNew(SButton)
		.ToolTipText(LOCTEXT("RewindButtonTooltip", "Rewind the media to the beginning"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnRewindButtonClicked();
			}
			return FReply::Unhandled();
		})
		.IsEnabled_Lambda([WeakStepsController = CalibrationStepsController]() -> bool
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->DoesMediaSupportSeeking();
			}
			return false;
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.RewindMedia.Small")))
		];

	TSharedRef<SWidget> ReverseButton = SNew(SButton)
		.ToolTipText(LOCTEXT("ReverseButtonTooltip", "Reverse media playback"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnReverseButtonClicked();
			}
			return FReply::Unhandled();
		})
		.IsEnabled_Lambda([WeakStepsController = CalibrationStepsController]() -> bool
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->DoesMediaSupportNextReverseRate();
			}
			return false;
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.ReverseMedia.Small")))
		];

	TSharedRef<SWidget> StepBackButton = SNew(SButton)
		.ToolTipText(LOCTEXT("StepBackButtonTooltip", "Step back one frame, or one time interval as set in project settings"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnStepBackButtonClicked();
			}
			return FReply::Unhandled();
		})
		.IsEnabled_Lambda([WeakStepsController = CalibrationStepsController]() -> bool
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->DoesMediaSupportSeeking();
			}
			return false;
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.StepBackMedia.Small")))
		];

	TSharedRef<SWidget> PlayButton = SNew(SButton)
		.ToolTipText(LOCTEXT("PlayButtonTooltip", "Start media playback"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnPlayButtonClicked();
			}
			return FReply::Unhandled();
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.PlayMedia.Small")))
		];

	TSharedRef<SWidget> PauseButton = SNew(SButton)
		.ToolTipText(LOCTEXT("PauseButtonTooltip", "Pause media playback"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnPauseButtonClicked();
			}
			return FReply::Unhandled();
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.PauseMedia.Small")))
		];

	TSharedRef<SWidget> StepForwardButton = SNew(SButton)
		.ToolTipText(LOCTEXT("StepForwardButtonTooltip", "Step forward one frame, or one time interval as set in project settings"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnStepForwardButtonClicked();
			}
			return FReply::Unhandled();
		})
		.IsEnabled_Lambda([WeakStepsController = CalibrationStepsController]() -> bool
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->DoesMediaSupportSeeking();
			}
			return false;
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.StepForwardMedia.Small")))
		];


	TSharedRef<SWidget> ForwardButton = SNew(SButton)
		.ToolTipText(LOCTEXT("ForwardButtonTooltip", "Fast forward media playback"))
		.OnClicked_Lambda([WeakStepsController = CalibrationStepsController]() -> FReply
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->OnForwardButtonClicked();
			}
			return FReply::Unhandled();
		})
		.IsEnabled_Lambda([WeakStepsController = CalibrationStepsController]() -> bool
		{
			if (TSharedPtr<FCameraCalibrationStepsController> StepsControllerShared = WeakStepsController.Pin())
			{
				return StepsControllerShared->DoesMediaSupportNextForwardRate();
			}
			return false;
		})
		[
			SNew(SImage)
			.Image(FCameraCalibrationEditorStyle::Get().GetBrush(TEXT("CameraCalibration.ForwardMedia.Small")))
		];


	ChildSlot
	[		
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(0.75f) 
		[ 
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Steps selection
			.VAlign(EVerticalAlignment::VAlign_Fill)
			.AutoHeight()
			[ BuildStepSelectionWidget() ]

			+ SVerticalBox::Slot() // Simulcam Viewport
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNew(SSimulcamViewport)
				.ViewActor_Lambda([WeakStepsController = CalibrationStepsController]()->ACameraActor*
				{
					if (TSharedPtr<FCameraCalibrationStepsController> PinnedStepsController = WeakStepsController.Pin())
					{
						return PinnedStepsController->GetCamera();
					}

					return nullptr;
				})
				.FilmbackAspectRatio_Lambda([WeakStepsController = CalibrationStepsController]
				{
					if (TSharedPtr<FCameraCalibrationStepsController> PinnedStepsController = WeakStepsController.Pin())
					{
						return PinnedStepsController->GetCameraFeedAspectRatio();
					}

					return 1.0f;
				})
				.MediaOverlayTexture_Lambda([WeakStepsController = CalibrationStepsController]()->UTexture*
				{
					if (TSharedPtr<FCameraCalibrationStepsController> PinnedStepsController = WeakStepsController.Pin())
					{
						return PinnedStepsController->GetMediaOverlayTexture();
					}

					return nullptr;
				})
				.MediaOverlayOpacity_Lambda([WeakStepsController = CalibrationStepsController]
				{
					if (TSharedPtr<FCameraCalibrationStepsController> PinnedStepsController = WeakStepsController.Pin())
					{
						return 1.0f - PinnedStepsController->GetWiperWeight();
					}

					return 0.0f;
				})
				.ToolOverlayMaterial_Lambda([WeakStepsController = CalibrationStepsController]()->UMaterialInterface*
				{
					if (TSharedPtr<FCameraCalibrationStepsController> PinnedStepsController = WeakStepsController.Pin())
					{
						if (PinnedStepsController->IsOverlayEnabled(EOverlayPassType::ToolOverlay))
						{
							return PinnedStepsController->GetOverlayMaterial(EOverlayPassType::ToolOverlay);
						}
					}

					return nullptr;
				})
				.UserOverlayMaterial_Lambda([WeakStepsController = CalibrationStepsController]()->UMaterialInterface*
				{
					if (TSharedPtr<FCameraCalibrationStepsController> PinnedStepsController = WeakStepsController.Pin())
					{
						if (PinnedStepsController->IsOverlayEnabled(EOverlayPassType::UserOverlay))
						{
							return PinnedStepsController->GetOverlayMaterial(EOverlayPassType::UserOverlay);
						}
					}

					return nullptr;
				})
				.OnSimulcamViewportClicked_Raw(CalibrationStepsController.Pin().Get(), &FCameraCalibrationStepsController::OnSimulcamViewportClicked)
				.OnSimulcamViewportInputKey_Raw(CalibrationStepsController.Pin().Get(), &FCameraCalibrationStepsController::OnSimulcamViewportInputKey)
				.OnSimulcamViewportMarqueeSelect_Raw(CalibrationStepsController.Pin().Get(), &FCameraCalibrationStepsController::OnSimulcamViewportMarqueeSelect)
			]

			+ SVerticalBox::Slot() // Media playback buttons
			.AutoHeight()
			.Padding(5)
			[
				SNew(SUniformWrapPanel)
				.HAlign(HAlign_Center)
				.Visibility(this, &SCameraCalibrationSteps::GetMediaPlaybackControlsVisibility)

				+ SUniformWrapPanel::Slot()
				[RewindButton]

				+ SUniformWrapPanel::Slot()
				[ReverseButton]

				+ SUniformWrapPanel::Slot()
				[StepBackButton]

				+ SUniformWrapPanel::Slot()
				[PlayButton]

				+ SUniformWrapPanel::Slot()
				[PauseButton]

				+ SUniformWrapPanel::Slot()
				[StepForwardButton]

				+ SUniformWrapPanel::Slot()
				[ForwardButton]
			]
		]

		+ SHorizontalBox::Slot() // Right toolbar
		.FillWidth(0.25f)
		[ 
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot() // Viewport Title
				.Padding(0, 5)
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
					.MaxDesiredHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
					[
						SNew(SBorder) // Background color for title
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.BorderBackgroundColor(FLinearColor::White)
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(SOverlay) 
							+ SOverlay::Slot() // Used to add left padding to the title
							.Padding(5,0,0,0)
							[
								SNew(STextBlock) // Title text
								.Text(LOCTEXT("ViewportSettings", "Viewport Settings"))
								.TransformPolicy(ETextTransformPolicy::ToUpper)
								.Font(FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.BoldFont")))
								.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
							]
						]
					]
				]

				+ SVerticalBox::Slot() // Wiper
				.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
				[ FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Transparency", "Transparency"), BuildSimulcamWiperWidget())]
				
				+ SVerticalBox::Slot() // Camera picker
				.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
				[ FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Camera", "Camera"), BuildCameraPickerWidget())]
				
				+ SVerticalBox::Slot()
				.AutoHeight()// Media Source picker
				[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("MediaSource", "Media Source"), BuildMediaSourceWidget())]

				+ SVerticalBox::Slot() // Overlay picker
				.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
				[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Overlay", "Overlay"), BuildOverlayWidget())]

				+ SVerticalBox::Slot() // Overlay parameters
				.AutoHeight()
				[
					OverlayParameterWidget.ToSharedRef()
				]

				+ SVerticalBox::Slot() // Step Title
				.Padding(0, 5)
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(SBox) // Constrain the height
					.MinDesiredHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
					.MaxDesiredHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
					[
						SNew(SBorder) // Background color of title
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.BorderBackgroundColor(FLinearColor::White)
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(SOverlay) 
							+ SOverlay::Slot() // Used to add left padding to the title
							.Padding(5, 0, 0, 0)
							[
								SNew(STextBlock) // Title text
								.Text_Lambda([&]() -> FText
								{
									for (const TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationStepsController.Pin()->GetCalibrationSteps())
									{
										if (!Step.IsValid() || !Step->IsActive())
										{
											continue;
										}

										return FText::FromName(Step->FriendlyName());
									}

									return LOCTEXT("StepSettings", "Step");
								})
								.TransformPolicy(ETextTransformPolicy::ToUpper)
								.Font(FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.BoldFont")))
								.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
							]
						]
					]
				]

				+ SVerticalBox::Slot() // Step UI
				.AutoHeight()
				[StepWidgetSwitcher.ToSharedRef()]
			]
		]
	];

	// Select the first step
	for (const TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationStepsController.Pin()->GetCalibrationSteps())
	{
		if (!Step.IsValid())
		{
			continue;
		}

		SelectStep(Step->FriendlyName());
		break;
	}
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildCameraPickerWidget()
{
	return SNew(SObjectPropertyEntryBox)
		.AllowedClass(ACameraActor::StaticClass())
		.OnObjectChanged_Lambda([&](const FAssetData& AssetData)
		{
			if (AssetData.IsValid())
			{
				CalibrationStepsController.Pin()->SetCamera(Cast<ACameraActor>(AssetData.GetAsset()));
			}
		})
		.ObjectPath_Lambda([&]() -> FString
		{
			if (ACameraActor* Camera = CalibrationStepsController.Pin()->GetCamera())
			{
				FAssetData AssetData(Camera, true);
				return AssetData.GetObjectPathString();
			}

			return TEXT("");
		});
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildSimulcamWiperWidget()
{
	return SNew(SSpinBox<float>)
		.Value_Lambda([&]() { return CalibrationStepsController.Pin()->GetWiperWeight(); })
		.ToolTipText(LOCTEXT("CGWiper", "CG/Media Wiper"))
		.OnValueChanged_Lambda([&](double InValue)
		{
			CalibrationStepsController.Pin()->SetWiperWeight(float(InValue));
		})
		.MinValue(0.0f)
		.MaxValue(1.0f)
		.Delta(0.01f);
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildMediaSourceTypeWidget()
{
	static TArray<TSharedPtr<EMediaSourceType>> MediaSourceTypes =
	{
		MakeShared<EMediaSourceType>(EMediaSourceType::MediaSource),
		MakeShared<EMediaSourceType>(EMediaSourceType::MediaTexture),
		MakeShared<EMediaSourceType>(EMediaSourceType::MediaProfile),
		MakeShared<EMediaSourceType>(EMediaSourceType::None),
	};

	static auto ToDisplayText = [](EMediaSourceType Type)
	{
		switch (Type)
		{
		case EMediaSourceType::MediaSource: return LOCTEXT("MediaSourceTypeLabel", "Media Source");
		case EMediaSourceType::MediaTexture: return LOCTEXT("MediaTextureTypeLabel", "Media Texture");
		case EMediaSourceType::MediaProfile: return LOCTEXT("MediaProfileTypeLabel", "Media Profile");
		case EMediaSourceType::None: return LOCTEXT("MediaNoneTypeLabel", "None");
		default: return FText::GetEmpty();
		}
	};
	
	return SNew(SComboBox<TSharedPtr<EMediaSourceType>>)
	.OptionsSource(&MediaSourceTypes)
	.OnGenerateWidget_Lambda([](TSharedPtr<EMediaSourceType> Type)
	{
		return SNew(STextBlock).Text(ToDisplayText(*Type));
	})
	.OnSelectionChanged_Lambda([this](TSharedPtr<EMediaSourceType> SelectedType, ESelectInfo::Type SelectType)
	{
		MediaSourceType = *SelectedType;

		// Clear out any selected media sources
		if (MediaSourceType == EMediaSourceType::None)
		{
			if (CalibrationStepsController.IsValid())
			{
				CalibrationStepsController.Pin()->ClearMedia();
				MediaProfileSourcesComboBox->SetSelectedItem(nullptr);
			}
		}
	})
	[
		SNew(STextBlock)
		.Text_Lambda([this]()
		{
			return ToDisplayText(MediaSourceType);
		})
	];
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildMediaSourceWidget()
{
	return SNew(SVerticalBox)

	+SVerticalBox::Slot()
	.AutoHeight()
	.Padding(0.0, 2.0)
	[
		BuildMediaSourceTypeWidget()
	]

	+SVerticalBox::Slot()
	.AutoHeight()
	.Padding(0.0, 2.0)
	[
		SNew(SWidgetSwitcher)
		.Visibility_Lambda([this]() { return MediaSourceType != EMediaSourceType::None ? EVisibility::Visible : EVisibility::Collapsed; })
		.WidgetIndex_Lambda([this]() { return (int32)MediaSourceType; })

		+SWidgetSwitcher::Slot()
		[
			BuildMediaSourceAssetPicker()
		]

		+SWidgetSwitcher::Slot()
		[
			BuildMediaTextureAssetPicker()
		]

		+SWidgetSwitcher::Slot()
		[
			BuildMediaProfileSourcePicker()
		]
	];
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildMediaSourceAssetPicker()
{
	return SNew(SObjectPropertyEntryBox)
		.AllowedClass(UMediaSource::StaticClass())
		.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
		.AllowCreate(false)
		.AllowClear(true)
		.ObjectPath_Lambda([this]()
		{
			if (!CalibrationStepsController.IsValid())
			{
				return FString();
			}

			UMediaSource* MediaSource = CalibrationStepsController.Pin()->GetMediaSource();
			if (!MediaSource)
			{
				return FString();
			}

			return MediaSource->GetPathName();
		})
		.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
		{
			if (!CalibrationStepsController.IsValid())
			{
				return;
			}

			CalibrationStepsController.Pin()->SetMediaSource(Cast<UMediaSource>(InAssetData.GetAsset()));
			MediaProfileSourcesComboBox->SetSelectedItem(nullptr);
		});
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildMediaTextureAssetPicker()
{
	return SNew(SObjectPropertyEntryBox)
		.AllowedClass(UMediaTexture::StaticClass())
		.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
		.AllowCreate(false)
		.AllowClear(true)
		.ObjectPath_Lambda([this]()
		{
			if (!CalibrationStepsController.IsValid())
			{
				return FString();
			}

			UMediaTexture* MediaTexture = CalibrationStepsController.Pin()->GetMediaTexture();
			if (!MediaTexture)
			{
				return FString();
			}

			return MediaTexture->GetPathName();
		})
		.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
		{
			if (!CalibrationStepsController.IsValid())
			{
				return;
			}
			
			CalibrationStepsController.Pin()->SetMediaTexture(Cast<UMediaTexture>(InAssetData.GetAsset()));
			MediaProfileSourcesComboBox->SetSelectedItem(nullptr);
		});
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildMediaProfileSourcePicker()
{
	return SAssignNew(MediaProfileSourcesComboBox, SComboBox<TWeakObjectPtr<UMediaSource>>)
		.OptionsSource(&MediaProfileSources)
		.OnComboBoxOpening_Lambda([this]()
		{
			MediaProfileSources.Empty();

			if (!CalibrationStepsController.IsValid())
			{
				MediaProfileSourcesComboBox->RefreshOptions();
				return;
			}

			CalibrationStepsController.Pin()->GetMediaProfileSources(MediaProfileSources);
			MediaProfileSourcesComboBox->RefreshOptions();
		})
		.OnSelectionChanged_Lambda([&](TWeakObjectPtr<UMediaSource> NewValue, ESelectInfo::Type Type) -> void
		{
			if (Type == ESelectInfo::Direct)
			{
				return;
			}
			
			if (!CalibrationStepsController.IsValid() || !NewValue.IsValid())
			{
				return;
			}

			CalibrationStepsController.Pin()->SetMediaProfileSource(NewValue.Get());
		})
		.OnGenerateWidget_Lambda([&](TWeakObjectPtr<UMediaSource> InMediaSource) -> TSharedRef<SWidget>
		{
			FText MediaSourceUrl = FText::GetEmpty();
			if (InMediaSource.IsValid())
			{
				MediaSourceUrl = FText::FromString(InMediaSource->GetUrl());
			}
			
			return SNew(STextBlock).Text(MediaSourceUrl);
		})
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				const FText None = LOCTEXT("NoneComboOption", "None");
				if (!CalibrationStepsController.IsValid())
				{
					return None;
				}

				UMediaSource* MediaSource = CalibrationStepsController.Pin()->GetMediaProfileSource();
				if (!MediaSource)
				{
					return None;
				}

				if (!MediaSource->GetOuter()->IsA<UMediaProfile>())
				{
					return None;
				}

				return FText::FromString(MediaSource->GetUrl());
			})
		];
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildOverlayWidget()
{
	const UCameraCalibrationSubsystem* const SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();

	for (const FName& Name : SubSystem->GetOverlayMaterialNames())
	{
		SharedOverlayNames.Add(MakeShared<FName>(Name));

		if (UMaterialInterface* OverlayMaterial = SubSystem->GetOverlayMaterial(Name))
		{
			OverlayMIDs.Add(Name, UMaterialInstanceDynamic::Create(OverlayMaterial, GetTransientPackage()));
		}
	}

	SharedOverlayNames.Sort([](const TSharedPtr<FName>& LHS, const TSharedPtr<FName>& RHS) { return LHS->Compare(*RHS) <= 0; });

	SharedOverlayNames.Insert(MakeShared<FName>(FName(TEXT("None"))), 0);

	OverlayComboBox = SNew(SComboBox<TSharedPtr<FName>>)
		.OptionsSource(&SharedOverlayNames)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FName> NewValue, ESelectInfo::Type Type) -> void
		{
			if (CalibrationStepsController.IsValid())
			{
				CurrentOverlayMID = OverlayMIDs.FindRef(*NewValue).Get();
				CalibrationStepsController.Pin()->SetOverlayMaterial(CurrentOverlayMID, true, EOverlayPassType::UserOverlay);
				UpdateOverlayMaterialParameterWidget();
			}
		})
		.OnGenerateWidget_Lambda([&](TSharedPtr<FName> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock).Text(FText::FromName(*InOption));
		})
		.InitiallySelectedItem(nullptr)
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				if (OverlayComboBox.IsValid() && OverlayComboBox->GetSelectedItem().IsValid())
				{
					return FText::FromName(*OverlayComboBox->GetSelectedItem());
				}

				return LOCTEXT("NoneComboOption", "None");
			})
		];

	OverlayParameterWidget = SNew(SHorizontalBox);
	OverlayParameterListWidget = SNew(SVerticalBox);

	return OverlayComboBox.ToSharedRef();
}

void SCameraCalibrationSteps::UpdateOverlayMaterialParameterWidget()
{
	OverlayParameterWidget->ClearChildren();
	OverlayParameterListWidget->ClearChildren();

	if (!CalibrationStepsController.IsValid())
	{
		return;
	}

	if (CalibrationStepsController.Pin()->IsOverlayEnabled(EOverlayPassType::UserOverlay))
	{
		if (CurrentOverlayMID)
		{
			TMap<FMaterialParameterInfo, FMaterialParameterMetadata> ScalarParams;
			CurrentOverlayMID->GetAllParametersOfType(EMaterialParameterType::Scalar, ScalarParams);

			TMap<FMaterialParameterInfo, FMaterialParameterMetadata> VectorParams;
			CurrentOverlayMID->GetAllParametersOfType(EMaterialParameterType::Vector, VectorParams);

			// Early-exit if there are no material parameters to display
			if (ScalarParams.Num() + VectorParams.Num() == 0)
			{
				return;
			}

			for (const TPair<FMaterialParameterInfo, FMaterialParameterMetadata>& Param : ScalarParams)
			{
				const FMaterialParameterInfo ParameterInfo = Param.Key;
				const FMaterialParameterMetadata ParameterData = Param.Value;

				OverlayParameterListWidget->AddSlot()
				.Padding(5, 5)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(FText::FromString(ParameterInfo.Name.ToString()))
					]

					+ SHorizontalBox::Slot()
					[
						SNew(SNumericEntryBox<float>)
						.AllowSpin(true)
						.MinValue(TOptional<float>())
						.MaxValue(TOptional<float>())
						.Delta(0.0f)
						.Value_Lambda([this, ParameterInfo]() 
						{ 
							float ScalarValue = 0.0f;
							
							if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
							{
								Overlay->GetScalarParameterValue(ParameterInfo.Name, ScalarValue);
							}

							return ScalarValue;
						})
						.MinSliderValue_Lambda([this, ParameterInfo]()
						{ 
							float MinValue = 0.0f;
							float MaxValue = 0.0f;

							if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
							{
								Overlay->GetScalarParameterSliderMinMax(ParameterInfo.Name, MinValue, MaxValue);
							}

							return MinValue;
						})
						.MaxSliderValue_Lambda([this, ParameterInfo]()
						{ 
							float MinValue = 0.0f;
							float MaxValue = 0.0f;
								
							if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
							{
								Overlay->GetScalarParameterSliderMinMax(ParameterInfo.Name, MinValue, MaxValue);
							}

							return MaxValue;
						})
						.OnValueChanged_Lambda([this, ParameterInfo](float NewValue)
						{ 
							if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
							{
								Overlay->SetScalarParameterValue(ParameterInfo.Name, NewValue);
							}
						})
						.OnValueCommitted_Lambda([this, ParameterInfo](float NewValue, ETextCommit::Type)
						{
							if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
							{
								Overlay->SetScalarParameterValue(ParameterInfo.Name, NewValue);
							}
						})
					]
				]
				;
			}

			for (const TPair<FMaterialParameterInfo, FMaterialParameterMetadata>& Param : VectorParams)
			{
				const FMaterialParameterInfo ParameterInfo = Param.Key;
				const FMaterialParameterMetadata ParameterData = Param.Value;

				OverlayParameterListWidget->AddSlot()
				.Padding(5, 5)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(FText::FromString(ParameterInfo.Name.ToString()))
					]

					+ SHorizontalBox::Slot()
					[
						SNew(SComboButton)
						.ContentPadding(0)
						.HasDownArrow(false)
						.CollapseMenuOnParentFocus(true)
						.ButtonStyle(FAppStyle::Get(), "Sequencer.AnimationOutliner.ColorStrip") // Style matches the button used in cinematic film overlays
						.OnGetMenuContent_Lambda([this, ParameterInfo]() -> TSharedRef<SWidget>
						{
							return SNew(SColorPicker)
								.UseAlpha(true)
								.TargetColorAttribute_Lambda([this, ParameterInfo]()
								{
									FLinearColor ColorValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
										
									if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
									{
										Overlay->GetVectorParameterValue(ParameterInfo.Name, ColorValue);
									}

									return ColorValue;
								})
								.OnColorCommitted_Lambda([this, ParameterInfo](FLinearColor NewColor)
								{
									if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
									{
										Overlay->SetVectorParameterValue(ParameterInfo.Name, NewColor);
									}
								});
						})
						.ButtonContent()
						[
							SNew(SColorBlock)
							.ShowBackgroundForAlpha(true)
							.Size(FVector2D(10.0f, 20.0f))
							.Color_Lambda([this, ParameterInfo]()
							{
								FLinearColor ColorValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

								if (UMaterialInstanceDynamic* Overlay = CurrentOverlayMID.Get())
								{
									Overlay->GetVectorParameterValue(ParameterInfo.Name, ColorValue);
								}

								return ColorValue;
							})
						]
					]
				]
				;
			}

			OverlayParameterWidget->AddSlot()
				.VAlign(VAlign_Top)
				.Padding(5, 10)
				.FillWidth(0.35f)
				[
					SNew(STextBlock).Text(LOCTEXT("OverlayParams", "Overlay Parameters"))
				]
			;

			OverlayParameterWidget->AddSlot()
				.VAlign(VAlign_Center)
				.Padding(10, 5)
				.FillWidth(0.65f)
				[
					OverlayParameterListWidget.ToSharedRef()
				]
			;
		}
	}
}

TSharedRef<SWidget> SCameraCalibrationSteps::BuildStepSelectionWidget()
{
	if (!CalibrationStepsController.IsValid())
	{
		return SNew(SHorizontalBox);
	}

	StepToggles.Empty();

	TSharedPtr<SHorizontalBox> ToggleButtonsBox = SNew(SHorizontalBox);

	for (const TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationStepsController.Pin()->GetCalibrationSteps())
	{
		if (!Step.IsValid())
		{
			continue;
		}
	
		const FName StepName = Step->FriendlyName();

		TSharedPtr<SCheckBox> ToggleButton = SNew(SCheckBox) // Toggle buttons are implemented as checkboxes
			.Style(FAppStyle::Get(), "PlacementBrowser.Tab")
			.OnCheckStateChanged_Lambda([&, StepName](ECheckBoxState CheckState)->void
			{
				SelectStep(StepName);
			})
			.IsChecked_Lambda([&, StepName]() -> ECheckBoxState
			{
				// Note: This will be called every tick

				if (!CalibrationStepsController.IsValid())
				{
					return ECheckBoxState::Unchecked;
				}

				// Return checked state only for the active step
				for (const TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationStepsController.Pin()->GetCalibrationSteps())
				{
					if (!Step.IsValid())
					{
						continue;
					}

					if (Step->FriendlyName() == StepName)
					{
						return Step->IsActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
				}

				return ECheckBoxState::Unchecked;
			})
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.Padding(FMargin(6, 0, 0, 0))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("PlacementBrowser.Tab.Text"))
					.Text(FText::FromName(Step->FriendlyName()))
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Fill)
				.Padding(0,0,0,5) // This separates the line from the bottom and makes it more discernible against unpredictable media plate colors.
				[
					SNew(SImage) // Draws line that enforces the indication of the selected step
					.Image_Lambda([&, StepName]() -> const FSlateBrush*
					{
						// Note: This will be called every tick

						if (!CalibrationStepsController.IsValid())
						{
								return nullptr;
						}

						for (const TStrongObjectPtr<UCameraCalibrationStep>& Step : CalibrationStepsController.Pin()->GetCalibrationSteps())
						{
							if (!Step.IsValid())
							{
								continue;
							}

							if (Step->FriendlyName() == StepName)
							{
								return Step->IsActive() ? FAppStyle::GetBrush(TEXT("PlacementBrowser.ActiveTabBar")) : nullptr;
							}
						}

						return nullptr;
					})
				]
			];

		StepToggles.Add(StepName, ToggleButton);

		ToggleButtonsBox->AddSlot()
		[ToggleButton.ToSharedRef() ];
	}

	return SNew(SBox)
		.MinDesiredHeight(1.5 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		.MaxDesiredHeight(1.5 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[ToggleButtonsBox.ToSharedRef()];
}

void SCameraCalibrationSteps::SelectStep(const FName& StepName)
{
	if (!CalibrationStepsController.IsValid() || !StepWidgetSwitcher.IsValid())
	{
		UE_LOGF(LogCameraCalibrationEditor, Error, "CalibrationStepsController and/or StepWidgetSwitcher were unexpectedly invalid");
		return;
	}

	// Tell the steps controller that the user has selected a different step.
	CalibrationStepsController.Pin()->SelectStep(StepName);

	// Switch the UI to the selected step

	int32 StepIdx = 0;

	for (const TStrongObjectPtr<UCameraCalibrationStep>& Step: CalibrationStepsController.Pin()->GetCalibrationSteps())
	{
		if (!Step.IsValid())
		{
			continue;
		}

		if (Step->FriendlyName() == StepName)
		{
			StepWidgetSwitcher->SetActiveWidgetIndex(StepIdx);
			break;
		}

		StepIdx++;
	}
}

EVisibility SCameraCalibrationSteps::GetMediaPlaybackControlsVisibility() const
{
	TSharedPtr<FCameraCalibrationStepsController> StepsController = CalibrationStepsController.Pin();
	if (StepsController)
	{
		return StepsController->AreMediaPlaybackControlsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
