// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CameraCalibrationStepsController.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"

class FString;
class SCheckBox;

template<typename OptionType>
class SComboBox;
class SHorizontalBox;
class SVerticalBox;
class SWidgetSwitcher;

struct FArguments;

/**
 * UI for the nodal offset calibration.
 * It also holds the UI given by the selected nodal offset algorithm.
 */
class SCameraCalibrationSteps : public SCompoundWidget, public FGCObject
{
	/** Type indicating the kind of media source the user can pick from */
	enum class EMediaSourceType : uint8
	{
		MediaSource,
		MediaTexture,
		MediaProfile,
		None
	};
	
	SLATE_BEGIN_ARGS(SCameraCalibrationSteps) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TWeakPtr<FCameraCalibrationStepsController> InCalibrationStepsController);

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SCameraCalibrationSteps");
	}
	//~ End FGCObject interface

private:

	/** Builds the UI used to pick the camera used for the CG layer of the comp */
	TSharedRef<SWidget> BuildCameraPickerWidget();

	/** Builds the UI for the simulcam wiper */
	TSharedRef<SWidget> BuildSimulcamWiperWidget();

	/** Builds the media source type picker */
	TSharedRef<SWidget> BuildMediaSourceTypeWidget();
	
	/** Builds the UI for the media source picker */
	TSharedRef<SWidget> BuildMediaSourceWidget();

	/** Builds an asset picker for media source assets */
	TSharedRef<SWidget> BuildMediaSourceAssetPicker();

	/** Builds an asset picker for media texture assets */
	TSharedRef<SWidget> BuildMediaTextureAssetPicker();

	/** Builds a combo box that displays the media source in the current media profile */
	TSharedRef<SWidget> BuildMediaProfileSourcePicker();
	
	/** Builds the UI for the overlay picker */
	TSharedRef<SWidget> BuildOverlayWidget();

	/** Builds the UI for the calibration step selection */
	TSharedRef<SWidget> BuildStepSelectionWidget();

	/** Updates the material parameter widget to display the parameters for the currently selected overlay */
	void UpdateOverlayMaterialParameterWidget();

	/** Expected to be called when user selects a new step via the UI */
	void SelectStep(const FName& StepName);

	/** Determines the visibility of the media playback control buttons */
	EVisibility GetMediaPlaybackControlsVisibility() const;

private:

	/** The controller object */
	TWeakPtr<class FCameraCalibrationStepsController> CalibrationStepsController;

	/** The currently selected media source type */
	EMediaSourceType MediaSourceType = EMediaSourceType::MediaProfile;
	
	/** Options source for the MediaSourcesComboBox. Lists the currently available media sources */
	TArray<TWeakObjectPtr<UMediaSource>> MediaProfileSources;

	/** The combobox that presents the available media sources */
	TSharedPtr<SComboBox<TWeakObjectPtr<UMediaSource>>> MediaProfileSourcesComboBox;

	/** The combox that presents the available overlays */
	TSharedPtr<SComboBox<TSharedPtr<FName>>> OverlayComboBox;

	/** The overlay parameter widget that displays a category name and the parameter list */
	TSharedPtr<SHorizontalBox> OverlayParameterWidget;

	/** The overlay parameter list widget that display the scalar and vector parameters for the currently select overlay */
	TSharedPtr<SVerticalBox> OverlayParameterListWidget;

	/** List of overlay names for the combox box options */
	TArray<TSharedPtr<FName>> SharedOverlayNames;

	/** Map of overlay names to MIDs to use for those overlays */
	TMap<FName, TObjectPtr<UMaterialInstanceDynamic>> OverlayMIDs;

	/** Current overlay MID in use, set by the overlay combo box */
	TObjectPtr<UMaterialInstanceDynamic> CurrentOverlayMID;

	/** Widget switcher to only display the UI of the selected step */
	TSharedPtr<SWidgetSwitcher> StepWidgetSwitcher;

	/** Step selection buttons */
	TMap<FName, TSharedPtr<SCheckBox>> StepToggles;
};
