// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImageCenterToolPanel.h"

#include "CameraCalibrationSubsystem.h"
#include "Dialog/SCustomDialog.h"
#include "EditorFontGlyphs.h"
#include "Engine/Engine.h"
#include "ImageCenterTool.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "ImageCenterTool"


void SImageCenterToolPanel::Construct(const FArguments& InArgs, UImageCenterTool* InImageCenterTool)
{
	ImageCenterTool = InImageCenterTool;

	// This will be the widget wrapper of the custom algo UI.
	ImageCenterUI = SNew(SVerticalBox);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot() // Right toolbar
		.FillWidth(0.25f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Algo picker
			.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
			[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("ImageCenterAlgo", "Image Center Algo"), BuildImageCenterAlgoPickerWidget())]

			+ SVerticalBox::Slot() // Algo UI
			.AutoHeight()
			[BuildImageCenterUIWrapper()]

			+ SVerticalBox::Slot() // Save Image Center
			.AutoHeight()
			.Padding(0, 20)
			[
				SNew(SButton).Text(LOCTEXT("AddToImageCenterLUT", "Update Image Center Calibration"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([&]() -> FReply
				{
					if (ImageCenterTool.IsValid())
					{
						ImageCenterTool->OnSaveCurrentImageCenter();
					}
					return FReply::Handled();
				})
			]
		]
	];
}

TSharedRef<SWidget> SImageCenterToolPanel::BuildImageCenterUIWrapper()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Algo's Widget
		.AutoHeight()
		[ImageCenterUI.ToSharedRef()];
}

void SImageCenterToolPanel::UpdateImageCenterUI()
{
	check(AlgosComboBox.IsValid());

	// Get current algo to later compare with new one
	const UCameraImageCenterAlgo* OldAlgo = ImageCenterTool->GetAlgo();

	// Set new algo by name
	const FName AlgoName(*AlgosComboBox->GetSelectedItem());
	ImageCenterTool->SetAlgo(AlgoName);

	// Get the new algo
	UCameraImageCenterAlgo* Algo = ImageCenterTool->GetAlgo();

	// nullptr may indicate that it was unregistered, so refresh combobox options.
	if (!Algo)
	{
		UpdateAlgosOptions();
		return;
	}

	// If we didn't change the algo, we're done here.
	if (Algo == OldAlgo)
	{
		return;
	}

	// Remove old UI
	check(ImageCenterUI.IsValid());
	ImageCenterUI->ClearChildren();

	// Assign GUI
	ImageCenterUI->AddSlot()[Algo->BuildUI()];
}

void SImageCenterToolPanel::UpdateAlgosOptions()
{
	CurrentAlgos.Empty();

	// Ask the subsystem for the list of registered Algos
	UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	check(Subsystem);

	for (FName& Name : Subsystem->GetCameraImageCenterAlgos())
	{
		CurrentAlgos.Add(MakeShared<FString>(Name.ToString()));
	}

	// Ask the ComboBox to refresh its options from its source (that we just updated)
	AlgosComboBox->RefreshOptions();
}

TSharedRef<SWidget> SImageCenterToolPanel::BuildImageCenterAlgoPickerWidget()
{
	// Create ComboBox widget

	AlgosComboBox = SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&CurrentAlgos)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FString> NewValue, ESelectInfo::Type Type) -> void
		{
			// Replace the custom algo widget
			UpdateImageCenterUI();
		})
		.OnGenerateWidget_Lambda([&](TSharedPtr<FString> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock).Text(FText::FromString(*InOption));
		})
		.InitiallySelectedItem(nullptr)
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				if (AlgosComboBox.IsValid() && AlgosComboBox->GetSelectedItem().IsValid())
				{
					return FText::FromString(*AlgosComboBox->GetSelectedItem());
				}

				return LOCTEXT("InvalidComboOption", "Invalid");
			})
		];

	// Update the object holding this combobox's options source
	UpdateAlgosOptions();

	// Pick the first option by default (if available)
	if (CurrentAlgos.Num())
	{
		AlgosComboBox->SetSelectedItem(CurrentAlgos[0]);
	}
	else
	{
		AlgosComboBox->SetSelectedItem(nullptr);
	}

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot() // algo picker
		[
			AlgosComboBox.ToSharedRef()
		]

		+ SHorizontalBox::Slot() // Help button
		.AutoWidth()
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("ShowHelp_Tip", "Help about this algo"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked_Lambda([&]() -> FReply
			{
				if (!ImageCenterTool.IsValid())
				{
					return FReply::Handled();
				}

				UCameraImageCenterAlgo* Algo = ImageCenterTool->GetAlgo();

				if (!Algo)
				{
					return FReply::Handled();
				}

				TSharedRef<SCustomDialog> AlgoHelpWindow =
					SNew(SCustomDialog)
					.Title(FText::FromName(ImageCenterTool->FriendlyName()))
					.Content()
					[
						Algo->BuildHelpWidget()
					]
					.Buttons({ SCustomDialog::FButton(LOCTEXT("Ok", "Ok")) });

				AlgoHelpWindow->Show();

				return FReply::Handled();
			})
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.12"))
				.Text(FEditorFontGlyphs::Info_Circle)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
	;
}


#undef LOCTEXT_NAMESPACE
