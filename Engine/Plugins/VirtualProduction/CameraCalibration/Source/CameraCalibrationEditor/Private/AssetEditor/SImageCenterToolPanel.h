// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
 
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

template<typename OptionType>
class SComboBox;

class UImageCenterTool;

/**
 * UI for the image center adjustment tool.
 * It also holds the UI given by the selected image center algorithm.
 */
class SImageCenterToolPanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SImageCenterToolPanel) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UImageCenterTool* InImageCenterTool);

private:

	/** Builds the wrapper for the currently selected iamge center UI */
	TSharedRef<SWidget> BuildImageCenterUIWrapper();

	/** Builds the UI for the image center algorithm picker */
	TSharedRef<SWidget> BuildImageCenterAlgoPickerWidget();

	/** Updates the UI so that it matches the selected image center algorithm (if necessary) */
	void UpdateImageCenterUI();

	/** Refreshes the list of available image center algorithms shown in the AlgosComboBox */
	void UpdateAlgosOptions();

private:

	/** The image center tool controller object */
	TWeakObjectPtr<class UImageCenterTool> ImageCenterTool;

	/** The box containing the UI given by the selected image center algorithm */
	TSharedPtr<class SVerticalBox> ImageCenterUI;

private:

	/** Options source for the AlgosComboBox. Lists the currently available image center algorithms */
	TArray<TSharedPtr<FString>> CurrentAlgos;

	/** The combobox that presents the available image center algorithms */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> AlgosComboBox;
};
