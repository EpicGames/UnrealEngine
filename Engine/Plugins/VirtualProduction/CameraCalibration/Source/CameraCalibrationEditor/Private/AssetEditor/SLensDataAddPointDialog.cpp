// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensDataAddPointDialog.h"

#include "CameraCalibrationSettings.h"
#include "CameraCalibrationToolkit.h"
#include "Curves/RichCurve.h"
#include "EngineAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "ISinglePropertyView.h"
#include "IStructureDetailsView.h"
#include "LensFile.h"
#include "Models/LensModel.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "LensDataAddPointDialog"

namespace UE::LensAddDataAnalytics
{
	void RecordEvent(ELensDataCategory Category)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;

			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Category"), *UEnum::GetDisplayValueAsText(Category).ToString()));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CameraCalibration.AddLensDataPoint"), EventAttributes);
		}
	}
}

/**
 * Instanced customization used to display the distortion parameters
 * data structure associated with selected model
 */
class FDistortionInfoCustomization : public IPropertyTypeCustomization
{
public:

	FDistortionInfoCustomization(TSubclassOf<ULensModel> InLensModel)
		: IPropertyTypeCustomization()
		, CurrentLensModel(MoveTemp(InLensModel))
	{}

	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		if (CurrentLensModel)
		{
			FStructureDetailsViewArgs StructureViewArgs;
			FDetailsViewArgs DetailArgs;
			DetailArgs.bAllowSearch = false;
			DetailArgs.bShowScrollBar = true;

			FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

			DistortionParameters = MakeShared<FStructOnScope>(CurrentLensModel.GetDefaultObject()->GetParameterStruct());
			TSharedPtr<IStructureDetailsView> StructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, DistortionParameters);

			uint32 NumChildren;
			StructPropertyHandle->GetNumChildren(NumChildren);

			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				const TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
				const FName PropertyName = ChildHandle->GetProperty()->GetFName();

				if (PropertyName == GET_MEMBER_NAME_CHECKED(FDistortionInfo, Parameters))
				{
					DistortionParametersArrayHandle = ChildHandle;
					IDetailPropertyRow* ParametersRow = StructBuilder.AddExternalStructure(DistortionParameters.ToSharedRef());
					ParametersRow->GetPropertyHandle()->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDistortionInfoCustomization::OnDistortionParametersChanged));
				}
				else
				{
					StructBuilder.AddProperty(ChildHandle).ShowPropertyButtons(false);
				}
			}

			//Start with initial parameters value
			OnDistortionParametersChanged();
		}
	}

private:

	void OnDistortionParametersChanged() const
	{
		//Whenever a parameter value has changed, update the associated array
		if (!CurrentLensModel)
		{
			return;
		}

		TArray<void*> RawData;
		DistortionParametersArrayHandle->AccessRawData(RawData);
		if (RawData.Num() == 1 && RawData[0])
		{
			TArray<float>* RawParametersArray = reinterpret_cast<TArray<float>*>(RawData[0]);
			CurrentLensModel.GetDefaultObject()->ToArray(*DistortionParameters, *RawParametersArray);
		}
	}

private:

	/** Handle to the raw distortion parameters array */
	TSharedPtr<IPropertyHandle> DistortionParametersArrayHandle;

	/** Holds named parameters data structure */
	TSharedPtr<FStructOnScope> DistortionParameters;

	/** LensModel being edited */
	TSubclassOf<ULensModel> CurrentLensModel;
};

void SLensDataAddPointDialog::Construct(const FArguments& InArgs, ULensFile* InLensFile, ELensDataCategory InitialDataCategory)
{
	LensFile = TStrongObjectPtr<ULensFile>(InLensFile);
	CachedFIZ = InArgs._CachedFIZData;
	OnDataPointAdded = InArgs._OnDataPointAdded;

	//Initialize different data structure we'll use to store data
	FocalLengthData = MakeShared<TStructOnScope<FFocalLengthInfo>>();
	DistortionInfoData = MakeShared<TStructOnScope<FDistortionInfoContainer>>();
	ImageCenterData = MakeShared<TStructOnScope<FImageCenterInfo>>();
	NodalOffsetData = MakeShared<TStructOnScope<FNodalPointOffset>>();
	STMapData = MakeShared<TStructOnScope<FSTMapInfo>>();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DataCategoryLabel", "Data Type"))
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.TransformPolicy(ETextTransformPolicy::ToUpper)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &SLensDataAddPointDialog::MakeDataCategoryMenuWidget)
					.ContentPadding(FMargin(4.0, 2.0))
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(MakeAttributeLambda([this]
						{
							switch (SelectedCategory)
							{
							case ELensDataCategory::Focus: return LOCTEXT("FocusCategory", "Encoder - Focus");
							case ELensDataCategory::Iris: return LOCTEXT("IrisCategory", "Encoder - Iris");
							case ELensDataCategory::Zoom: return LOCTEXT("ZoomCategory", "Focal Length");
							case ELensDataCategory::Distortion: return LOCTEXT("DistortionCategory", "Distortion Parameters");
							case ELensDataCategory::ImageCenter: return LOCTEXT("ImageCenterCategory", "Image Center");
							case ELensDataCategory::NodalOffset: return LOCTEXT("NodalOffsetCategory", "Nodal offset");
							case ELensDataCategory::STMap: return LOCTEXT("STMapCategory", "STMap");
							default: return LOCTEXT("InvalidCategory", "Invalid");
							}
						}))
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.AutoHeight()
		[
			SAssignNew(TrackingDataContainer, SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
		]
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.FillHeight(0.7f)
		[
			SAssignNew(LensDataContainer, SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
		]
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				MakeButtonsWidget()
			]
		]
	];
	
	SetDataCategory(InitialDataCategory);
}

void SLensDataAddPointDialog::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	RefreshEvaluationData();
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SLensDataAddPointDialog::OpenDialog(ULensFile* InLensFile, ELensDataCategory InitialDataCategory, TAttribute<FCachedFIZData> InCachedFIZData, const FSimpleDelegate& InOnDataPointAdded)
{
	TSharedPtr<SWindow> PopupWindow = FCameraCalibrationToolkit::OpenPopupWindow(LOCTEXT("LensEditorAddPointDialog", "Add Lens Data Point"));

	TSharedPtr<SLensDataAddPointDialog> AddPointDialog = 
		SNew(SLensDataAddPointDialog, InLensFile, InitialDataCategory)
		.CachedFIZData(MoveTemp(InCachedFIZData))
		.OnDataPointAdded(InOnDataPointAdded);

	PopupWindow->SetContent(AddPointDialog.ToSharedRef());
}

FReply SLensDataAddPointDialog::OnAddDataPointClicked()
{
	const FScopedTransaction MapPointAdded(LOCTEXT("MapPointAdded", "Map Point Added"));
	LensFile->Modify();

	AddDataToLensFile();

	UE::LensAddDataAnalytics::RecordEvent(SelectedCategory);

	OnDataPointAdded.ExecuteIfBound();

	CloseDialog();
	return FReply::Handled();
}

FReply SLensDataAddPointDialog::OnCancelDataPointClicked()
{
	CloseDialog();
	return FReply::Handled();
}

TSharedRef<SWidget> SLensDataAddPointDialog::MakeTrackingDataWidget()
{
	TSharedPtr<SWidget> TrackingWidget = SNullWidget::NullWidget;

	const auto MakeRowWidget = [this](const FString& Label, int32 Index) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 8.f, 0.f, 8.f)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked(MakeAttributeLambda([this, Index]() { return IsTrackingDataOverrided(Index) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }))
				.OnCheckStateChanged(this, &SLensDataAddPointDialog::OnOverrideTrackingData, Index)
				.ToolTipText(LOCTEXT("TrackingDataCheckboxTooltip", "Check to override incoming tracking data for this input"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Label))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<float>)
				.IsEnabled(MakeAttributeLambda([this, Index]() { return IsTrackingDataOverrided(Index); }))
				.Value(MakeAttributeLambda([this, Index]() { return TOptional<float>(GetTrackingData(Index)); }))
				.OnValueChanged(this, &SLensDataAddPointDialog::SetTrackingData, Index)
			];
	};

	//Based on category, either have one tracking input or two
	switch (SelectedCategory)
	{
		case ELensDataCategory::Focus:
		{
			TrackingWidget = MakeRowWidget(TEXT("Input Focus"), 0);
			break;
		}
		case ELensDataCategory::Iris:
		{
			TrackingWidget = MakeRowWidget(TEXT("Input Iris"), 0);
			break;
		}
		case ELensDataCategory::Zoom:
		case ELensDataCategory::Distortion:
		case ELensDataCategory::ImageCenter:
		case ELensDataCategory::NodalOffset:
		case ELensDataCategory::STMap:
		default:
		{
			TrackingWidget = 
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					MakeRowWidget(TEXT("Input Focus"), 0)
				]
				+ SVerticalBox::Slot()
				[
					MakeRowWidget(TEXT("Input Zoom"), 1)
				];
			break;
		}	
	}

	return TrackingWidget.ToSharedRef();
}

void SLensDataAddPointDialog::OnUnitsChanged()
{
	// Refresh the widget with updated default values
	LensDataContainer->SetContent(MakeLensDataWidget());
}

FVector2D SLensDataAddPointDialog::GetDefaultFocalLengthValue() const
{
	const ELensDisplayUnit DisplayUnit = GetDefault<UCameraCalibrationEditorSettings>()->DefaultDisplayUnit;

	FVector2D DefaultValue;
	switch (DisplayUnit)
	{
	case ELensDisplayUnit::Millimeters:
		DefaultValue = FVector2D(LensFile->LensInfo.SensorDimensions[0], LensFile->LensInfo.SensorDimensions[0]);
		break;
	case ELensDisplayUnit::Pixels:
		DefaultValue = FVector2D(LensFile->LensInfo.ImageDimensions[0], LensFile->LensInfo.ImageDimensions[0]);
		break;
	case ELensDisplayUnit::Normalized: // falls through
	default:
		DefaultValue = FVector2D(1.0, (16.0 / 9.0));
		break;
	}
	return DefaultValue;
}

FVector2D SLensDataAddPointDialog::GetDefaultImageCenterValue() const
{
	const ELensDisplayUnit DisplayUnit = GetDefault<UCameraCalibrationEditorSettings>()->DefaultDisplayUnit;

	FVector2D DefaultValue;
	switch (DisplayUnit)
	{
	case ELensDisplayUnit::Millimeters:
		DefaultValue = FVector2D(LensFile->LensInfo.SensorDimensions[0] * 0.5, LensFile->LensInfo.SensorDimensions[1] * 0.5);
		break;
	case ELensDisplayUnit::Pixels:
		DefaultValue = FVector2D(LensFile->LensInfo.ImageDimensions[0] * 0.5, LensFile->LensInfo.ImageDimensions[1] * 0.5);
		break;
	case ELensDisplayUnit::Normalized: // falls through
	default:
		DefaultValue = FVector2D(0.5, 0.5);
		break;
	}
	return DefaultValue;
}

TSharedRef<SWidget> SLensDataAddPointDialog::MakeLensDataWidget()
{
	FStructureDetailsViewArgs StructureViewArgs;
	FDetailsViewArgs DetailArgs;
	DetailArgs.bAllowSearch = false;
	DetailArgs.bShowScrollBar = true;

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	TSharedPtr<SWidget> LensDataWidget = SNullWidget::NullWidget;
	
	FSinglePropertyParams InitParams;
	InitParams.NamePlacement = EPropertyNamePlacement::Hidden;
	const TSharedPtr<ISinglePropertyView> UnitWidget = PropertyEditor.CreateSingleProperty(
		GetMutableDefault<UCameraCalibrationEditorSettings>(), 
		GET_MEMBER_NAME_CHECKED(UCameraCalibrationEditorSettings, DefaultDisplayUnit),
		InitParams);

	FSimpleDelegate OnUnitsChangedDelegate = FSimpleDelegate::CreateSP(this, &SLensDataAddPointDialog::OnUnitsChanged);
	UnitWidget->SetOnPropertyValueChanged(OnUnitsChangedDelegate);

	switch (SelectedCategory)
	{
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		{
			LensDataWidget = MakeEncoderMappingWidget();
			break;
		}
		case ELensDataCategory::Zoom:
		{
			FocalLengthData->InitializeAs<FFocalLengthInfo>();
			FFocalLengthInfo* FocalLengthInstanceData = FocalLengthData->Cast<FFocalLengthInfo>();
			FocalLengthInstanceData->FxFy = GetDefaultFocalLengthValue();

			TSharedPtr<IStructureDetailsView> FocalLengthStructDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, FocalLengthData);

			LensDataWidget =
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5.0f, 5.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FxFyDisplayUnitHelpText", 
						"Choose the units that should be used for the value of FxFy below.\n"
						"If using mm or pixels, check that the sensor and image dimensions are set correctly in the Lens Information."
					))
					.AutoWrapText(true)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					UnitWidget.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					FocalLengthStructDetailsView->GetWidget().ToSharedRef()
				];

			break;
		}
		case ELensDataCategory::Distortion:
		{
			DistortionInfoData->InitializeAs<FDistortionInfoContainer>();
			TSharedPtr<IStructureDetailsView> DistortionDataStructDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());
			DistortionDataStructDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(FDistortionInfo::StaticStruct()->GetFName(),
				FOnGetPropertyTypeCustomizationInstance::CreateLambda([this]() { return MakeShared<FDistortionInfoCustomization>(LensFile->LensInfo.LensModel); })
			);
			DistortionDataStructDetailsView->SetStructureData(DistortionInfoData);
				
			FocalLengthData->InitializeAs<FFocalLengthInfo>();
			TSharedPtr<IStructureDetailsView> FocalLengthStructDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, FocalLengthData);
			FFocalLengthInfo* FocalLengthInstanceData = FocalLengthData->Cast<FFocalLengthInfo>();
			FocalLengthInstanceData->FxFy = GetDefaultFocalLengthValue();

			LensDataWidget =
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5.0f, 5.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FxFyDisplayUnitHelpText",
						"Choose the units that should be used for the value of FxFy below.\n"
						"If using mm or pixels, check that the sensor and image dimensions are set correctly in the Lens Information."
					))
					.AutoWrapText(true)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					UnitWidget.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					DistortionDataStructDetailsView->GetWidget().ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					FocalLengthStructDetailsView->GetWidget().ToSharedRef()
				];

			break;
		}
		case ELensDataCategory::ImageCenter:
		{
			ImageCenterData->InitializeAs<FImageCenterInfo>();
			TSharedPtr<IStructureDetailsView> StructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, ImageCenterData);
			FImageCenterInfo* ImageCenterInstanceData = ImageCenterData->Cast<FImageCenterInfo>();
			ImageCenterInstanceData->PrincipalPoint = GetDefaultImageCenterValue();

			LensDataWidget =
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5.0f, 5.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ImageCenterDisplayUnitHelpText",
						"Choose the units that should be used for the value of ImageCenter below.\n"
						"If using mm or pixels, check that the sensor and image dimensions are set correctly in the Lens Information."
					))
				.AutoWrapText(true)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					UnitWidget.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					StructureDetailsView->GetWidget().ToSharedRef()
				];

			break;
		}
		case ELensDataCategory::NodalOffset:
		{
			NodalOffsetData->InitializeAs<FNodalPointOffset>();
			TSharedPtr<IStructureDetailsView> StructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, NodalOffsetData);
			LensDataWidget = StructureDetailsView->GetWidget();
			break;
		}
		case ELensDataCategory::STMap:
		{
			STMapData->InitializeAs<FSTMapInfo>();
			TSharedPtr<IStructureDetailsView> StructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, STMapData);

			FocalLengthData->InitializeAs<FFocalLengthInfo>();
			TSharedPtr<IStructureDetailsView> FocalLengthStructDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, FocalLengthData);
			FFocalLengthInfo* FocalLengthInstanceData = FocalLengthData->Cast<FFocalLengthInfo>();
			FocalLengthInstanceData->FxFy = GetDefaultFocalLengthValue();

			LensDataWidget =
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5.0f, 5.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FxFyDisplayUnitHelpText",
						"Choose the units that should be used for the value of FxFy below.\n"
						"If using mm or pixels, check that the sensor and image dimensions are set correctly in the Lens Information."
					))
					.AutoWrapText(true)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					UnitWidget.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					StructureDetailsView->GetWidget().ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 5.0f)
				[
					FocalLengthStructDetailsView->GetWidget().ToSharedRef()
				];

			break;
		}
		default:
		{
			break;
		}
	}

	return LensDataWidget.ToSharedRef();
}

TSharedRef<SWidget> SLensDataAddPointDialog::MakeButtonsWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.OnClicked(this, &SLensDataAddPointDialog::OnAddDataPointClicked)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("AddDataPoint", "Add"))
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.OnClicked(this, &SLensDataAddPointDialog::OnCancelDataPointClicked)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("CancelAddingDataPoint", "Cancel"))
		];
}

TSharedRef<SWidget> SLensDataAddPointDialog::MakeDataCategoryMenuWidget()
{
	constexpr bool bShouldCloseWindowAfterClosing = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterClosing, nullptr);

	MenuBuilder.BeginSection("LensDataCategories", LOCTEXT("LensDataCategories", "Lens data categories"));
	{
		const auto AddMenuEntryFn = [&MenuBuilder, this](const FString& Label, ELensDataCategory Category)
		{
			MenuBuilder.AddMenuEntry
			(
				FText::FromString(Label)
				, FText::FromString(Label)
				, FSlateIcon()
				, FUIAction
				(
					FExecuteAction::CreateLambda([this, Category] { SetDataCategory(Category); })
					, FCanExecuteAction()
					, FIsActionChecked::CreateLambda([this, Category] { return SelectedCategory == Category; })
				)
				, NAME_None
				, EUserInterfaceActionType::RadioButton
			);
		};
		
		//Add different categories that we support
		AddMenuEntryFn(TEXT("Focus"), ELensDataCategory::Focus);
		AddMenuEntryFn(TEXT("Iris"), ELensDataCategory::Iris);
		AddMenuEntryFn(TEXT("Focal Length"), ELensDataCategory::Zoom);
		AddMenuEntryFn(TEXT("Distortion Parameters"), ELensDataCategory::Distortion);
		AddMenuEntryFn(TEXT("Image Center"), ELensDataCategory::ImageCenter);
		AddMenuEntryFn(TEXT("Nodal Offset"), ELensDataCategory::NodalOffset);
		AddMenuEntryFn(TEXT("STMap"), ELensDataCategory::STMap);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SLensDataAddPointDialog::MakeEncoderMappingWidget()
{
	return 
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSplitter)
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			.PhysicalSplitterHandleSize(1.0f)
			.HitDetectionSplitterHandleSize(5.0f)
			+ SSplitter::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0, 1, 0, 1))
				.FillWidth(1)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EncoderMappingValueLabel", "Encoder mapping :"))
					.ToolTipText_Lambda([this]()
					{
						if(SelectedCategory == ELensDataCategory::Focus)
						{
							return LOCTEXT("FocusEncoderTooltip", "Focus in cm");
						}
						else if(SelectedCategory == ELensDataCategory::Iris)
						{
							return LOCTEXT("IrisEncoderTooltip", "Aperture in FStop");
						}
						else
						{
							return LOCTEXT("InvalidEncoderType", "Invalid Encoder Type");
						}
					})
				]
			]
			+ SSplitter::Slot()
			[
				SNew(SNumericEntryBox<float>)
				.AllowSpin(true)
				.Value_Lambda([this]() { return EncoderMappingValue; })
				.OnValueChanged(SNumericEntryBox<float>::FOnValueChanged::CreateLambda([this](float NewValue) { EncoderMappingValue = NewValue; }))
				.OnValueCommitted(SNumericEntryBox<float>::FOnValueCommitted::CreateLambda([this](float NewValue, ETextCommit::Type CommitType) { EncoderMappingValue = NewValue; }))
			]
		];
}

void SLensDataAddPointDialog::SetDataCategory(ELensDataCategory NewCategory)
{
	if (NewCategory != SelectedCategory)
	{
		SelectedCategory = NewCategory;
		EncoderMappingValue = 0.0f;
	}

	//Updates displayed widgets based on category
	TrackingDataContainer->SetContent(MakeTrackingDataWidget());
	LensDataContainer->SetContent(MakeLensDataWidget());
}

void SLensDataAddPointDialog::CloseDialog()
{
	FCameraCalibrationToolkit::DestroyPopupWindow();
}

bool SLensDataAddPointDialog::IsTrackingDataOverrided(int32 Index) const
{
	if (ensure(Index >= 0 && Index <= 1))
	{
		return TrackingInputData[Index].bIsOverrided;
	}

	return false;
}
void SLensDataAddPointDialog::OnOverrideTrackingData(ECheckBoxState NewState, int32 Index)
{
	if (ensure(Index >= 0 && Index <= 1))
	{
		TrackingInputData[Index].bIsOverrided = NewState == ECheckBoxState::Checked;
	}
}

float SLensDataAddPointDialog::GetTrackingData(int32 Index) const
{
	if (ensure(Index >= 0 && Index <= 1))
	{
		return TrackingInputData[Index].Value;
	}

	return 0.0f;
}

void SLensDataAddPointDialog::SetTrackingData(float Value, int32 Index)
{
	if (ensure(Index >= 0 && Index <= 1))
	{
		TrackingInputData[Index].Value = Value;
	}
}

void SLensDataAddPointDialog::RefreshEvaluationData()
{
	//Get FIZ coming from the lens component. If normalized values are available, take them. Otherwise, look for precalibrated values or default to 0.0f
	const FCachedFIZData& CachedData = CachedFIZ.Get();
	const float Focus = CachedData.RawFocus.IsSet() ? CachedData.RawFocus.GetValue() : CachedData.EvaluatedFocus.IsSet() ? CachedData.EvaluatedFocus.GetValue() : 0.0f;
	const float Iris = CachedData.RawIris.IsSet() ? CachedData.RawIris.GetValue() : CachedData.EvaluatedIris.IsSet() ? CachedData.EvaluatedIris.GetValue() : 0.0f;
	const float Zoom = CachedData.RawZoom.IsSet() ? CachedData.RawZoom.GetValue() : CachedData.EvaluatedZoom.IsSet() ? CachedData.EvaluatedZoom.GetValue() : 0.0f;
	
	switch (SelectedCategory)
	{
		case ELensDataCategory::Focus:
		{
			if (IsTrackingDataOverrided(0) == false)
			{
				TrackingInputData[0].Value = Focus;
			}
			break;
		}
		case ELensDataCategory::Iris:
		{
			if (IsTrackingDataOverrided(0) == false)
			{
				TrackingInputData[0].Value = Iris;
			}
			break;
		}
		case ELensDataCategory::Distortion:
		case ELensDataCategory::ImageCenter:
		case ELensDataCategory::NodalOffset:
		case ELensDataCategory::STMap:
		default:
		{
			if (IsTrackingDataOverrided(0) == false)
			{
				TrackingInputData[0].Value = Focus;
			}
			if (IsTrackingDataOverrided(1) == false)
			{
				TrackingInputData[1].Value = Zoom;
			}
			break;
		}
	}
}

void SLensDataAddPointDialog::NormalizeValue(FVector2D& InOutValue) const
{
	const ELensDisplayUnit DisplayUnit = GetDefault<UCameraCalibrationEditorSettings>()->DefaultDisplayUnit;

	switch (DisplayUnit)
	{
	case ELensDisplayUnit::Millimeters:
	{
		FVector2D SensorDimensions = LensFile->LensInfo.SensorDimensions;
		if (SensorDimensions[0] > 0.0 && SensorDimensions[1] > 0.0)
		{
			InOutValue[0] = InOutValue[0] / SensorDimensions[0];
			InOutValue[1] = InOutValue[1] / SensorDimensions[1];
		}
		break;
	}
	case ELensDisplayUnit::Pixels:
	{
		FVector2D ImageDimensions = LensFile->LensInfo.ImageDimensions;
		if (ImageDimensions[0] > 0.0 && ImageDimensions[1] > 0.0)
		{
			InOutValue[0] = InOutValue[0] / ImageDimensions[0];
			InOutValue[1] = InOutValue[1] / ImageDimensions[1];
		}
		break;
	}
	case ELensDisplayUnit::Normalized: // falls through
	default:
		// do nothing
		break;
	}
}

void SLensDataAddPointDialog::AddDataToLensFile() const
{
	switch (SelectedCategory)
	{
		case ELensDataCategory::Focus:
		{
			LensFile->EncodersTable.Focus.AddKey(TrackingInputData[0].Value, EncoderMappingValue);
			break;
		}
		case ELensDataCategory::Iris:
		{
			LensFile->EncodersTable.Iris.AddKey(TrackingInputData[0].Value, EncoderMappingValue);
			break;
		}
		case ELensDataCategory::Zoom:
		{
			FFocalLengthInfo FocalLengthValue = *FocalLengthData->Get();
			NormalizeValue(FocalLengthValue.FxFy);
			LensFile->AddFocalLengthPoint(TrackingInputData[0].Value, TrackingInputData[1].Value, FocalLengthValue);
			break;
		}	
		case ELensDataCategory::Distortion:
		{
			FFocalLengthInfo FocalLengthValue = *FocalLengthData->Get();
			NormalizeValue(FocalLengthValue.FxFy);
			LensFile->AddDistortionPoint(TrackingInputData[0].Value, TrackingInputData[1].Value, DistortionInfoData->Get()->DistortionInfo, FocalLengthValue);
			break;
		}	
		case ELensDataCategory::ImageCenter:
		{
			FImageCenterInfo ImageCenterValue = *ImageCenterData->Get();
			NormalizeValue(ImageCenterValue.PrincipalPoint);
			LensFile->AddImageCenterPoint(TrackingInputData[0].Value, TrackingInputData[1].Value, ImageCenterValue);
			break;
		}
		case ELensDataCategory::NodalOffset:
		{
			LensFile->AddNodalOffsetPoint(TrackingInputData[0].Value, TrackingInputData[1].Value, *NodalOffsetData->Get());
			break;
		}
		case ELensDataCategory::STMap:
		{
			LensFile->AddSTMapPoint(TrackingInputData[0].Value, TrackingInputData[1].Value, *STMapData->Get());

			FFocalLengthInfo FocalLengthValue = *FocalLengthData->Get();
			NormalizeValue(FocalLengthValue.FxFy);
			LensFile->AddFocalLengthPoint(TrackingInputData[0].Value, TrackingInputData[1].Value, FocalLengthValue);
			break;
		}
		default:
			break;
	}
}

#undef LOCTEXT_NAMESPACE 
