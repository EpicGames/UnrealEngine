// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

#include "CameraCalibrationEditorCommon.h"
#include "LensFile.h"
#include "SLensEvaluation.h"
#include "UObject/StrongObjectPtr.h"

class SBorder;

class SLensDataAddPointDialog : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:

	SLATE_BEGIN_ARGS(SLensDataAddPointDialog)
	{}

		/** FIZ data */
		SLATE_ATTRIBUTE(FCachedFIZData, CachedFIZData)

		/** If bound, used to notify a point was added to LensFile. */
		SLATE_EVENT(FSimpleDelegate, OnDataPointAdded)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULensFile* InLensFile, ELensDataCategory InitialDataCategory);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Finds currently opened dialog window or spawns a new one */
	static void OpenDialog(ULensFile* InLensFile, ELensDataCategory InitialDataCategory, TAttribute<FCachedFIZData> InCachedFIZData, const FSimpleDelegate& InOnDataPointAdded);

private:

	/** Called when AddPoint button is clicked */
	FReply OnAddDataPointClicked();
	
	/** Called when Cancel button is clicked */
	FReply OnCancelDataPointClicked();

	/** Makes the different widgets used in this dialog */
	TSharedRef<SWidget> MakeTrackingDataWidget();
	TSharedRef<SWidget> MakeLensDataWidget();
	TSharedRef<SWidget> MakeButtonsWidget();
	TSharedRef<SWidget> MakeDataCategoryMenuWidget();
	TSharedRef<SWidget> MakeEncoderMappingWidget();

	/** Updates the desired data category */
	void SetDataCategory(ELensDataCategory NewCategory);

	/** Closes this dialog */
	void CloseDialog();

	/** Whether tracking data is overrided or not */
	bool IsTrackingDataOverrided(int32 Index) const;
	
	/** Triggered when override TrackingData state changes */
	void OnOverrideTrackingData(ECheckBoxState NewState, int32 Index);

	/** Gets tracking data for Index */
	float GetTrackingData(int32 Index) const;

	/** Updates tracking data for desired Index */
	void SetTrackingData(float Value, int32 Index);

	/** Updates the current inputs values */
	void RefreshEvaluationData();
	
	/** Called when ready to add data to LensFile */
	void AddDataToLensFile() const;

	/** Called when the display units are changed */
	void OnUnitsChanged();

	/** Returns the default value to display for focal length based on the display units of the LensFile */
	FVector2D GetDefaultFocalLengthValue() const;

	/** Returns the default value to display for image center based on the display units of the LensFile */
	FVector2D GetDefaultImageCenterValue() const;

	/** Normalize the input value based on the display units of the LensFile */
	void NormalizeValue(FVector2D& InputValue) const;

private:

	/**
	 * Used to build tracking input widget
	 * Depending on the category, there is either one
	 * input (Encoder) or two (Focus / Zoom)
	 */
	struct FTrackingInputData
	{
		FString Label;
		bool bIsOverrided = false;
		float Value = 0.0f;
	};
	FTrackingInputData TrackingInputData[2];

	/** LensFile being edited */
	TStrongObjectPtr<ULensFile> LensFile;

	/** Container for tracking inputs */
	TSharedPtr<SBorder> TrackingDataContainer;

	/** Container for lens data */
	TSharedPtr<SBorder> LensDataContainer;
	
	/** Category of data being added */
	ELensDataCategory SelectedCategory = ELensDataCategory::NodalOffset;

	/** When adding an encoder mapping value, float holding the current one */
	float EncoderMappingValue = 0.0f;

	/** ImageCenterData, valid for image center category */
	TSharedPtr<TStructOnScope<FImageCenterInfo>> ImageCenterData;

	/** FocalLengthInfo, valid for distortion or focal length category */
	TSharedPtr<TStructOnScope<FFocalLengthInfo>> FocalLengthData;

	/** DistortionInfo, valid for distortion category */
	TSharedPtr<TStructOnScope<FDistortionInfoContainer>> DistortionInfoData;

	/** NodalOffset, valid for nodal offset category */
	TSharedPtr<TStructOnScope<FNodalPointOffset>> NodalOffsetData;
	
	/** STMapInfo, valid for STMap category */
	TSharedPtr<TStructOnScope<FSTMapInfo>> STMapData;
	
	/** Evaluated FIZ for the current frame */
	TAttribute<FCachedFIZData> CachedFIZ;

	/** Delegate to call when point is added to LensFile */
	FSimpleDelegate OnDataPointAdded;
};