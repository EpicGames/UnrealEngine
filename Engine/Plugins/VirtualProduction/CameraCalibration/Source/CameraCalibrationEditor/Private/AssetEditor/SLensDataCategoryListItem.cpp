// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensDataCategoryListItem.h"
#include "LensFile.h"
#include "SLensDataListItem.h"
#include "Widgets/Input/SButton.h"
#include "IStructureDetailsView.h"


#define LOCTEXT_NAMESPACE "LensDataCategoryListItem"



FLensDataCategoryItem::FLensDataCategoryItem(ULensFile* InLensFile, TWeakPtr<FLensDataCategoryItem> InParent, ELensDataCategory InCategory, int32 InParameterIndex, FName InLabel)
	: Category(InCategory)
	, ParameterIndex(InParameterIndex)
	, Label(InLabel)
	, Parent(InParent)
	, LensFile(InLensFile)
{
}

TSharedRef<ITableRow> FLensDataCategoryItem::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SLensDataCategoryItem, InOwnerTable, AsShared());
}

void SLensDataCategoryItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FLensDataCategoryItem> InItemData)
{
	WeakItem = InItemData;

	STableRow<TSharedPtr<FLensDataCategoryItem>>::Construct(
		STableRow<TSharedPtr<FLensDataCategoryItem>>::FArguments()
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &SLensDataCategoryItem::GetLabelText)
		], OwnerTable);
}

FText SLensDataCategoryItem::GetLabelText() const
{
	return (FText::FromName(WeakItem.Pin()->Label));
}




#undef LOCTEXT_NAMESPACE /* LensDataViewer */



