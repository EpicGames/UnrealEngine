// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CameraCalibrationEditorCommon.h"
#include "LensFile.h"
#include "SLensFilePanel.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STreeView.h"


class FLensDataListItem;


/**
 * Data category item
 */
class FLensDataCategoryItem : public TSharedFromThis<FLensDataCategoryItem>
{
public:
	FLensDataCategoryItem(ULensFile* InLensFile, TWeakPtr<FLensDataCategoryItem> Parent, ELensDataCategory InCategory, int32 InParameterIndex, FName InLabel);
	virtual ~FLensDataCategoryItem() = default;

	/** Makes the widget for its associated row */
	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable);
	
public:

	/** Category this item is associated with */
	ELensDataCategory Category;

	/** Used to identify which parameter this represents */
	int32 ParameterIndex = INDEX_NONE;
	
	/** Label of this category */
	FName Label;

	/** WeakPtr to parent of this item */
	TWeakPtr<FLensDataCategoryItem> Parent;

	/** Children of this category */
	TArray<TSharedPtr<FLensDataCategoryItem>> Children;

	/** LensFile being edited */
	TWeakObjectPtr<ULensFile> LensFile;
};

/**
 * Data category row widget
 */
class SLensDataCategoryItem : public STableRow<TSharedPtr<FLensDataCategoryItem>>
{
	SLATE_BEGIN_ARGS(SLensDataCategoryItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FLensDataCategoryItem> InItemData);

private:

	/** Returns the label of this row */
	FText GetLabelText() const;

private:

	/** WeakPtr to source data item */
	TWeakPtr<FLensDataCategoryItem> WeakItem;
};
