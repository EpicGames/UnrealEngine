// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CameraCalibrationToolkit.h"
#include "IStructureDetailsView.h"
#include "LensFile.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/StructOnScope.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "LensDataEditPointDialog"


/**
 * Editing Lens Point Dialog, it opens in separate dialog popup window
 */
template<typename TStructType>
class SLensDataEditPointDialog : public SCompoundWidget
{
private:
	struct FTrackingInputData
	{
		float InitialValue = 0.0;
		float CurrentValue = 0.0;
		bool bIsOverridden = false;
	};
	
public:
	/** Delegate for save struct on scope with specific struct type */
	DECLARE_DELEGATE_ThreeParams(FOnSave, TSharedPtr<TStructOnScope<TStructType>> /*InStructToEdit */, TOptional<float> /*NewFocus*/, TOptional<float> /*NewZoom*/)
	
	SLATE_BEGIN_ARGS(SLensDataEditPointDialog)
	{}
		SLATE_EVENT(FOnSave, OnSave)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULensFile* InLensFile, ELensDataCategory InCategory, TSharedPtr<TStructOnScope<TStructType>> InStructToEdit, float InFocus, float InZoom)
	{
		StructToEdit = InStructToEdit;
        LensFile = TStrongObjectPtr<ULensFile>(InLensFile);
		Category = InCategory;
		Focus.InitialValue = Focus.CurrentValue = InFocus;
		Zoom.InitialValue = Zoom.CurrentValue = InZoom;
		OnSaveDelegate = InArgs._OnSave;
		
        const TSharedPtr<SWidget> LensDataWidget = [this]()
        {
        	TSharedPtr<SWidget> WidgetToReturn;
    
        	// In case the struct does not have valid pointer or the data is invalid
        	if (!StructToEdit.IsValid() || !StructToEdit->IsValid())
        	{
        		WidgetToReturn = SNew(STextBlock).Text(LOCTEXT("ErrorEditStruct", "Point can't be edited"));
        	}
        	else
        	{
        		const FStructureDetailsViewArgs StructureViewArgs;
        		FDetailsViewArgs DetailArgs;
        		DetailArgs.bAllowSearch = false;
        		DetailArgs.bShowScrollBar = true;
    
        		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
        		TSharedPtr<IStructureDetailsView> StructureDetailView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, StructToEdit);
    
        		WidgetToReturn = StructureDetailView->GetWidget();
        	}
        	
        	return WidgetToReturn;
        }();
    
        const TSharedPtr<SWidget> ButtonsWidget = [this]()
        {
        	return SNew(SHorizontalBox)
        		+ SHorizontalBox::Slot()
        		[
        			SNew(SButton)
        			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
        			.OnClicked(this, &SLensDataEditPointDialog::OnSaveDataPointClicked)
        			.HAlign(HAlign_Center)
        			.Text(LOCTEXT("SaveDataPoint", "Save"))
        		]
        		+ SHorizontalBox::Slot()
        		[
        			SNew(SButton)
        			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
        			.OnClicked(this, &SLensDataEditPointDialog::OnCancelDataPointClicked)
        			.HAlign(HAlign_Center)
        			.Text(LOCTEXT("CancelEditDataPoint", "Cancel"))
        		];
        }();
		
        ChildSlot
        [
        	SNew(SVerticalBox)
        	
        	+SVerticalBox::Slot()
			.Padding(5.0f, 5.0f)
			.AutoHeight()
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
				[
					MakeTrackingDataWidget()
				]
			]
			
        	+SVerticalBox::Slot()
        	.Padding(5.0f, 5.0f)
        	.FillHeight(1.f)
        	[
        		SNew(SBorder)
        		.HAlign(HAlign_Fill)
        		.VAlign(VAlign_Fill)
        		.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
        		[
        			LensDataWidget.ToSharedRef()
        		]
        	]
        	
        	+SVerticalBox::Slot()
        	.Padding(5.0f, 5.0f)
        	.AutoHeight()
        	[
        		SNew(SBorder)
        		.HAlign(HAlign_Fill)
        		.VAlign(VAlign_Fill)
        		.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
        		[
        			ButtonsWidget.ToSharedRef()
        		]
        	]
        ];
	}

private:
	TSharedRef<SWidget> MakeTrackingDataWidget()
	{
		TSharedPtr<SWidget> TrackingWidget = SNullWidget::NullWidget;

		const auto MakeRowWidget = [this](const FString& Label, FTrackingInputData& Data) -> TSharedRef<SWidget>
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 8.f, 0.f, 8.f)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([&Data]() { return Data.bIsOverridden ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([&Data](ECheckBoxState NewState) { Data.bIsOverridden = NewState == ECheckBoxState::Checked; }))
					.ToolTipText(LOCTEXT("TrackingDataCheckboxTooltip", "Check to override incoming tracking data for this point"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Label))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<float>)
					.IsEnabled_Lambda([&Data](){ return Data.bIsOverridden; })
					.Value_Lambda([&Data]() { return TOptional<float>(Data.CurrentValue); })
					.OnValueChanged(SNumericEntryBox<float>::FOnValueChanged::CreateLambda([&Data](float NewValue) { Data.CurrentValue = NewValue; }))
				];
		};

		//Based on category, either have one tracking input or two
		switch (Category)
		{
			case ELensDataCategory::Focus:
			{
				TrackingWidget = MakeRowWidget(TEXT("Input Focus"), Focus);
				break;
			}
			case ELensDataCategory::Iris:
			{
				TrackingWidget = MakeRowWidget(TEXT("Input Iris"), Focus);
				break;
			}
			case ELensDataCategory::Zoom:
			case ELensDataCategory::Distortion:
			case ELensDataCategory::ImageCenter:
			case ELensDataCategory::NodalOffset:
			case ELensDataCategory::STMap:
			default:
			{
				TrackingWidget = 
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					[
						MakeRowWidget(TEXT("Input Focus"), Focus)
					]
					+ SVerticalBox::Slot()
					[
						MakeRowWidget(TEXT("Input Zoom"), Zoom)
					];
				break;
			}	
		}

		return TrackingWidget.ToSharedRef();
	}
	
	/** Save Button handler */
	FReply OnSaveDataPointClicked() const
	{
		if (!StructToEdit.IsValid())
		{
			return FReply::Unhandled();
		}

		const FScopedTransaction MapPointEdited(LOCTEXT("MapPointEdited", "Map Point Edited"));
		LensFile->Modify();
		
		TOptional<float> NewFocus = TOptional<float>();
		if (Focus.bIsOverridden && Focus.CurrentValue != Focus.InitialValue)
		{
			NewFocus = Focus.CurrentValue;
		}

		TOptional<float> NewZoom = TOptional<float>();
		if (Zoom.bIsOverridden && Zoom.CurrentValue != Zoom.InitialValue)
		{
			NewZoom = Zoom.CurrentValue;
		}
		
		OnSaveDelegate.ExecuteIfBound(StructToEdit, NewFocus, NewZoom);

		FCameraCalibrationToolkit::DestroyPopupWindow();
	
		return FReply::Handled();
	}

	/** Cancel Button handler */
	FReply OnCancelDataPointClicked() const
	{
		FCameraCalibrationToolkit::DestroyPopupWindow();

		return FReply::Handled();
	}

private:
	/** LensFile being edited */
	TStrongObjectPtr<ULensFile> LensFile;

	/** The category of the data struct being edited */
	ELensDataCategory Category = ELensDataCategory::Focus;
	
	/** Editing Struct for visualize with StructureDetailsView */ 
	TSharedPtr<TStructOnScope<TStructType>> StructToEdit;
	
	/** Focus of the data point being edited */
	FTrackingInputData Focus;

	/** Zoom of the data point being edited */
	FTrackingInputData Zoom;
	
	/** On save struct delegate instance */
	FOnSave OnSaveDelegate;
};

namespace LensDataEditPointDialog
{
	/** Get specific struct for edit, based on Data Table Type */
	template<typename TStructType, typename TableType>
	TSharedPtr<TStructOnScope<TStructType>> GetStructToEdit(const float InFocus, const float InZoom, const TableType& InTable)
	{
		TSharedPtr<TStructOnScope<TStructType>> ReturnStruct = nullptr;
		
		TStructType PointCopy;
		if (InTable.GetPoint(InFocus, InZoom, PointCopy))
		{
			const FStructOnScope StructOnScopeCopy(TStructType::StaticStruct(),reinterpret_cast<uint8*>(&PointCopy));
			ReturnStruct = MakeShared<TStructOnScope<TStructType>>();
			ReturnStruct->InitializeFrom(StructOnScopeCopy);
		}

		return ReturnStruct;
	}
	
	/** Finds currently opened dialog window or spawns a new one for editing */
	template<typename TStructType, typename TableType>
	static void OpenDialog(ULensFile* InLensFile, ELensDataCategory InCategory, const float InFocus, const float InZoom, TableType& InTable, const FSimpleDelegate& OnPointSaved)
	{
		using FOnSaveType = typename SLensDataEditPointDialog<TStructType>::FOnSave;
		using FStructType = TSharedPtr<TStructOnScope<TStructType>>;

		// On save struct delegate
		FOnSaveType OnSaveDelegate = FOnSaveType::CreateLambda([InFocus, InZoom, &InTable, OnPointSaved](FStructType InStructOnScope, TOptional<float> NewFocus, TOptional<float> NewZoom)
		{
			InTable.SetPoint(InFocus, InZoom, *InStructOnScope->Get());

			// Set new zoom first so that any zoom replace conflicts are dealt with before attempting to change/merge the focus
			if (NewZoom.IsSet())
			{
				if (InTable.HasZoomPoint(InFocus, NewZoom.GetValue()))
				{
					if (FCameraCalibrationWidgetHelpers::ShowReplaceZoomWarning())
					{
						InTable.ChangeZoomPoint(InFocus, InZoom, NewZoom.GetValue());
					}
				}
				else
				{
					InTable.ChangeZoomPoint(InFocus, InZoom, NewZoom.GetValue());
				}
			}
			
			if (NewFocus.IsSet())
			{
				if (InTable.HasFocusPoint(NewFocus.GetValue()))
				{
					bool bReplaceExisting = false;
					if (FCameraCalibrationWidgetHelpers::ShowMergeFocusWarning(bReplaceExisting))
					{
						InTable.MergeFocusPoint(InFocus, NewFocus.GetValue(), bReplaceExisting);
					}
				}
				else
				{
					InTable.ChangeFocusPoint(InFocus, NewFocus.GetValue());
				}
			}

			OnPointSaved.ExecuteIfBound();
		});

		FStructType InStructToEdit = GetStructToEdit<TStructType>(InFocus, InZoom, InTable);
		TSharedPtr<SWindow> PopupWindow = FCameraCalibrationToolkit::OpenPopupWindow(LOCTEXT("LensEditorEditPointDialog", "Edit Lens Data Point"));
		PopupWindow->SetContent(SNew(SLensDataEditPointDialog<TStructType>, InLensFile, InCategory, InStructToEdit, InFocus, InZoom).OnSave(OnSaveDelegate));
	}
}

#undef LOCTEXT_NAMESPACE
