// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensDataListItem.h"

#include "LensFile.h"
#include "SCameraCalibrationLinkedPointsDialog.h"
#include "ScopedTransaction.h"
#include "SLensDataEditPointDialog.h"
#include "UI/CameraCalibrationEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "LensDataListItem"

FLensDataListItem::FLensDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, FOnDataChanged InOnDataChangedCallback)
	: Category(InCategory)
	, SubCategoryIndex(InSubCategoryIndex)
	, WeakLensFile(InLensFile)
	, OnDataChangedCallback(InOnDataChangedCallback)
{
}

FEncoderDataListItem::FEncoderDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, float InInputValue, int32 InIndex, FOnDataChanged InOnDataChangedCallback)
	: FLensDataListItem(InLensFile, InCategory, INDEX_NONE, InOnDataChangedCallback)
	, InputValue(InInputValue)
	, EntryIndex()
{
}

void FEncoderDataListItem::OnRemoveRequested() const
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveEncoderPointTransaction", "Remove encoder point"));
		LensFilePtr->Modify();

		//Pass encoder mapping raw input value as focus to remove it
		LensFilePtr->RemoveFocusPoint(Category, InputValue);
		OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataRemoved, InputValue, TOptional<float>());
	}
}

TSharedRef<ITableRow> FEncoderDataListItem::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SLensDataItem, InOwnerTable, AsShared())
		.EntryLabel(LOCTEXT("EncoderLabel", "Input:"))
		.EntryValue(InputValue)
		.AllowRemoval(true);
}

FFocusDataListItem::FFocusDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, float InFocus, FOnDataChanged InOnDataChangedCallback)
	: FLensDataListItem(InLensFile, InCategory, InSubCategoryIndex, InOnDataChangedCallback)
	, Focus(InFocus)
{
}

TSharedRef<ITableRow> FFocusDataListItem::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SLensDataItem, InOwnerTable, AsShared())
		.EntryLabel(LOCTEXT("FocusLabel", "Focus: "))
		.EntryValue(Focus)
		.AllowEditEntryValue(true)
		.OnEntryValueChanged(this, &FFocusDataListItem::OnFocusValueChanged)
		.AllowRemoval(SubCategoryIndex == INDEX_NONE);
}

void FFocusDataListItem::OnRemoveRequested() const
{	
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		const FBaseLensTable* const LinkDataTable = LensFilePtr->GetDataTable(Category);
		if (!ensure(LinkDataTable))
		{
			return;	
		}
		
		if (LinkDataTable->HasLinkedFocusValues(Focus))
		{
			if (RemoveLinkedFocusValues())
			{
				OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataRemoved, Focus, TOptional<float>());
			}
		}
		else
		{
			FScopedTransaction Transaction(LOCTEXT("RemoveFocusPointsTransaction", "Remove Focus Points"));
			LensFilePtr->Modify();

			LensFilePtr->RemoveFocusPoint(Category, Focus);
			OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataRemoved, Focus, TOptional<float>());
		}
	}
}

bool FFocusDataListItem::OnFocusValueChanged(float NewFocusValue)
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		const FBaseLensTable* const DataTable = LensFilePtr->GetDataTable(Category);
		if (!ensure(DataTable))
		{
			return false;
		}

		if (DataTable->HasLinkedFocusValues(Focus))
		{
			// If there are any linked focus values, allow the linked focus dialog box to handle any needed changes to the focuses in the lens file
			if (ChangeLinkedFocusValues(NewFocusValue))
			{
				Focus = NewFocusValue;
				OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataChanged, Focus, TOptional<float>());
				return true;
			}

			return false;
		}
		
		if (LensFilePtr->HasFocusPoint(Category, NewFocusValue))
		{
			// If the data already has a point for the new focus value, we want to merge this focus point's data with
			// the existing focus points, with user permission
			bool bReplaceExistingZoomPoints = false;
			const bool bMergeFocusPoints = FCameraCalibrationWidgetHelpers::ShowMergeFocusWarning(bReplaceExistingZoomPoints);
			if (bMergeFocusPoints)
			{
				FScopedTransaction Transaction(LOCTEXT("MergeFocusPointTransaction", "Merge Focus Point"));
				LensFilePtr->Modify();

				LensFilePtr->MergeFocusPoint(Category, Focus, NewFocusValue, bReplaceExistingZoomPoints);

				Focus = NewFocusValue;
				OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataChanged, Focus, TOptional<float>());

				return true;
			}
		}
		else
		{
			// Otherwise, we can just change the data point's value directly
			FScopedTransaction Transaction(LOCTEXT("ChangeFocusPointTransaction", "Change Focus Point"));
			LensFilePtr->Modify();

			LensFilePtr->ChangeFocusPoint(Category, Focus, NewFocusValue);
			Focus = NewFocusValue;

			return true;
		}
	}

	return false;
}

bool FFocusDataListItem::ChangeLinkedFocusValues(float NewFocusValue) const
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		bool bFocusValuesChanged = false;
		bool bReplaceExisting = false;

		TSharedRef<SWidget> DialogContent = SNew(SCheckBox)
			.ToolTipText(LOCTEXT("ReplaceExistingZoomPointsInFocusToolTip", "When checked, any existing zoom points in the destination focus will be replaced with those in the source focus"))
			.IsChecked_Lambda([&bReplaceExisting]() { return bReplaceExisting ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([&bReplaceExisting](ECheckBoxState CheckBoxState) { bReplaceExisting = CheckBoxState == ECheckBoxState::Checked; })
			.Padding(FMargin(4.0, 0.0))
			[
				SNew(STextBlock).Text(LOCTEXT("ReplaceExistingZoomPointsLabel", "Replace existing zoom points?"))
			];
		
		auto OnApplyChange = [LensFilePtr, NewFocusValue, &bReplaceExisting, &bFocusValuesChanged](const TArray<SCameraCalibrationLinkedPointsDialog::FLinkedItem>& LinkedItems)
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeLinkedFocusPointsTransaction", "Change Linked Focus Points"));
			LensFilePtr->Modify();

			for (const SCameraCalibrationLinkedPointsDialog::FLinkedItem& LinkedItem : LinkedItems)
			{
				if (LensFilePtr->HasFocusPoint(LinkedItem.Category, NewFocusValue))
				{
					LensFilePtr->MergeFocusPoint(LinkedItem.Category, LinkedItem.Focus, NewFocusValue, bReplaceExisting);
				}
				else
				{
					LensFilePtr->ChangeFocusPoint(LinkedItem.Category, LinkedItem.Focus, NewFocusValue);
				}
			}

			bFocusValuesChanged = true;
		};

		const SCameraCalibrationLinkedPointsDialog::FLinkedItem Item(Category, Focus);
		TSharedRef<SCameraCalibrationLinkedPointsDialog> DialogBox = SNew(SCameraCalibrationLinkedPointsDialog, LensFilePtr, Item)
			.LinkedItemMode(SCameraCalibrationLinkedPointsDialog::ELinkedItemMode::Focus)
			.DialogText(LOCTEXT("ChangeLinkedFocusDialogText", "The calibration data you wish to change may be inherently linked to additional data.\nChoose any and all linked data you wish to change."))
			.AcceptButtonText(LOCTEXT("ChangeLinkedFocusAcceptButton", "Change Focus"))
			.OnApplyLinkedAction_Lambda(OnApplyChange)
			[
				DialogContent
			];
		
		SCameraCalibrationLinkedPointsDialog::OpenWindow(LOCTEXT("ChangeFocusValueWindowLabel", "Change Focus Value"), DialogBox);

		return bFocusValuesChanged;
	}

	return false;
}

bool FFocusDataListItem::RemoveLinkedFocusValues() const
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		bool bFocusValuesRemoved = false;

		auto OnApplyChange = [LensFilePtr, &bFocusValuesRemoved](const TArray<SCameraCalibrationLinkedPointsDialog::FLinkedItem>& LinkedItems)
		{
			FScopedTransaction Transaction(LOCTEXT("RemoveLinkedFocusPointsTransaction", "Remove Points"));
			LensFilePtr->Modify();

			for (const SCameraCalibrationLinkedPointsDialog::FLinkedItem& LinkedItem : LinkedItems)
			{
				if (LinkedItem.Zoom.IsSet())
				{
					LensFilePtr->RemoveZoomPoint(LinkedItem.Category, LinkedItem.Focus, LinkedItem.Zoom.GetValue());
				}
				else
				{
					LensFilePtr->RemoveFocusPoint(LinkedItem.Category, LinkedItem.Focus);
				}
			}

			bFocusValuesRemoved = true;
		};

		const SCameraCalibrationLinkedPointsDialog::FLinkedItem Item(Category, Focus);
		TSharedRef<SCameraCalibrationLinkedPointsDialog> DialogBox = SNew(SCameraCalibrationLinkedPointsDialog, LensFilePtr, Item)
			.LinkedItemMode(SCameraCalibrationLinkedPointsDialog::ELinkedItemMode::Both)
			.DialogText(LOCTEXT("RemoveLinkedFocusDialogText", "The calibration data you wish to delete may be inherently linked to additional data.\nChoose any and all linked data you wish to delete."))
			.AcceptButtonText(LOCTEXT("RemoveLinkedFocusAcceptButton", "Remove Selected"))
			.OnApplyLinkedAction_Lambda(OnApplyChange);

		SCameraCalibrationLinkedPointsDialog::OpenWindow(LOCTEXT("RemoveFocusWindowLabel", "Remove Points"), DialogBox);
		
		return bFocusValuesRemoved;
	}
	
	return false;
}

FZoomDataListItem::FZoomDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, const TSharedRef<FFocusDataListItem> InParent, float InZoom, FOnDataChanged InOnDataChangedCallback)
	: FLensDataListItem(InLensFile, InCategory, InSubCategoryIndex, InOnDataChangedCallback)
	, Zoom(InZoom)
	, WeakParent(InParent)
{
}

TSharedRef<ITableRow> FZoomDataListItem::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SLensDataItem, InOwnerTable, SharedThis(this))
		.EntryLabel(LOCTEXT("ZoomLabel", "Zoom: "))
		.EntryValue(Zoom)
		.AllowEditEntryValue(true)
		.OnEntryValueChanged(this, &FZoomDataListItem::OnZoomValueChanged)
		.AllowRemoval(SubCategoryIndex == INDEX_NONE )
		.EditPointVisibility(EVisibility::Visible)
		.AllowEditPoint(MakeAttributeLambda([this]() { return SubCategoryIndex == INDEX_NONE; }));
}

void FZoomDataListItem::OnRemoveRequested() const
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		if(TSharedPtr<FFocusDataListItem> ParentItem = WeakParent.Pin())
		{
			const FBaseLensTable* const LinkDataTable = LensFilePtr->GetDataTable(Category);
			if (!ensure(LinkDataTable))
			{
				return;	
			}
			
			if (LinkDataTable->HasLinkedZoomValues(ParentItem->Focus, Zoom))
			{
				if (RemoveLinkedZoomValues())
				{
					OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataRemoved, ParentItem->Focus, Zoom);
				}
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("RemoveZoomPointTransaction", "Remove Zoom Point"));
				LensFilePtr->Modify();
				LensFilePtr->RemoveZoomPoint(Category, ParentItem->Focus, Zoom);
				OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataRemoved, ParentItem->Focus, Zoom);
			}
		}
	}
}

TOptional<float> FZoomDataListItem::GetFocus() const
{
	if (TSharedPtr<FFocusDataListItem> ParentItem = WeakParent.Pin())
	{
		return ParentItem->GetFocus();
	}

	return TOptional<float>();
}

void FZoomDataListItem::EditItem()
{
	ULensFile* LensFilePtr = WeakLensFile.Get();
	if (!ensure(LensFilePtr))
	{
		return;
	}

	if (!ensure(GetFocus().IsSet()))
	{
		return;
	}

	FSimpleDelegate OnPointSaved = FSimpleDelegate::CreateLambda([this]()
	{
		OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataChanged, GetFocus().GetValue(), Zoom);	
	});
	
	switch (Category)
	{
		case ELensDataCategory::Zoom:
		{
			LensDataEditPointDialog::OpenDialog<FFocalLengthInfo>(LensFilePtr, Category, GetFocus().GetValue(), Zoom, LensFilePtr->FocalLengthTable, OnPointSaved);
			break;
		}
		case ELensDataCategory::ImageCenter:
		{
			LensDataEditPointDialog::OpenDialog<FImageCenterInfo>(LensFilePtr, Category, GetFocus().GetValue(), Zoom, LensFilePtr->ImageCenterTable, OnPointSaved);
			break;
		}
		case ELensDataCategory::Distortion:
		{
			LensDataEditPointDialog::OpenDialog<FDistortionInfo>(LensFilePtr, Category, GetFocus().GetValue(), Zoom, LensFilePtr->DistortionTable, OnPointSaved);
			break;
		}
		case ELensDataCategory::NodalOffset:
		{
			LensDataEditPointDialog::OpenDialog<FNodalPointOffset>(LensFilePtr, Category, GetFocus().GetValue(), Zoom, LensFilePtr->NodalOffsetTable, OnPointSaved);
			break;
		}
		case ELensDataCategory::STMap:
		{
			LensDataEditPointDialog::OpenDialog<FSTMapInfo>(LensFilePtr, Category, GetFocus().GetValue(), Zoom, LensFilePtr->STMapTable, OnPointSaved);
			break;
		}
	}
}

bool FZoomDataListItem::OnZoomValueChanged(float NewZoomValue)
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		if(TSharedPtr<FFocusDataListItem> ParentItem = WeakParent.Pin())
		{
			const FBaseLensTable* const DataTable = LensFilePtr->GetDataTable(Category);
			if (!ensure(DataTable))
			{
				return false;	
			}

			if (DataTable->HasLinkedZoomValues(ParentItem->Focus, Zoom))
			{
				// If there are any linked zoom values, allow the linked zoom dialog box to handle any needed changes to the focuses in the lens file
				if (ChangeLinkedZoomValues(NewZoomValue))
				{
					Zoom = NewZoomValue;
					OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataChanged, ParentItem->Focus, Zoom);
					return true;
				}
				
				return false;
			}
			
			if (LensFilePtr->HasZoomPoint(Category, ParentItem->Focus, NewZoomValue))
			{
				if (FCameraCalibrationWidgetHelpers::ShowReplaceZoomWarning())
				{
					FScopedTransaction Transaction(LOCTEXT("ReplaceZoomPointTransaction", "Replace Zoom Point"));
				
					LensFilePtr->Modify();
					LensFilePtr->ChangeZoomPoint(Category, ParentItem->Focus, Zoom, NewZoomValue);

					Zoom = NewZoomValue;
					OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataChanged, ParentItem->Focus, Zoom);

					return true;
				}
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("ChangeZoomPointTransaction", "Change Zoom Point"));
				
				LensFilePtr->Modify();
				LensFilePtr->ChangeZoomPoint(Category, ParentItem->Focus, Zoom, NewZoomValue);

				Zoom = NewZoomValue;
				OnDataChangedCallback.ExecuteIfBound(ELensDataChangedReason::DataChanged, ParentItem->Focus, Zoom);

				return true;
			}
		}
	}

	return false;
}

bool FZoomDataListItem::ChangeLinkedZoomValues(float NewZoomValue) const
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		if(TSharedPtr<FFocusDataListItem> ParentItem = WeakParent.Pin())
		{
			const float Focus = ParentItem->Focus;
			bool bZoomValuesChanged = false;
			bool bReplaceExisting = false;

			TSharedRef<SWidget> DialogContent = SNew(SCheckBox)
				.ToolTipText(LOCTEXT("ReplaceExistingZoomPointsToolTip", "When checked, any existing zoom points will be replaced with the zoom point being changed"))
				.IsChecked_Lambda([&bReplaceExisting]() { return bReplaceExisting ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([&bReplaceExisting](ECheckBoxState CheckBoxState) { bReplaceExisting = CheckBoxState == ECheckBoxState::Checked; })
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock).Text(LOCTEXT("ReplaceExistingZoomPointsLabel", "Replace existing zoom points?"))
				];
			
			auto OnApplyChange = [LensFilePtr, Focus, NewZoomValue, &bReplaceExisting, &bZoomValuesChanged](const TArray<SCameraCalibrationLinkedPointsDialog::FLinkedItem>& LinkedItems)
			{
				FScopedTransaction Transaction(LOCTEXT("ChangeLinkedZoomPointsTransaction", "Change Linked Zoom Points"));
				LensFilePtr->Modify();

				for (const SCameraCalibrationLinkedPointsDialog::FLinkedItem& LinkedItem : LinkedItems)
				{
					if (!LensFilePtr->HasZoomPoint(LinkedItem.Category, Focus, NewZoomValue) || bReplaceExisting)
					{
						LensFilePtr->ChangeZoomPoint(LinkedItem.Category, LinkedItem.Focus, LinkedItem.Zoom.GetValue(), NewZoomValue);
					}
				}
				
				bZoomValuesChanged = true;
			};
			
			const SCameraCalibrationLinkedPointsDialog::FLinkedItem Item(Category, ParentItem->Focus, Zoom);
			TSharedRef<SCameraCalibrationLinkedPointsDialog> DialogBox = SNew(SCameraCalibrationLinkedPointsDialog, LensFilePtr, Item)
				.LinkedItemMode(SCameraCalibrationLinkedPointsDialog::ELinkedItemMode::Zoom)
				.DialogText(LOCTEXT("ChangeLinkedZoomDialogText", "The calibration data you wish to change may be inherently linked to additional data.\nChoose any and all linked data you wish to change."))
				.AcceptButtonText(LOCTEXT("ChangeLinkedZoomAcceptButton", "Change Zoom"))
				.OnApplyLinkedAction_Lambda(OnApplyChange)
				[
					DialogContent
				];
			
			SCameraCalibrationLinkedPointsDialog::OpenWindow(LOCTEXT("ChangeLinkedZoomValuesWindowLabel", "Change Zoom Value"), DialogBox);

			return bZoomValuesChanged;
		}
	}

	return false;
}

bool FZoomDataListItem::RemoveLinkedZoomValues() const
{
	if (ULensFile* LensFilePtr = WeakLensFile.Get())
	{
		if (TSharedPtr<FFocusDataListItem> ParentItem = WeakParent.Pin())
		{
			bool bZoomValuesRemoved = false;

			auto OnApplyChange = [LensFilePtr, &bZoomValuesRemoved](const TArray<SCameraCalibrationLinkedPointsDialog::FLinkedItem>& LinkedItems)
			{
				FScopedTransaction Transaction(LOCTEXT("RemoveLinkedZoomPointsTransaction", "Remove Points"));
				LensFilePtr->Modify();

				for (const SCameraCalibrationLinkedPointsDialog::FLinkedItem& LinkedItem : LinkedItems)
				{
					if (LinkedItem.Zoom.IsSet())
					{
						LensFilePtr->RemoveZoomPoint(LinkedItem.Category, LinkedItem.Focus, LinkedItem.Zoom.GetValue());
					}
				}

				bZoomValuesRemoved = true;
			};

			const SCameraCalibrationLinkedPointsDialog::FLinkedItem Item(Category, ParentItem->Focus, Zoom);
			TSharedRef<SCameraCalibrationLinkedPointsDialog> DialogBox = SNew(SCameraCalibrationLinkedPointsDialog, LensFilePtr, Item)
				.LinkedItemMode(SCameraCalibrationLinkedPointsDialog::ELinkedItemMode::Zoom)
				.DialogText(LOCTEXT("RemoveLinkedZoomDialogText", "The calibration data you wish to delete may be inherently linked to additional data.\nChoose any and all linked data you wish to delete."))
				.AcceptButtonText(LOCTEXT("RemoveLinkedZoomAcceptButton", "Remove Selected"))
				.OnApplyLinkedAction_Lambda(OnApplyChange);

			SCameraCalibrationLinkedPointsDialog::OpenWindow(LOCTEXT("RemoveZoomWindowLabel", "Remove Points"), DialogBox);
		
			return bZoomValuesRemoved;
		}
	}
	
	return false;
}

void SLensDataItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FLensDataListItem> InItemData)
{
	WeakItem = InItemData;
	OnEntryValueChanged = InArgs._OnEntryValueChanged;
	EntryValue = InArgs._EntryValue;

	const TAttribute<bool> AllowEditEntryValue = InArgs._AllowEditEntryValue;

	STableRow<TSharedPtr<FLensDataListItem>>::Construct(
		STableRow<TSharedPtr<FLensDataListItem>>::FArguments()
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(5.0f, 5.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(InArgs._EntryLabel)
			]
			+ SHorizontalBox::Slot()
			.Padding(5.0f, 5.0f)
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([AllowEditEntryValue]() { return AllowEditEntryValue.Get(false) ? 1 : 0; })

				+SWidgetSwitcher::Slot()
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(EntryValue))
				]

				+SWidgetSwitcher::Slot()
				[
					SNew(SNumericEntryBox<float>)
					.Value_Lambda([this]() { return EntryValue; })
					.OnValueCommitted(this, &SLensDataItem::OnEntryValueCommitted)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SLensDataItem::OnEditPointClicked)
				.IsEnabled(InArgs._AllowEditPoint)
				.Visibility(InArgs._EditPointVisibility)
				.ButtonStyle(FAppStyle::Get(), "FlatButton")
				.ToolTipText(LOCTEXT("EditLensDataPoint", "Edit the value at this point"))
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Edit"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.IsEnabled(InArgs._AllowRemoval)
				.OnClicked(this, &SLensDataItem::OnRemovePointClicked)
				.ButtonStyle(FAppStyle::Get(), "FlatButton")
				.ToolTipText(LOCTEXT("RemoveLensDataPoint", "Remove this point"))
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				]
			]
		], OwnerTable);
}

FReply SLensDataItem::OnRemovePointClicked() const
{
	if (TSharedPtr<FLensDataListItem> Item = WeakItem.Pin())
	{
		Item->OnRemoveRequested();
	}

	return FReply::Handled();
}

void SLensDataItem::OnEntryValueCommitted(float NewValue, ETextCommit::Type CommitType)
{
	// Avoid duplicate handling if we are already waiting for a commit to resolve, which could happen
	// if OnEntryValueChange invoked a modal and is waiting for a response from the user
	if (bIsCommittingValue)
	{
		return;
	}

	bIsCommittingValue = true;
	if (EntryValue != NewValue)
	{
		if (OnEntryValueChanged.IsBound() && OnEntryValueChanged.Execute(NewValue))
		{
			EntryValue = NewValue;
		}
	}
	bIsCommittingValue = false;
}

FReply SLensDataItem::OnEditPointClicked() const
{
	if (TSharedPtr<FLensDataListItem> Item = WeakItem.Pin())
	{
		Item->EditItem();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE /* LensDataListItem */
