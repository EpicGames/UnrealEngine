// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#include "CameraCalibrationEditorCommon.h"
#include "LensFile.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

enum class ELensDataChangedReason : uint8
{
	DataRemoved,
	DataChanged
};

DECLARE_DELEGATE_ThreeParams(FOnDataChanged, ELensDataChangedReason /** ChangedReason */, float /** Focus */, TOptional<float> /** Possible Zoom */);

/**
* Data entry item
*/
class FLensDataListItem : public TSharedFromThis<FLensDataListItem>
{
public:
	FLensDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, FOnDataChanged InOnDataChangedCallback);

	virtual ~FLensDataListItem() = default;
	
	virtual void OnRemoveRequested() const = 0;
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) = 0;
	virtual TOptional<float> GetFocus() const { return TOptional<float>(); }
	virtual TOptional<float> GetZoom() const { return TOptional<float>(); }
	virtual int32 GetIndex() const { return INDEX_NONE; }
	virtual void EditItem() {};

	/** Lens data category of that entry */
	ELensDataCategory Category;

	/** Used to know if it's a root category or not */
	int32 SubCategoryIndex;

	/** LensFile we're editing */
	TWeakObjectPtr<ULensFile> WeakLensFile;

	/** Children of this item */
	TArray<TSharedPtr<FLensDataListItem>> Children;

	/** Delegate to call when data is changed */
	FOnDataChanged OnDataChangedCallback;
};

/**
 * Encoder item
 */
class FEncoderDataListItem : public FLensDataListItem
{
public:
	FEncoderDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, float InInput, int32 InIndex, FOnDataChanged InOnDataChangedCallback);

	virtual void OnRemoveRequested() const override;
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) override;

	/** Encoder input */
	float InputValue;

	/** Identifier for this focus point */
	int32 EntryIndex;
};

/**
 * Data entry item
 */
class FFocusDataListItem : public FLensDataListItem
{
public:
	FFocusDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, float InFocus, FOnDataChanged InOnDataChangedCallback);

	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) override;
	virtual void OnRemoveRequested() const override;
	virtual TOptional<float> GetFocus() const override { return Focus; }

	/** Raised when the focus value is changed on this focus item */
	bool OnFocusValueChanged(float NewFocusValue);

	/** Creates a dialog box that allows users to change linked focus values when this focus item is changed
	 * @returns true if the user presses the accept button on the dialog, false otherwise */
	bool ChangeLinkedFocusValues(float NewFocusValue) const;

	/** Creates a dialog box that allows users to remove linked focus and zoom values when this focus item is removed
	 * @returns true if the user presses the accept button on the dialog, false otherwise */
	bool RemoveLinkedFocusValues() const;
	
	/** Focus value of this item */
	float Focus;
};

/**
 * Zoom data entry item
 */
class FZoomDataListItem : public FLensDataListItem
{
public:
	FZoomDataListItem(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, const TSharedRef<FFocusDataListItem> InParent, float InZoom, FOnDataChanged InOnDataChangedCallback);

	//~ Begin FLensDataListItem interface
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable) override;
	virtual void OnRemoveRequested() const override;
	virtual TOptional<float> GetFocus() const override;
	virtual TOptional<float> GetZoom() const override { return Zoom; }
	virtual void EditItem() override;
	//~ End FLensDataListItem interface

	/** Raised when the zoom value is changed on this zoom item */
	bool OnZoomValueChanged(float NewZoomValue);

	/** Creates a dialog box that allows users to change linked zoom values when this zoom item is changed
	 * @returns true if the user presses the accept button on the dialog, false otherwise */
	bool ChangeLinkedZoomValues(float NewZoomValue) const;

	/** Creates a dialog box that allows users to remove linked zoom values when this zoom item is removed
	 * @returns true if the user presses the accept button on the dialog, false otherwise */
	bool RemoveLinkedZoomValues() const;
	
	/** Zoom value of this item */
	float Zoom = 0.0f;

	/** Focus this zoom point is associated with */
	TWeakPtr<FFocusDataListItem> WeakParent;
};

/**
 * Widget a focus point entry
 */
class SLensDataItem : public STableRow<TSharedPtr<FLensDataListItem>>
{
public:
	/** Delegate raised when the entry value of the data item has been changed. Returns whether the change should be committed or not */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnEntryValueChanged, float /*NewValue*/);

	SLATE_BEGIN_ARGS(SLensDataItem)
		:  _EntryLabel(FText::GetEmpty())
		,  _EntryValue(0.f)
		, _AllowRemoval(false)
		, _EditPointVisibility(EVisibility::Collapsed)
		, _AllowEditPoint(false)
		{}

		SLATE_ARGUMENT(FText, EntryLabel)

		SLATE_ARGUMENT(float, EntryValue)

		SLATE_ARGUMENT(bool, AllowRemoval)

		/** Whether Item point Visible */
		SLATE_ARGUMENT(EVisibility, EditPointVisibility)

		/** Whether Item point editable */
		SLATE_ATTRIBUTE(bool, AllowEditPoint)

		/** Whether the entry's value should be editable */
		SLATE_ATTRIBUTE(bool, AllowEditEntryValue)
		SLATE_EVENT(FOnEntryValueChanged, OnEntryValueChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FLensDataListItem> InItemData);

private:
	/** Edit Button Handler */
	FReply OnEditPointClicked() const;

	/** Remove Button Handler */
	FReply OnRemovePointClicked() const;

	void OnEntryValueCommitted(float NewValue, ETextCommit::Type CommitType);

private:

	/** WeakPtr to source data item */
	TWeakPtr<FLensDataListItem> WeakItem;

	float EntryValue = 0.0;
	bool bIsCommittingValue = false;
	FOnEntryValueChanged OnEntryValueChanged;
};
