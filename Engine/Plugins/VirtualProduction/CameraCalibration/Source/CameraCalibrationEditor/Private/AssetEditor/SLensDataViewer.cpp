// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensDataViewer.h"

#include "SCameraCalibrationCurveEditorPanel.h"

#include "CameraCalibrationCurveEditor.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationStepsController.h"
#include "CameraCalibrationTimeSliderController.h"
#include "CurveEditorSettings.h"
#include "ICurveEditorModule.h"
#include "ISinglePropertyView.h"
#include "LensFile.h"
#include "RichCurveEditorModel.h"
#include "SLensDataAddPointDialog.h"
#include "SLensDataCategoryListItem.h"
#include "SLensDataListItem.h"

#include "Curves/LensDataCurveModel.h"
#include "Curves/LensEncodersCurveModel.h"
#include "Curves/LensMultiAxisCurveModel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SSpacer.h"


#define LOCTEXT_NAMESPACE "LensDataViewer"

namespace LensDataUtils
{
	const FName EncoderCategoryLabel(TEXT("Encoders"));
	const FName EncoderFocusLabel(TEXT("Focus"));
	const FName EncoderIrisLabel(TEXT("Iris"));
	const FName EncoderZoomLabel(TEXT("Focal Length"));
	const FName DistortionCategoryLabel(TEXT("Distortion"));
	const FName FxLabel(TEXT("Fx"));
	const FName FyLabel(TEXT("Fy"));
	const FName MapsCategoryLabel(TEXT("STMaps"));
	const FName ImageCenterCategory(TEXT("Image Center"));
	const FName CxLabel(TEXT("Cx"));
	const FName CyLabel(TEXT("Cy"));
	const FName NodalOffsetCategoryLabel(TEXT("Nodal Offset"));
	const FName LocationXLabel(TEXT("Location - X"));
	const FName LocationYLabel(TEXT("Location - Y"));
	const FName LocationZLabel(TEXT("Location - Z"));
	const FName RotationXLabel(TEXT("Yaw"));
	const FName RotationYLabel(TEXT("Pitch"));
	const FName RotationZLabel(TEXT("Roll"));

	template<typename TFocusPoint>
	void MakeFocusEntries(ULensFile* InLensFile, ELensDataCategory InCategory, int32 InSubCategoryIndex, TConstArrayView<TFocusPoint> FocusPoints, TArray<TSharedPtr<FLensDataListItem>>& OutDataItems, FOnDataChanged InDataChangedCallback)
	{
		OutDataItems.Reserve(FocusPoints.Num());
		for (const TFocusPoint& Point : FocusPoints)
		{
			//Add entry for focus
			TSharedPtr<FFocusDataListItem> CurrentFocus = MakeShared<FFocusDataListItem>(InLensFile, InCategory, InSubCategoryIndex, Point.Focus, InDataChangedCallback);
			OutDataItems.Add(CurrentFocus);

			for(int32 Index = 0; Index < Point.GetNumPoints(); ++Index)
			{
				//Add zoom points for this focus
				TSharedPtr<FZoomDataListItem> ZoomItem = MakeShared<FZoomDataListItem>(InLensFile, InCategory, InSubCategoryIndex, CurrentFocus.ToSharedRef(),Point.GetZoom(Index), InDataChangedCallback);
				CurrentFocus->Children.Add(ZoomItem);
			}
		}
	}
}

/**
 * Custom curve bounds based on live input
 */
class FCameraCalibrationCurveEditorBounds : public ICurveEditorBounds
{
public:
	FCameraCalibrationCurveEditorBounds(TSharedPtr<ITimeSliderController> InExternalTimeSliderController)
		: TimeSliderControllerWeakPtr(InExternalTimeSliderController)
	{}

	virtual void GetInputBounds(double& OutMin, double& OutMax) const override
	{
		if (const TSharedPtr<ITimeSliderController> ExternalTimeSliderController = TimeSliderControllerWeakPtr.Pin())
		{
			const FAnimatedRange ViewRange = ExternalTimeSliderController->GetViewRange();
			OutMin = ViewRange.GetLowerBoundValue();
			OutMax = ViewRange.GetUpperBoundValue();
		}
	}

	virtual void SetInputBounds(double InMin, double InMax) override
	{
		if (const TSharedPtr<ITimeSliderController> ExternalTimeSliderController = TimeSliderControllerWeakPtr.Pin())
		{
			ExternalTimeSliderController->SetViewRange(InMin, InMax, EViewRangeInterpolation::Immediate);
		}
	}

	TWeakPtr<ITimeSliderController> TimeSliderControllerWeakPtr;
};

void SLensDataViewer::Construct(const FArguments& InArgs, ULensFile* InLensFile, const TSharedRef<FCameraCalibrationStepsController>& InCalibrationStepsController)
{
	const UCameraCalibrationEditorSettings* EditorSettings = GetDefault<UCameraCalibrationEditorSettings>();
	
	LensFile = TStrongObjectPtr<ULensFile>(InLensFile);
		
	//Setup curve editor
	CurveEditor = MakeShared<FCameraCalibrationCurveEditor>();
	const FCurveEditorInitParams InitParams;
	CurveEditor->InitCurveEditor(InitParams);
	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");
	
	TUniquePtr<ICurveEditorBounds> EditorBounds;

	// We need to keep Time Slider outside the scope in order to be valid whe it passed to CurveEditorPanel
	TSharedPtr<FCameraCalibrationTimeSliderController> TimeSliderController;
	if (EditorSettings->bEnableTimeSlider)
	{
		TimeSliderController = MakeShared<FCameraCalibrationTimeSliderController>(InCalibrationStepsController, InLensFile);
		TimeSliderControllerWeakPtr = TimeSliderController;
		EditorBounds = MakeUnique<FCameraCalibrationCurveEditorBounds>(TimeSliderController);
	}
	else
	{
		EditorBounds = MakeUnique<FStaticCurveEditorBounds>();
		EditorBounds->SetInputBounds(0.05, 1.05);
	}
	CurveEditor->SetBounds(MoveTemp(EditorBounds));

	// Set zoom as mouse zoom by default
	check(CurveEditor->GetSettings()); // Should be valid all the time
	CurveEditor->GetSettings()->SetZoomPosition(ECurveEditorZoomPosition::MousePosition);

	// Set Delegates
	CurveEditor->OnAddDataPointDelegate.BindSP(this, &SLensDataViewer::OnAddDataPointHandler);

	// Snap only Y axis
	FCurveEditorAxisSnap SnapYAxisOnly = CurveEditor->GetAxisSnap();
	SnapYAxisOnly.RestrictedAxisList = ECurveEditorSnapAxis::CESA_Y;
	CurveEditor->SetAxisSnap(SnapYAxisOnly);

	CurvePanel = SNew(SCameraCalibrationCurveEditorPanel, CurveEditor.ToSharedRef(), TimeSliderController);
	CurveEditor->ZoomToFit();

	CachedFIZ = InArgs._CachedFIZData;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeToolbarWidget(CurvePanel.ToSharedRef())
		]
		+ SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.4f)
			[
				MakeLensDataWidget()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.6f)
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				[
					CurvePanel.ToSharedRef()
				]

				+SOverlay::Slot()
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					[
						SNew(SSpacer)
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						MakeCurveEditorToolbarWidget()
					]
				]
			]
		]
	];

	Refresh();
}

TSharedPtr<FLensDataCategoryItem> SLensDataViewer::GetDataCategorySelection() const
{
	TArray<TSharedPtr<FLensDataCategoryItem>> SelectedNodes;
	TreeView->GetSelectedItems(SelectedNodes);
	if (SelectedNodes.Num())
	{
		return SelectedNodes[0];
	}
	return nullptr;
}

TSharedRef<ITableRow> SLensDataViewer::OnGenerateDataCategoryRow(TSharedPtr<FLensDataCategoryItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return Item->MakeTreeRowWidget(OwnerTable);
}

void SLensDataViewer::OnGetDataCategoryItemChildren(TSharedPtr<FLensDataCategoryItem> Item, TArray<TSharedPtr<FLensDataCategoryItem>>& OutChildren)
{
	if (Item.IsValid())
	{
		OutChildren = Item->Children;
	}
}

void SLensDataViewer::OnDataCategorySelectionChanged(TSharedPtr<FLensDataCategoryItem> Item, ESelectInfo::Type SelectInfo)
{
	//Don't filter based on SelectInfo. We want to update on arrow key usage
	RefreshDataEntriesTree();
}

TSharedPtr<FLensDataListItem> SLensDataViewer::GetSelectedDataEntry() const
{
	TArray<TSharedPtr<FLensDataListItem>> SelectedNodes;
	DataEntriesTree->GetSelectedItems(SelectedNodes);
	if (SelectedNodes.Num())
	{
		return SelectedNodes[0];
	}
	return nullptr;
}

TSharedRef<ITableRow> SLensDataViewer::OnGenerateDataEntryRow(TSharedPtr<FLensDataListItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return Item->MakeTreeRowWidget(OwnerTable);
}

void SLensDataViewer::OnGetDataEntryChildren(TSharedPtr<FLensDataListItem> Item, TArray<TSharedPtr<FLensDataListItem>>& OutItems)
{
	if (Item.IsValid())
	{
		OutItems = Item->Children;
	}
}

void SLensDataViewer::OnDataEntrySelectionChanged(TSharedPtr<FLensDataListItem> Node, ESelectInfo::Type SelectInfo)
{
	RefreshCurve();

	RefreshTimeSlider();
}

void SLensDataViewer::PostUndo(bool bSuccess)
{
	//Items in category could have changed
	RefreshDataEntriesTree();	
}

void SLensDataViewer::PostRedo(bool bSuccess)
{
	//Items in category could have changed
	RefreshDataEntriesTree();
}

TSharedRef<SWidget> SLensDataViewer::MakeLensDataWidget()
{
	const FSinglePropertyParams InitParams;
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const TSharedPtr<ISinglePropertyView> DataModeWidget = PropertyEditorModule.CreateSingleProperty(
		LensFile.Get()
		, GET_MEMBER_NAME_CHECKED(ULensFile, DataMode)
		, InitParams);

	FSimpleDelegate OnDataModeChangedDelegate = FSimpleDelegate::CreateSP(this, &SLensDataViewer::OnDataModeChanged);
	DataModeWidget->SetOnPropertyValueChanged(OnDataModeChangedDelegate);

	return 
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				DataModeWidget.ToSharedRef()
			]
		]
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.FillHeight(.5f)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				SAssignNew(TreeView, STreeView<TSharedPtr<FLensDataCategoryItem>>)
				.TreeItemsSource(&DataCategories)
				.OnGenerateRow(this, &SLensDataViewer::OnGenerateDataCategoryRow)
				.OnGetChildren(this, &SLensDataViewer::OnGetDataCategoryItemChildren)
				.OnSelectionChanged(this, &SLensDataViewer::OnDataCategorySelectionChanged)
				.ClearSelectionOnClick(false)
			]
			
		]
		+ SVerticalBox::Slot()
		.Padding(5.0f, 5.0f)
		.FillHeight(.5f)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						if(const TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection())
						{
							return FText::FromName(CategoryItem->Label);
						}
						else
						{
							return LOCTEXT("NoCategorySelected", "Select a category");
						}
					})
				]
			+ SVerticalBox::Slot()
			[
					SAssignNew(DataEntriesTree, STreeView<TSharedPtr<FLensDataListItem>>)
					.TreeItemsSource(&DataEntries)
					.OnGenerateRow(this, &SLensDataViewer::OnGenerateDataEntryRow)
					.OnGetChildren(this, &SLensDataViewer::OnGetDataEntryChildren)
					.OnSelectionChanged(this, &SLensDataViewer::OnDataEntrySelectionChanged)
					.ClearSelectionOnClick(false)
				]
			]
		];
}

TSharedRef<SWidget> SLensDataViewer::MakeToolbarWidget(TSharedRef<SCameraCalibrationCurveEditorPanel> InEditorPanel)
{
	// Curve toolbar
	FToolBarBuilder ToolBarBuilder(CurvePanel->GetCommands(), FMultiBoxCustomization::None, CurvePanel->GetToolbarExtender(), true);
	ToolBarBuilder.BeginSection("Asset");
	ToolBarBuilder.BeginStyleOverride("AssetEditorToolbar");

	TSharedRef<SWidget> AddPointButton =
			 SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("AddLensDataPoint", "Add a lens data point"))
			.OnClicked_Lambda([this]()
			{
				OnAddDataPointHandler();
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
			];

	TSharedRef<SWidget> ClearAllButton =
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("DeleteLensData", "Delete all calibrated lens data"))
			.OnClicked(this, &SLensDataViewer::OnClearLensFileClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
			];

	ToolBarBuilder.AddSeparator();
	ToolBarBuilder.AddWidget(AddPointButton);
	ToolBarBuilder.AddSeparator();
	ToolBarBuilder.AddWidget(ClearAllButton);
	ToolBarBuilder.AddSeparator();
		
	ToolBarBuilder.EndSection();

	return SNew(SBox)
		.Padding(FMargin(2.f, 0.f))
		[
			ToolBarBuilder.MakeWidget()
		];
}

TSharedRef<SWidget> SLensDataViewer::MakeCurveEditorToolbarWidget()
{
	return SNew(SHorizontalBox)

	+SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
		.Visibility(this, &SLensDataViewer::GetCurveAxisButtonVisibility)
		[
			SNew(SStackBox)
			.Orientation(Orient_Horizontal)

			+SStackBox::Slot()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckBox")
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					CurveAxisType = ELensCurveAxis::Zoom;
					RefreshCurve();
				})
				.IsChecked_Lambda([this]() { return CurveAxisType == ELensCurveAxis::Zoom ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.Padding(4.f)
				[
					SNew(STextBlock).Text(LOCTEXT("ZoomAxisLabel", "Zoom"))
				]
			]
			
			+SStackBox::Slot()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckBox")
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					CurveAxisType = ELensCurveAxis::Focus;
					RefreshCurve();
				})
				.IsChecked_Lambda([this]() { return CurveAxisType == ELensCurveAxis::Focus ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.Padding(4.f)
				[
					SNew(STextBlock).Text(LOCTEXT("FocusAxisLabel", "Focus"))
				]
			]
		]
	];
}

void SLensDataViewer::OnAddDataPointHandler()
{
	const FSimpleDelegate OnDataPointAdded = FSimpleDelegate::CreateSP(this, &SLensDataViewer::OnLensDataPointAdded);

	ELensDataCategory InitialCategory = ELensDataCategory::Distortion;
	if (TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection())
	{
		InitialCategory = CategoryItem->Category;
	}

	SLensDataAddPointDialog::OpenDialog(LensFile.Get(), InitialCategory, CachedFIZ, OnDataPointAdded);
}

FReply SLensDataViewer::OnClearLensFileClicked()
{
	FScopedTransaction Transaction(LOCTEXT("LensFileClearAll", "Cleared LensFile"));
	LensFile->Modify();

	//Warn the user that they are about to clear everything
	const FText Message = LOCTEXT("ClearAllWarning", "This will erase all data contained in this LensFile. Do you wish to continue?");
	if (FMessageDialog::Open(EAppMsgType::OkCancel, Message) != EAppReturnType::Ok)
	{
		Transaction.Cancel();
		return FReply::Handled();
	}

	LensFile->ClearAll();
	RefreshDataEntriesTree();

	return FReply::Handled();
}

void SLensDataViewer::OnDataModeChanged()
{
	Refresh();
}

void SLensDataViewer::Refresh()
{
	RefreshDataCategoriesTree();
	RefreshDataEntriesTree();
}

void SLensDataViewer::RefreshDataCategoriesTree()
{
	//Builds the data category tree
	
	DataCategories.Reset();

	DataCategories.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), nullptr, ELensDataCategory::Focus, INDEX_NONE, LensDataUtils::EncoderFocusLabel));
	DataCategories.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), nullptr, ELensDataCategory::Iris, INDEX_NONE, LensDataUtils::EncoderIrisLabel));

	TSharedPtr<FLensDataCategoryItem> FocalLengthCategory = MakeShared<FLensDataCategoryItem>(
		LensFile.Get(), nullptr, ELensDataCategory::Zoom, FFocalLengthTable::FParameters::Aggregate,
		LensDataUtils::EncoderZoomLabel);
	
	FocalLengthCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), FocalLengthCategory, ELensDataCategory::Zoom, FFocalLengthTable::FParameters::Fx, LensDataUtils::FxLabel));
	FocalLengthCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), FocalLengthCategory, ELensDataCategory::Zoom, FFocalLengthTable::FParameters::Fy, LensDataUtils::FyLabel));
	DataCategories.Add(FocalLengthCategory);

	
	if (LensFile->DataMode == ELensDataMode::Parameters)
	{
		TSharedPtr<FLensDataCategoryItem> DistortionEntry = MakeShared<FLensDataCategoryItem>(
			LensFile.Get(), nullptr, ELensDataCategory::Distortion, FDistortionTable::FParameters::Aggregate,
			LensDataUtils::DistortionCategoryLabel);
		
		DataCategories.Add(DistortionEntry);

		TArray<FText> Parameters;
		if (LensFile->LensInfo.LensModel)
		{
			Parameters = LensFile->LensInfo.LensModel.GetDefaultObject()->GetParameterDisplayNames();
		}

		for (int32 Index = 0; Index< Parameters.Num(); ++Index)
		{
			const FText& Parameter = Parameters[Index];
			DistortionEntry->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), DistortionEntry, ELensDataCategory::Distortion, Index, *Parameter.ToString()));
		}
	}
	else
	{
		DataCategories.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), nullptr, ELensDataCategory::STMap, INDEX_NONE, LensDataUtils::MapsCategoryLabel));
	}

	TSharedPtr<FLensDataCategoryItem> ImageCenterEntry = MakeShared<FLensDataCategoryItem>(LensFile.Get(), nullptr, ELensDataCategory::ImageCenter, INDEX_NONE, LensDataUtils::ImageCenterCategory);
	DataCategories.Add(ImageCenterEntry);
	ImageCenterEntry->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), ImageCenterEntry, ELensDataCategory::ImageCenter, FImageCenterTable::FParameters::Cx, LensDataUtils::CxLabel));
	ImageCenterEntry->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), ImageCenterEntry, ELensDataCategory::ImageCenter, FImageCenterTable::FParameters::Cy, LensDataUtils::CyLabel));

	TSharedPtr<FLensDataCategoryItem> NodalOffsetCategory = MakeShared<FLensDataCategoryItem>(LensFile.Get(), nullptr, ELensDataCategory::NodalOffset, INDEX_NONE, LensDataUtils::NodalOffsetCategoryLabel);
	DataCategories.Add(NodalOffsetCategory);

	{
		using FParameters = FNodalOffsetTable::FParameters;
		const ELensDataCategory Category = ELensDataCategory::NodalOffset;
		
		NodalOffsetCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), NodalOffsetCategory, Category, FParameters::Compose(FParameters::Location, EAxis::X), LensDataUtils::LocationXLabel));
		NodalOffsetCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), NodalOffsetCategory, Category, FParameters::Compose(FParameters::Location, EAxis::Y), LensDataUtils::LocationYLabel));
		NodalOffsetCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), NodalOffsetCategory, Category, FParameters::Compose(FParameters::Location, EAxis::Z), LensDataUtils::LocationZLabel));
		NodalOffsetCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), NodalOffsetCategory, Category, FParameters::Compose(FParameters::Rotation, EAxis::X), LensDataUtils::RotationXLabel));
		NodalOffsetCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), NodalOffsetCategory, Category, FParameters::Compose(FParameters::Rotation, EAxis::Y), LensDataUtils::RotationYLabel));
		NodalOffsetCategory->Children.Add(MakeShared<FLensDataCategoryItem>(LensFile.Get(), NodalOffsetCategory, Category, FParameters::Compose(FParameters::Rotation, EAxis::Z), LensDataUtils::RotationZLabel));
	}
	
	TreeView->RequestTreeRefresh();
}

void SLensDataViewer::RefreshDataEntriesTree()
{
	TSharedPtr<FLensDataListItem> CurrentSelection = GetSelectedDataEntry();

	// Save the items that are expanded, so that the expanded state can be restored after the tree has been refreshed
	TSet<TSharedPtr<FLensDataListItem>> ExpandedItems;
	DataEntriesTree->GetExpandedItems(ExpandedItems);

	TArray<float> FocusesToExpand;
	for (const TSharedPtr<FLensDataListItem>& Item : ExpandedItems)
	{
		TOptional<float> ItemFocus = Item->GetFocus();
		if (ItemFocus.IsSet())
		{
			FocusesToExpand.Add(ItemFocus.GetValue());
		}
	}
	
	DataEntries.Reset();

	if (TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection())
	{
		FOnDataChanged DataChangedCallback = FOnDataChanged::CreateSP(this, &SLensDataViewer::OnDataPointChanged);

		switch (CategoryItem->Category)
		{
			case ELensDataCategory::Focus:
			{
				for (int32 Index = 0; Index <LensFile->EncodersTable.GetNumFocusPoints(); ++Index)
				{
					DataEntries.Add(MakeShared<FEncoderDataListItem>(LensFile.Get(), CategoryItem->Category, LensFile->EncodersTable.GetFocusInput(Index), Index, DataChangedCallback));
				}
				break;
			}
			case ELensDataCategory::Iris:
			{
				for (int32 Index = 0; Index <LensFile->EncodersTable.GetNumIrisPoints(); ++Index)
				{
					DataEntries.Add(MakeShared<FEncoderDataListItem>(LensFile.Get(), CategoryItem->Category, LensFile->EncodersTable.GetIrisInput(Index), Index, DataChangedCallback));
				}
				break;
			}
			case ELensDataCategory::Zoom:
			{
				const TConstArrayView<FFocalLengthFocusPoint> FocusPoints = LensFile->FocalLengthTable.GetFocusPoints();
				LensDataUtils::MakeFocusEntries(LensFile.Get(), CategoryItem->Category, CategoryItem->ParameterIndex, FocusPoints, DataEntries, DataChangedCallback);
				break;
			}
			case ELensDataCategory::Distortion:
			{
				const TConstArrayView<FDistortionFocusPoint> FocusPoints = LensFile->DistortionTable.GetFocusPoints();
				LensDataUtils::MakeFocusEntries(LensFile.Get(), CategoryItem->Category, CategoryItem->ParameterIndex, FocusPoints, DataEntries, DataChangedCallback);
				break;
			}
			case ELensDataCategory::ImageCenter:
			{
				const TConstArrayView<FImageCenterFocusPoint> FocusPoints = LensFile->ImageCenterTable.GetFocusPoints();
				LensDataUtils::MakeFocusEntries(LensFile.Get(), CategoryItem->Category, CategoryItem->ParameterIndex, FocusPoints, DataEntries, DataChangedCallback);
				break;
			}
			case ELensDataCategory::NodalOffset:
			{
				const TConstArrayView<FNodalOffsetFocusPoint> Points = LensFile->NodalOffsetTable.GetFocusPoints();
				LensDataUtils::MakeFocusEntries(LensFile.Get(), CategoryItem->Category, CategoryItem->ParameterIndex, Points, DataEntries, DataChangedCallback);
				break;
			}
			case ELensDataCategory::STMap:
			{
				const TConstArrayView<FSTMapFocusPoint> Points = LensFile->STMapTable.GetFocusPoints();
				LensDataUtils::MakeFocusEntries(LensFile.Get(), CategoryItem->Category, CategoryItem->ParameterIndex, Points, DataEntries, DataChangedCallback);
				break;
			}
		}
	}

	//When data entries have been repopulated, refresh the tree and select first item
	DataEntriesTree->RequestListRefresh();

	// Restore the expanded items by focus. 
	for (const TSharedPtr<FLensDataListItem>& Item : DataEntries)
	{
		TOptional<float> ItemFocus = Item->GetFocus();
		if (ItemFocus.IsSet() && FocusesToExpand.Contains(ItemFocus.GetValue()))
		{
			DataEntriesTree->SetItemExpansion(Item, true);
		}
	}
	
	//Try to put back the same selected Focus/Zoom item
	UpdateDataSelection(CurrentSelection);
}

void SLensDataViewer::RefreshCurve() const
{
	CurveEditor->RemoveAllCurves();
	TUniquePtr<FLensDataCurveModel> NewCurve;

	TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection();
	if (CategoryItem.IsValid())
	{
		const ELensDataCategory Category = CategoryItem->Category;
		if (Category == ELensDataCategory::Focus || Category == ELensDataCategory::Iris)
		{
			const EEncoderType EncoderType = Category == ELensDataCategory::Focus ? EEncoderType::Focus : EEncoderType::Iris;
			NewCurve = MakeUnique<FLensEncodersCurveModel>(LensFile.Get(), EncoderType);
		}
		else
		{
			const TSharedPtr<FLensDataListItem> CurrentDataItem = GetSelectedDataEntry();
			if (CurrentDataItem)
			{
				TOptional<float> CurveValue = TOptional<float>();
				if (CurveAxisType == ELensCurveAxis::Focus)
				{
					CurveValue = CurrentDataItem->GetZoom();
					if (!CurveValue.IsSet() && CurrentDataItem->Children.Num())
					{
						// If the current data item does not have a zoom value, check to see if it has any children, and if so,
						// use the zoom from the first child
						CurveValue = CurrentDataItem->Children[0]->GetZoom();
					}
				}
				else
				{
					CurveValue = CurrentDataItem->GetFocus();
				}

				if (CurveValue.IsSet())
				{
					NewCurve = MakeUnique<FLensDataMultiAxisCurveModel>(LensFile.Get(), CategoryItem->Category, CurveAxisType, CurveValue.GetValue(), CategoryItem->ParameterIndex);
				}
			}
		}
	}

	//If a curve was setup, add it to the editor
	if (NewCurve && NewCurve->IsValid())
	{
		NewCurve->SetShortDisplayName(FText::FromName(CategoryItem->Label));
		const UCameraCalibrationEditorSettings* EditorSettings = GetDefault<UCameraCalibrationEditorSettings>();
		NewCurve->SetColor(EditorSettings->CategoryColor.GetColorForCategory(CategoryItem->Category));
		const FCurveModelID CurveId = CurveEditor->AddCurve(MoveTemp(NewCurve));
		CurveEditor->PinCurve(CurveId);
	}
}

void SLensDataViewer::RefreshTimeSlider() const
{
	const TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection();
	const TSharedPtr<FLensDataListItem> CurrentDataItem = GetSelectedDataEntry();
	const TSharedPtr<FCameraCalibrationTimeSliderController> TimeSliderController = TimeSliderControllerWeakPtr.Pin();
	if (!TimeSliderController.IsValid())
	{
		return;
	}
	if (!CurrentDataItem.IsValid() || !CategoryItem.IsValid())
	{
		TimeSliderController->ResetSelection();
		return;
	}

	const bool bHasParameterIndex = CategoryItem->Category == ELensDataCategory::ImageCenter || CategoryItem->Category == ELensDataCategory::NodalOffset;
	const int32 ParameterIndex = CategoryItem->ParameterIndex;

	// Reset selection if no selection of the curve for ImageCenter or NodalOffset
	if (bHasParameterIndex && ParameterIndex == INDEX_NONE)
	{
		TimeSliderController->ResetSelection();
	}
	else
	{
		TimeSliderController->UpdateSelection(CurrentDataItem->Category, CurrentDataItem->GetFocus());
	}
}

void SLensDataViewer::OnLensDataPointAdded()
{
	RefreshDataEntriesTree();
}

void SLensDataViewer::OnDataPointChanged(ELensDataChangedReason ChangedReason, float InFocus, TOptional<float> InZoom)
{
	RefreshDataEntriesTree();
}

void SLensDataViewer::OnDataTablePointsUpdated(ELensDataCategory InCategory)
{
	if (TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection())
	{
		if (CategoryItem->Category == InCategory)
		{
			RefreshDataEntriesTree();
		}
	}
}

void SLensDataViewer::UpdateDataSelection(const TSharedPtr<FLensDataListItem>& PreviousSelection)
{
	if (PreviousSelection.IsValid())
	{
		const TOptional<float> FocusValue = PreviousSelection->GetFocus();
		if(FocusValue.IsSet())
		{
			for(const TSharedPtr<FLensDataListItem>& Item : DataEntries)
			{
				if(Item->GetFocus() == FocusValue)
				{
					DataEntriesTree->SetSelection(Item);
					return;
				}
			}
		}
	}

	//If we haven't found a selection
	if (DataEntries.Num())
	{
		DataEntriesTree->SetSelection(DataEntries[0]);
	}
	else
	{
		DataEntriesTree->SetSelection(nullptr);
	}
}

EVisibility SLensDataViewer::GetCurveAxisButtonVisibility() const
{
	TSharedPtr<FLensDataCategoryItem> CategoryItem = GetDataCategorySelection();
	if (CategoryItem.IsValid())
	{
		if (CategoryItem->Category != ELensDataCategory::Focus && CategoryItem->Category != ELensDataCategory::Iris)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE /* LensDataViewer */


