// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CameraCalibrationEditorCommon.h"
#include "EditorUndoClient.h"
#include "LensFile.h"
#include "SLensFilePanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STreeView.h"
#include "UObject/StrongObjectPtr.h"

class FLensDataListItem;
class FLensDataCategoryItem;
class FCameraCalibrationCurveEditor;
class SCameraCalibrationCurveEditorPanel;
class FCameraCalibrationStepsController;
class FCameraCalibrationTimeSliderController;
enum class ELensDataChangedReason : uint8;
enum class ELensCurveAxis : uint8;

/** Widget used to display data from the LensFile */
class SLensDataViewer : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SLensDataViewer)
	{}

		/** FIZ data */
		SLATE_ATTRIBUTE(FCachedFIZData, CachedFIZData)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ULensFile* InLensFile, const TSharedRef<FCameraCalibrationStepsController>& InCalibrationStepsController);

	/** Get the currently selected data category. */
	TSharedPtr<FLensDataCategoryItem> GetDataCategorySelection() const;

	/** Generates one data category row */
	TSharedRef<ITableRow> OnGenerateDataCategoryRow(TSharedPtr<FLensDataCategoryItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Used to get the children of a data category item */
	void OnGetDataCategoryItemChildren(TSharedPtr<FLensDataCategoryItem> Item, TArray<TSharedPtr<FLensDataCategoryItem>>& OutChildren);

	/** Triggered when data category selection has changed */
	void OnDataCategorySelectionChanged(TSharedPtr<FLensDataCategoryItem> Item, ESelectInfo::Type SelectInfo);

	/** Get the currently selected data entry */
	TSharedPtr<FLensDataListItem> GetSelectedDataEntry() const;

	/** Generates one data entry row */
	TSharedRef<ITableRow> OnGenerateDataEntryRow(TSharedPtr<FLensDataListItem> Node, const TSharedRef<STableViewBase>& OwnerTable);

	/** Used to get the children of a data entry item */
	void OnGetDataEntryChildren(TSharedPtr<FLensDataListItem> Node, TArray<TSharedPtr<FLensDataListItem>>& OutNodes);

	/** Triggered when data entry selection has changed */
	void OnDataEntrySelectionChanged(TSharedPtr<FLensDataListItem> Node, ESelectInfo::Type SelectInfo);

	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

private:

	/** Makes the widget showing lens data and curve editor */
	TSharedRef<SWidget> MakeLensDataWidget();

	/** Makes the Toolbar with data manipulation buttons */
	TSharedRef<SWidget> MakeToolbarWidget(TSharedRef<SCameraCalibrationCurveEditorPanel> InEditorPanel);

	/** Makes the toolbar for the curve editor, which sits on top of the curve editor */
	TSharedRef<SWidget> MakeCurveEditorToolbarWidget();
	
	/** Curve Editor add point delegate handler*/
	void OnAddDataPointHandler();
	
	/** Called when user clicks on Clear LensFile button */
	FReply OnClearLensFileClicked();

	/** Called when DataMode is changed */
	void OnDataModeChanged();

	/** Refreshes the widget's trees */
	void Refresh();

	/** Refreshes data categories tree */
	void RefreshDataCategoriesTree();

	/** Refreshes data entries tree */
	void RefreshDataEntriesTree();

	/** Refreshes curve editor */
	void RefreshCurve() const;

	/** Refreshes time slider controller */
	void RefreshTimeSlider() const;

	/** Callbacked when user clicks AddPoint from the dialog */
	void OnLensDataPointAdded();

	/** Callbacked when user changes anything about the data point that may require a refresh */
	void OnDataPointChanged(ELensDataChangedReason ChangedReason, float InFocus, TOptional<float> InZoom);
	
	/** Called when the data table points list of a data category was updated */
	void OnDataTablePointsUpdated(ELensDataCategory InCategory);

	/** Used to keep same selected data point when rebuilding tree */
	void UpdateDataSelection(const TSharedPtr<FLensDataListItem>& PreviousSelection);

	/** Gets whether the axis selection buttons should be visible in the curve editor toolbar */
	EVisibility GetCurveAxisButtonVisibility() const;

private:
	
	/** Data category TreeView */
	TSharedPtr<STreeView<TSharedPtr<FLensDataCategoryItem>>> TreeView;

	/** List of data category items */
	TArray<TSharedPtr<FLensDataCategoryItem>> DataCategories;

	/**
	 * Cached selected item to detect category change
	 * If category hasn't changed, don't recreate data entries
	 */
	TSharedPtr<FLensDataCategoryItem> CachedSelectedCategoryItem;
	
	/** Data items associated with selected data category TreeView */
	TSharedPtr<STreeView<TSharedPtr<FLensDataListItem>>> DataEntriesTree;

	/** List of data items for the selected data category */
	TArray<TSharedPtr<FLensDataListItem>> DataEntries;

	/** Data entries title */
	TSharedPtr<STextBlock> DataEntryNameWidget;

	/** LensFile being edited */
	TStrongObjectPtr<ULensFile> LensFile;

	/** Curve editor manager and panel to display */
	TSharedPtr<FCameraCalibrationCurveEditor> CurveEditor;

	/** Child class of curve editor panel */
	TSharedPtr<SCameraCalibrationCurveEditorPanel> CurvePanel;

	/** The current axis type to display in the curve editor */
	ELensCurveAxis CurveAxisType = (ELensCurveAxis)0;
	
	/** Evaluated FIZ for the current frame */
	TAttribute<FCachedFIZData> CachedFIZ;

	/** Weak reference to Time Slider Controller */
	TWeakPtr<FCameraCalibrationTimeSliderController> TimeSliderControllerWeakPtr;
};