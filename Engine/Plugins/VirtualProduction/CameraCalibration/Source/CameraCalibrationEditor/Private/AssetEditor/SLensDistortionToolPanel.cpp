// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensDistortionToolPanel.h"

#include "CalibrationPointComponent.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "ImageUtils.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "UI/SFilterableActorPicker.h"
#include "UI/SImageTexture.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SLensDistortionToolPanel"

void SCalibrationDatasetRow::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, TSharedPtr<FCalibrationRow>& InRowData)
{
	RowData = InRowData;

	FSuperRowType::FArguments StyleArguments = FSuperRowType::FArguments()
		.Padding(1.0f)
		.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"));

	SMultiColumnTableRow<TSharedPtr<FCalibrationRow>>::Construct(StyleArguments, OwnerTableView);
}

TSharedRef<SWidget> SCalibrationDatasetRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == TEXT("Index"))
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[SNew(STextBlock).Text(FText::AsNumber(RowData->Index))];
	}
	else if (ColumnName == TEXT("Image"))
	{
		// Generate a transient thumbnail texture to display in the tool
		const FImage& MediaImage = RowData->MediaImage;

		if ((MediaImage.SizeX < 1) || (MediaImage.SizeY < 1))
		{
			const FString Text = FString::Printf(TEXT("Image Unavailable"));
			return SNew(STextBlock).Text(FText::FromString(Text));
		}

		constexpr int32 ResolutionDivider = 4;
		const FIntPoint ThumbnailSize = FIntPoint(MediaImage.SizeX / ResolutionDivider, MediaImage.SizeY / ResolutionDivider);

		FImage ThumbnailImage;
		FImageCore::ResizeTo(MediaImage, ThumbnailImage, ThumbnailSize.X, ThumbnailSize.Y, MediaImage.Format, MediaImage.GetGammaSpace());

		if (UTexture2D* Thumbnail = FImageUtils::CreateTexture2DFromImage(ThumbnailImage))
		{
			return SNew(SImageTexture, Thumbnail)
				.MinDesiredHeight(4 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
				.MaxDesiredHeight(4 * FCameraCalibrationWidgetHelpers::DefaultRowHeight);
		}
		else
		{
			const FString Text = FString::Printf(TEXT("Image Unavailable"));
			return SNew(STextBlock).Text(FText::FromString(Text));
		}
	}
	else if (ColumnName == TEXT("ImagePoint"))
	{
		const FString Text = FString::Printf(TEXT("(%.2f, %.2f)"),
			RowData->ImagePoints.Points[0].X,
			RowData->ImagePoints.Points[0].Y);

		return SNew(STextBlock).Text(FText::FromString(Text));
	}
	else if (ColumnName == TEXT("ObjectPoint"))
	{
		const FString Text = FString::Printf(TEXT("(%.2f, %.2f, %.2f)"),
			RowData->ObjectPoints.Points[0].X,
			RowData->ObjectPoints.Points[0].Y,
			RowData->ObjectPoints.Points[0].Z);

		return SNew(STextBlock).Text(FText::FromString(Text));
	}

	return SNullWidget::NullWidget;
}

void FCaptureSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FLensCaptureSettings, Calibrator));

	IDetailPropertyRow* CalibratorRow = DetailBuilder.EditDefaultProperty(PropertyHandle);
	CalibratorRow->CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SFilterableActorPicker)
				.OnSetObject(this, &FCaptureSettingsCustomization::OnCalibratorSelected)
				.OnShouldFilterAsset(this, &FCaptureSettingsCustomization::DoesAssetHaveCalibrationComponent)
				.ActorAssetData(this, &FCaptureSettingsCustomization::GetCalibratorAssetData)
		];

	NextPointPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FLensCaptureSettings, NextPoint));
	IDetailPropertyRow* NextPointRow = DetailBuilder.EditDefaultProperty(NextPointPropertyHandle);
	NextPointRow->CustomWidget()
		.NameContent()
		[
			NextPointPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(STextBlock)
				.Text(this, &FCaptureSettingsCustomization::GetNextPointName)
				.Font(DetailBuilder.GetDetailFont())
		];
}

FText FCaptureSettingsCustomization::GetNextPointName() const
{
	FText Name;
	NextPointPropertyHandle->GetValue(Name);
	return Name;
}

void FCaptureSettingsCustomization::OnCalibratorSelected(const FAssetData& AssetData)
{
	if (AssetData.IsValid())
	{
		PropertyHandle->SetValue(AssetData);
	}
}

bool FCaptureSettingsCustomization::DoesAssetHaveCalibrationComponent(const FAssetData& AssetData) const
{
	if (const AActor* Actor = Cast<AActor>(AssetData.GetAsset()))
	{
		constexpr uint32 NumInlineAllocations = 32;
		TArray<UCalibrationPointComponent*, TInlineAllocator<NumInlineAllocations>> CalibrationPoints;
		Actor->GetComponents(CalibrationPoints);

		return (CalibrationPoints.Num() > 0);
	}

	return false;
}

FAssetData FCaptureSettingsCustomization::GetCalibratorAssetData() const
{
	FAssetData Tmp;
	PropertyHandle->GetValue(Tmp);
	return Tmp;
}

bool SLensDistortionToolPanel::IsSolverSettingPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const
{
	if (PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(FLensSolverSettings, bSolveNodalOffset) && Tool.IsValid())
	{
		return !(Tool->CaptureSettings.bIsCalibratorTracked && Tool->CaptureSettings.bIsCameraTracked);
	}
	return false;
}

void SLensDistortionToolPanel::Construct(const FArguments& InArgs, ULensDistortionTool* InTool, TWeakPtr<FCameraCalibrationStepsController> InStepsController)
{
	Tool = InTool;
	WeakStepsController = InStepsController;

	BuildProgressWindow();

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FStructureDetailsViewArgs DefaultStructuDetailsViewArgs;
	FDetailsViewArgs DefaultDetailsViewArgs;
	DefaultDetailsViewArgs.bAllowSearch = false;

	TSharedRef<FStructOnScope> SolverSettingsStruct = MakeShared<FStructOnScope>(FLensSolverSettings::StaticStruct(), reinterpret_cast<uint8*>(&Tool->SolverSettings));
	TSharedPtr<IStructureDetailsView> SolverSettingsDetailsView = PropertyEditor.CreateStructureDetailView(DefaultDetailsViewArgs, DefaultStructuDetailsViewArgs, SolverSettingsStruct);
	SolverSettingsDetailsView->GetDetailsView()->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SLensDistortionToolPanel::IsSolverSettingPropertyReadOnly));

	TSharedRef<FStructOnScope> CaptureSettingsStruct = MakeShared<FStructOnScope>(FLensCaptureSettings::StaticStruct(), reinterpret_cast<uint8*>(&Tool->CaptureSettings));

	TSharedPtr<IStructureDetailsView> CaptureSettingsDetailsView = PropertyEditor.CreateStructureDetailView(DefaultDetailsViewArgs, DefaultStructuDetailsViewArgs, TSharedPtr<FStructOnScope>());
	CaptureSettingsDetailsView->GetDetailsView()->OnFinishedChangingProperties().AddSP(this, &SLensDistortionToolPanel::OnCaptureSettingsChanged);

	CaptureSettingsDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyLayout(FLensCaptureSettings::StaticStruct(),
		FOnGetDetailCustomizationInstance::CreateLambda([]() { return MakeShared<FCaptureSettingsCustomization>(); }));

	CaptureSettingsDetailsView->SetStructureData(CaptureSettingsStruct);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(0.25f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				CaptureSettingsDetailsView->GetWidget().ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SolverSettingsDetailsView->GetWidget().ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(12 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
			[
				BuildDatasetListView()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0, 20)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearAll", "Clear All"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SLensDistortionToolPanel::OnClearCalibrationRowsClicked)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0, 20)
			[
				SNew(SButton)
				.Text(LOCTEXT("ImportDataset", "Import Dataset"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SLensDistortionToolPanel::OnImportDatasetClicked)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0, 20)
			[
				SNew(SButton)
				.Text(LOCTEXT("CalibrateLens", "Calibrate Lens"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SLensDistortionToolPanel::OnCalibrateClicked)
			]
		]
	];
}

void SLensDistortionToolPanel::OpenProgressWindow()
{
	ProgressTextWidget->SetText(LOCTEXT("CalibrationProgressText", "Calibrating..."));

	// The okay button will be disabled until the calibration is complete. 
	OkayButton->SetEnabled(false);
	ProgressWindow->ShowWindow();
}

void SLensDistortionToolPanel::MarkProgressFinished()
{
	OkayButton->SetEnabled(true);
}

void SLensDistortionToolPanel::UpdateProgressText(const FText& ProgressText)
{
	ProgressTextWidget->SetText(ProgressText);
}

void SLensDistortionToolPanel::Shutdown()
{
	ProgressWindow->HideWindow();
}

void SLensDistortionToolPanel::OnCaptureSettingsChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!Tool.IsValid())
	{
		return;
	}

	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensCaptureSettings, bShowOverlay))
	{
		if (TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin())
		{
			StepsController->SetOverlayEnabled(Tool->CaptureSettings.bShowOverlay);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensCaptureSettings, Calibrator))
	{
		Tool->SetCalibrator(Tool->CaptureSettings.Calibrator.Get());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensCaptureSettings, CalibrationPattern))
	{
		RefreshListView();
	}
}

FReply SLensDistortionToolPanel::OnClearCalibrationRowsClicked()
{
	if (Tool.IsValid())
	{
		Tool->ClearCalibrationRows();
	}
	return FReply::Handled();
}

FReply SLensDistortionToolPanel::OnImportDatasetClicked()
{
	if (Tool.IsValid())
	{
		Tool->ImportCalibrationDataset();
	}
	return FReply::Handled();
}

FReply SLensDistortionToolPanel::OnCalibrateClicked()
{
	if (Tool.IsValid())
	{
		Tool->CalibrateLens();
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SLensDistortionToolPanel::BuildDatasetListView()
{
	if (!Tool.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TArray<FName> HiddenColumns = { "ImagePoint", "ObjectPoint" };

	DatasetListHeader = SNew(SHeaderRow)
		.HiddenColumnsList(HiddenColumns)

		+ SHeaderRow::Column("Index")
		.DefaultLabel(LOCTEXT("IndexHeaderLabel", "Index"))
		.FillWidth(0.2f)

		+ SHeaderRow::Column("Image")
		.DefaultLabel(LOCTEXT("ImageHeaderLabel", "Image"))
		.FillWidth(0.8f)

		+ SHeaderRow::Column("ImagePoint")
		.DefaultLabel(LOCTEXT("ImagePointHeaderLabel", "Pixel Location"))
		.FillWidth(0.4f)

		+ SHeaderRow::Column("ObjectPoint")
		.DefaultLabel(LOCTEXT("ObjectPointHeaderLabel", "World Position"))
		.FillWidth(0.4f);

	DatasetListView = SNew(SListView<TSharedPtr<FCalibrationRow>>)
		.ListItemsSource(&Tool->Dataset.CalibrationRows)
		.SelectionMode(ESelectionMode::Multi)
		.HeaderRow(DatasetListHeader.ToSharedRef())
		.OnGenerateRow(this, &SLensDistortionToolPanel::OnGenerateDatasetRow)
		.OnKeyDownHandler(this, &SLensDistortionToolPanel::OnDatasetRowKeyPressed);

	return DatasetListView.ToSharedRef();
}

TSharedRef<ITableRow> SLensDistortionToolPanel::OnGenerateDatasetRow(TSharedPtr<FCalibrationRow> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCalibrationDatasetRow, OwnerTable, InItem);
}

FReply SLensDistortionToolPanel::OnDatasetRowKeyPressed(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
{
	if (!Tool.IsValid())
	{
		return FReply::Unhandled();
	}

	if ((KeyEvent.GetKey() == EKeys::A) && KeyEvent.GetModifierKeys().IsControlDown())
	{
		// Select all items
		DatasetListView->SetItemSelection(Tool->Dataset.CalibrationRows, true);
		return FReply::Handled();
	}
	else if (KeyEvent.GetKey() == EKeys::Escape)
	{
		// De-select all items
		DatasetListView->ClearSelection();
		return FReply::Handled();
	}
	else if (KeyEvent.GetKey() == EKeys::Delete)
	{
		// Delete selected items
		const TArray<TSharedPtr<FCalibrationRow>> SelectedItems = DatasetListView->GetSelectedItems();

		for (const TSharedPtr<FCalibrationRow>& SelectedItem : SelectedItems)
		{
			Tool->Dataset.CalibrationRows.Remove(SelectedItem);
			Tool->DeleteExportedRow(SelectedItem->Index);
		}

		RefreshListView();

		Tool->RefreshCoverage();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SLensDistortionToolPanel::RefreshListView()
{
	if (DatasetListView)
	{
		DatasetListHeader->SetShowGeneratedColumn("Image", (Tool->CaptureSettings.CalibrationPattern != ECalibrationPattern::Points));
		DatasetListHeader->SetShowGeneratedColumn("ImagePoint", (Tool->CaptureSettings.CalibrationPattern == ECalibrationPattern::Points));
		DatasetListHeader->SetShowGeneratedColumn("ObjectPoint", (Tool->CaptureSettings.CalibrationPattern == ECalibrationPattern::Points));

		DatasetListView->RequestListRefresh();
	}
}

void SLensDistortionToolPanel::BuildProgressWindow()
{
	ProgressWindow = SNew(SWindow)
		.Title(LOCTEXT("ProgressWindowTitle", "Distortion Calibration Progress"))
		.SizingRule(ESizingRule::Autosized)
		.IsTopmostWindow(true)
		.HasCloseButton(false)
		.SupportsMaximize(false)
		.SupportsMinimize(true);

	ProgressTextWidget = SNew(STextBlock).Text(FText::GetEmpty());

	OkayButton = SNew(SButton)
		.IsEnabled(false)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Text(LOCTEXT("OkText", "Ok"))
		.OnClicked(this, &SLensDistortionToolPanel::OnOkPressed);

	TSharedRef<SWidget> WindowContent = SNew(SVerticalBox)

		// Text widget to display the current progress of the calibration
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			ProgressTextWidget.ToSharedRef()
		]

		// Ok and Cancel buttons
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				OkayButton.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("CancelText", "Cancel"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &SLensDistortionToolPanel::OnCancelPressed)
			]
		];

	ProgressWindow->SetContent(WindowContent);

	// Create the window, but start with it hidden. When the user initiates a calibration, the progress window will be shown.
	FSlateApplication::Get().AddWindow(ProgressWindow.ToSharedRef());
	ProgressWindow->HideWindow();
}

FReply SLensDistortionToolPanel::OnCancelPressed()
{
	if (Tool.IsValid())
	{
		Tool->CancelCalibration();
		Tool->CalibrationTask = {};
	}

	ProgressWindow->HideWindow();
	SetEnabled(true);

	return FReply::Handled();
}

FReply SLensDistortionToolPanel::OnOkPressed()
{
	if (Tool.IsValid())
	{
		Tool->SaveCalibrationResult();
	}

	ProgressWindow->HideWindow();
	SetEnabled(true);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
