// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CameraCalibrationStepsController.h"
#include "IDetailCustomization.h"
#include "LensDistortionTool.h"
#include "PropertyHandle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ULensDistortionTool;

struct FCalibrationRow;

/** Customization for the FCaptureSettings struct */
class FCaptureSettingsCustomization : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> NextPointPropertyHandle;

	FText GetNextPointName() const;

	void OnCalibratorSelected(const FAssetData& AssetData);

	bool DoesAssetHaveCalibrationComponent(const FAssetData& AssetData) const;

	FAssetData GetCalibratorAssetData() const;
};

/** Row widget for the calibration row list view */
class SCalibrationDatasetRow : public SMultiColumnTableRow<TSharedPtr<FCalibrationRow>>
{
public:
	SLATE_BEGIN_ARGS(SCalibrationDatasetRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, TSharedPtr<FCalibrationRow>& InRowData);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<FCalibrationRow> RowData;
};

/** UI for the Lens Distortion step */
class SLensDistortionToolPanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLensDistortionToolPanel) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, ULensDistortionTool* InTool, TWeakPtr<FCameraCalibrationStepsController> InStepsController);

	/** Refresh the dataset list view */
	void RefreshListView();

	/** Hides the modal progress window before the rest of the UI is destroyed */
	void Shutdown();

	/** Shows the modal progress window */
	void OpenProgressWindow();

	/** Enables the OkayButton in the progress window */
	void MarkProgressFinished();

	void UpdateProgressText(const FText& ProgressText);

private:
	/** Callback to handle changes to the customized capture settings */
	void OnCaptureSettingsChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	bool IsSolverSettingPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const;

	TSharedRef<SWidget> BuildDatasetListView();
	TSharedRef<ITableRow> OnGenerateDatasetRow(TSharedPtr<FCalibrationRow> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	FReply OnDatasetRowKeyPressed(const FGeometry& Geometry, const FKeyEvent& KeyEvent);

	FReply OnClearCalibrationRowsClicked();
	FReply OnImportDatasetClicked();
	FReply OnCalibrateClicked();

	void BuildProgressWindow();
	FReply OnOkPressed();
	FReply OnCancelPressed();
	
private:
	TWeakObjectPtr<ULensDistortionTool> Tool;
	TWeakPtr<FCameraCalibrationStepsController> WeakStepsController;

	TSharedPtr<SListView<TSharedPtr<FCalibrationRow>>> DatasetListView;
	TSharedPtr<SHeaderRow> DatasetListHeader;

	TSharedPtr<SWindow> ProgressWindow;
	TSharedPtr<STextBlock> ProgressTextWidget;
	TSharedPtr<SButton> OkayButton;
};
