// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensEvaluation.h"

#include "CameraCalibrationStepsController.h"
#include "EngineUtils.h"
#include "Features/IModularFeatures.h"
#include "LensComponent.h"
#include "LensFile.h"
#include "UI/CameraCalibrationEditorStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LensEvaluation"

void SLensEvaluation::Construct(const FArguments& InArgs, TWeakPtr<FCameraCalibrationStepsController> StepsController, ULensFile* InLensFile)
{
	LensFile = TStrongObjectPtr<ULensFile>(InLensFile);

	WeakStepsController = StepsController;

	if (InLensFile)
	{
		MakeDistortionWidget(InLensFile->LensInfo.LensModel);
		InLensFile->OnLensFileModelChanged().AddSP(this, &SLensEvaluation::MakeDistortionWidget);
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(4.f, 4.f, 4.f, 4.f))
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SNew(SHorizontalBox)

				//Tracking section
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f)
				.FillWidth(0.2f)
				[
					MakeTrackingWidget()
				]
				//Raw Input FIZ section
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f)
				.FillWidth(0.2f)
				[
					MakeRawInputFIZWidget()
				]
				//Evaluated FIZ section
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f)
				.FillWidth(0.2f)
				[
					MakeEvaluatedFIZWidget()
				]
				//Distortion section
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f)
				.FillWidth(0.2f)
				[
					DistortionWidget.ToSharedRef()
				]
				//Image Center section
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f)
				.FillWidth(0.2f)
				[
					MakeIntrinsicsWidget()
				]
				//Nodal offset section
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f)
				.FillWidth(0.2f)
				[
					MakeNodalOffsetWidget()	
				]
			]
		]
	];
}

SLensEvaluation::~SLensEvaluation()
{
	if (ULensFile* LensFilePtr = LensFile.Get())
	{
		LensFilePtr->OnLensFileModelChanged().RemoveAll(this);
	}
}

void SLensEvaluation::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	//Cache LensFile evaluation data every tick to be sure we have the right one for the frame during calibration
	CacheLensFileEvaluationInputs();
	CacheLensFileData();

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SLensEvaluation::CacheLensFileEvaluationInputs()
{
	//Start clean
	CachedFIZData.RawFocus.Reset();
	CachedFIZData.RawIris.Reset();
	CachedFIZData.RawZoom.Reset();
	CachedFIZData.EvaluatedFocus.Reset();
	CachedFIZData.EvaluatedIris.Reset();
	CachedFIZData.EvaluatedZoom.Reset();

	if (TSharedPtr<FCameraCalibrationStepsController> StepsController = WeakStepsController.Pin())
	{
		const FLensFileEvaluationInputs EvalInputs = StepsController->GetLensFileEvaluationInputs();
		if (!EvalInputs.bIsValid)
		{
			return;
		}

		CachedFIZData.RawFocus = EvalInputs.Focus;
		if (LensFile->HasFocusEncoderMapping())
		{
			CachedFIZData.EvaluatedFocus = LensFile->EvaluateNormalizedFocus(EvalInputs.Focus);
		}

		CachedFIZData.RawIris = EvalInputs.Iris;
		if (LensFile->HasIrisEncoderMapping())
		{
			CachedFIZData.EvaluatedIris = LensFile->EvaluateNormalizedIris(EvalInputs.Iris);
		}

		CachedFIZData.RawZoom = EvalInputs.Zoom;
		FFocalLengthInfo FocalLength;
		if (LensFile->EvaluateFocalLength(EvalInputs.Focus, EvalInputs.Zoom, FocalLength))
		{
			CachedFIZData.EvaluatedZoom = FocalLength.FxFy.X * LensFile->LensInfo.SensorDimensions.X * LensFile->LensInfo.SqueezeFactor;
		}
	}
}

void SLensEvaluation::CacheLensFileData()
{
	//Evaluate LensFile independently of valid FZ pair. Use default 0.0f if it's not present
	{
		const float Focus = CachedFIZData.RawFocus.IsSet() ? CachedFIZData.RawFocus.GetValue() : 0.0f;
		const float Zoom = CachedFIZData.RawZoom.IsSet() ? CachedFIZData.RawZoom.GetValue() : 0.0f;
		bCouldEvaluateDistortion = LensFile->EvaluateDistortionParameters(Focus, Zoom, CachedDistortionInfo);
		bCouldEvaluateFocalLength = LensFile->EvaluateFocalLength(Focus, Zoom, CachedFocalLengthInfo);
		bCouldEvaluateImageCenter = LensFile->EvaluateImageCenterParameters(Focus, Zoom, CachedImageCenter);
		bCouldEvaluateNodalOffset = LensFile->EvaluateNodalPointOffset(Focus, Zoom, CachedNodalOffset);
	}
}

TSharedRef<SWidget> SLensEvaluation::MakeTrackingWidget()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SGridPanel)
			.FillColumn(0, 1.0f)

			+ SGridPanel::Slot(0, 0)
			.Padding(0.0f, 5.0f, 0.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TrackedCameraLabelSection", "Tracked Camera"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			]
			+ SGridPanel::Slot(0, 1)
			[
				SNew(STextBlock)
				.Text(this, &SLensEvaluation::GetTrackedCameraLabel)
				.ColorAndOpacity(this, &SLensEvaluation::GetTrackedCameraLabelColor)
				.AutoWrapText(true)
			]
			+ SGridPanel::Slot(0, 2)
			.Padding(0.0f, 15.0f, 0.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LensComponentLabelSection", "Lens Component"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			]
			+ SGridPanel::Slot(0, 3)
			[
				SNew(STextBlock)
				.Text(this, &SLensEvaluation::GetLensComponentLabel)
				.ColorAndOpacity(this, &SLensEvaluation::GetLensComponentLabelColor)
				.AutoWrapText(true)
			]
		];
}

FText SLensEvaluation::GetTrackedCameraLabel() const
{
	if (WeakStepsController.IsValid())
	{
		if (ACameraActor* SelectedCamera = WeakStepsController.Pin()->GetCamera())
		{
			return FText::FromString(SelectedCamera->GetActorLabel());
		}
	}
	return LOCTEXT("NoTrackedCameraPresent", "No camera is selected. Select a CameraActor in the Calibration Steps tab.");
}

FSlateColor SLensEvaluation::GetTrackedCameraLabelColor() const
{
	if (WeakStepsController.IsValid())
	{
		if (ACameraActor* SelectedCamera = WeakStepsController.Pin()->GetCamera())
		{
			return FLinearColor::White;
		}
	}
	return FLinearColor::Yellow;
}


FText SLensEvaluation::GetLensComponentLabel() const
{
	if (WeakStepsController.IsValid())
	{
		if (ULensComponent* LensComponent = WeakStepsController.Pin()->FindLensComponent())
		{
			return FText::FromName(LensComponent->GetFName());
		}
	}
	return LOCTEXT("NoLensComponent", "The tracked camera does not have a Lens Component, or the LensFile assigned in that Lens Component does not match this one.");
}

FSlateColor SLensEvaluation::GetLensComponentLabelColor() const
{
	if (WeakStepsController.IsValid())
	{
		if (ULensComponent* LensComponent = WeakStepsController.Pin()->FindLensComponent())
		{
			return FLinearColor::White;
		}
	}
	return FLinearColor::Yellow;
}

TSharedRef<SWidget> SLensEvaluation::MakeRawInputFIZWidget() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 5.0f)
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FIZ Section", "Raw FIZ Input"))
			.ToolTipText(LOCTEXT("FIZSectionTooltip", "The raw values for Focus/Iris/Zoom (FIZ) used to evaluate the lens file"))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SGridPanel)
			.FillColumn(0, 0.2f)
			.FillColumn(1, 0.8f)

			+ SGridPanel::Slot(0, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FocusLabel", "Focus"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(0, 1)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IrisLabel", "Iris"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(0, 2)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ZoomLabel", "Zoom"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]

			+ SGridPanel::Slot(1, 0)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this]
				{
					if (CachedFIZData.RawFocus.IsSet())
					{
						return FText::AsNumber(CachedFIZData.RawFocus.GetValue());
					}
					return LOCTEXT("NoRawFocus", "No Focus Input");
				}))
				.ColorAndOpacity(MakeAttributeLambda([this]
				{
					if (CachedFIZData.RawFocus.IsSet())
					{
						return FSlateColor(FLinearColor::White);
					}
					return FSlateColor(FLinearColor::Yellow);
				}))
			]
			+ SGridPanel::Slot(1, 1)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this]
				{
					if (CachedFIZData.RawIris.IsSet())
					{
						return FText::AsNumber(CachedFIZData.RawIris.GetValue());
					}
					return LOCTEXT("NoRawIris", "No Iris Input");
				}))
				.ColorAndOpacity(MakeAttributeLambda([this]
				{
					if (CachedFIZData.RawIris.IsSet())
					{
						return FSlateColor(FLinearColor::White);
					}
					return FSlateColor(FLinearColor::Yellow);
				}))
			]
			+ SGridPanel::Slot(1, 2)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this]
				{
					if (CachedFIZData.RawZoom.IsSet())
					{
						return FText::AsNumber(CachedFIZData.RawZoom.GetValue());
					}
					return LOCTEXT("NoRawZoom", "No Zoom Input");
				}))
				.ColorAndOpacity(MakeAttributeLambda([this]
				{
					if (CachedFIZData.RawZoom.IsSet())
					{
						return FSlateColor(FLinearColor::White);
					}
					return FSlateColor(FLinearColor::Yellow);
				}))
			]
		];
}

TSharedRef<SWidget> SLensEvaluation::MakeEvaluatedFIZWidget() const
{
	FNumberFormattingOptions FloatOptions;
	FloatOptions.MinimumFractionalDigits = 3;
	FloatOptions.MaximumFractionalDigits = 3;

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 5.0f)
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("EvaluatedFIZSection", "Evaluated Camera Settings"))
			.ToolTipText(LOCTEXT("EvaluatedFIZSectionTooltip", "Camera settings that were evaluated from the lens file using the raw input FIZ data"))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SGridPanel)
			+ SGridPanel::Slot(0, 0)
			[
				SNew(STextBlock)
				.MinDesiredWidth(100.0f)
				.Text(LOCTEXT("FocusDistanceLabel", "Focus Distance"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(0, 1)
			[
				SNew(STextBlock)
				.MinDesiredWidth(100.0f)
				.Text(LOCTEXT("ApertureLabel", "Aperture"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(0, 2)
			[
				SNew(STextBlock)
				.MinDesiredWidth(100.0f)
				.Text(LOCTEXT("FocalLengthLabel", "Focal Length"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(0, 3)
			[
				SNew(STextBlock)
				.MinDesiredWidth(100.0f)
				.Text(LOCTEXT("FOVLabel", "Horizontal FOV"))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]

			+ SGridPanel::Slot(1, 0)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (CachedFIZData.EvaluatedFocus.IsSet())
					{
						const FText ValueText = FText::AsNumber(CachedFIZData.EvaluatedFocus.GetValue(), &FloatOptions);
						return FText::Format(LOCTEXT("PhysicalUnitsFocusValue", "{0} cm"), ValueText);
					}
					else if (!LensFile->HasSamples(ELensDataCategory::Focus))
					{
						return LOCTEXT("NoFocusCurve", "No Focus Curve");
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
				.ColorAndOpacity(MakeAttributeLambda([this]
				{
					if (CachedFIZData.EvaluatedFocus.IsSet())
					{
						return FSlateColor(FLinearColor::White);
					}
					else if (!LensFile->HasSamples(ELensDataCategory::Focus))
					{
						return FSlateColor(FLinearColor::Yellow);
					}
					return FSlateColor(FLinearColor::White);
				}))
			]
			+ SGridPanel::Slot(1, 1)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this]
				{
					if (CachedFIZData.EvaluatedIris.IsSet())
					{
						FNumberFormattingOptions IrisOptions;
						IrisOptions.MinimumFractionalDigits = 1;
						IrisOptions.MaximumFractionalDigits = 1;
						const FText ValueText = FText::AsNumber(CachedFIZData.EvaluatedIris.GetValue(), &IrisOptions);
						return FText::Format(LOCTEXT("PhysicalUnitsIrisValue", "{0} F-Stop"), ValueText);
					}
					else if (!LensFile->HasSamples(ELensDataCategory::Iris))
					{
						return LOCTEXT("NoIrisCurve", "No Iris Curve");
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
				.ColorAndOpacity(MakeAttributeLambda([this]
				{
					if (CachedFIZData.EvaluatedIris.IsSet())
					{
						return FSlateColor(FLinearColor::White);
					}
					else if (!LensFile->HasSamples(ELensDataCategory::Iris))
					{
						return FSlateColor(FLinearColor::Yellow);
					}
					return FSlateColor(FLinearColor::White);
				}))
			]
			+ SGridPanel::Slot(1, 2)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (CachedFIZData.EvaluatedZoom.IsSet())
					{
						const FText ValueText = FText::AsNumber(CachedFIZData.EvaluatedZoom.GetValue(), &FloatOptions);
						return FText::Format(LOCTEXT("PhysicalUnitsZoomValue", "{0} mm"), ValueText);
					}
					else if (!LensFile->HasSamples(ELensDataCategory::Zoom))
					{
						return LOCTEXT("NoZoomCurve", "No Focal Length Curve");
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
				.ColorAndOpacity(MakeAttributeLambda([this]
				{
					if (CachedFIZData.EvaluatedZoom.IsSet())
					{
						return FSlateColor(FLinearColor::White);
					}
					else if (!LensFile->HasSamples(ELensDataCategory::Zoom))
					{
						return FSlateColor(FLinearColor::Yellow);
					}
					return FSlateColor(FLinearColor::White);
				}))
			]
			+ SGridPanel::Slot(1, 3)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (CachedFIZData.EvaluatedZoom.IsSet())
					{
						const float DesqueezedSensorWidth = LensFile->LensInfo.SensorDimensions.X * LensFile->LensInfo.SqueezeFactor;
						const float FOV = FMath::RadiansToDegrees(2.f * FMath::Atan(DesqueezedSensorWidth / (2.f * CachedFIZData.EvaluatedZoom.GetValue())));
						const FText ValueText = FText::AsNumber(FOV, &FloatOptions);
						return FText::Format(LOCTEXT("PhysicalUnitsFOVValue", "{0} deg"), ValueText);
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
			]
		];
}

void SLensEvaluation::MakeDistortionWidget(const TSubclassOf<ULensModel>& LensModel)
{
	if (!DistortionWidget)
	{
		DistortionWidget = SNew(SVerticalBox);
	}

	DistortionWidget->ClearChildren();

	//Find the named distortion parameters the current model has
	TArray<FText> Parameters;
	if (LensFile->LensInfo.LensModel)
	{
		Parameters = LensFile->LensInfo.LensModel.GetDefaultObject()->GetParameterDisplayNames();
	}

	const TSharedRef<SWidget> Title = SNew(STextBlock)
		.Text(LOCTEXT("DistortionSection", "Distortion Parameters"))
		.ToolTipText(LOCTEXT("DistortionSectionTooltip", "Coefficients associated with the distortion equation of the selected lens model"))
		.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
		.ShadowOffset(FVector2D(1.0f, 1.0f));

	DistortionWidget->AddSlot()
		.Padding(0.0f, 5.0f)
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			Title
		]
	;

	//if there are no parameters, create a simpler widget
	if (Parameters.Num() <= 0)
	{
		DistortionWidget->AddSlot()
			.Padding(0.0f, 5.0f)
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoParameters", "No parameters"))
			];
	}

	TSharedRef<SGridPanel> ParameterGrid = SNew(SGridPanel);

	for (int32 Index = 0; Index < Parameters.Num(); ++Index)
	{
		ParameterGrid->AddSlot(0, Index)
			[
				SNew(STextBlock)
				.MinDesiredWidth(35.0f)
				.Text(MakeAttributeLambda([Parameters, Index]
				{
					if (Parameters.IsValidIndex(Index))
					{
						return Parameters[Index];
					}
					else
					{
						return LOCTEXT("InvalidParam", "Invalid");
					}
				}))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			];

		ParameterGrid->AddSlot(1, Index)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this, Index]
					{
						if (bCouldEvaluateDistortion && CachedDistortionInfo.Parameters.IsValidIndex(Index))
						{
							return FText::AsNumber(CachedDistortionInfo.Parameters[Index]);
						}
						return LOCTEXT("UndefinedValue", "N/A");
					}))
			];
	}

	DistortionWidget->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			ParameterGrid
		];
}

TSharedRef<SWidget> SLensEvaluation::MakeIntrinsicsWidget() const
{
	FNumberFormattingOptions FloatOptions;
	FloatOptions.MinimumFractionalDigits = 2;
	FloatOptions.MaximumFractionalDigits = 2;

	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 5.0f)
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("IntrinsicsSection", "Normalized Camera Intrinsics"))
			.ToolTipText(LOCTEXT("IntrinsicsSectionTooltip", "Normalized values from the camera intrinsic matrix"))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(SGridPanel)

			+ SGridPanel::Slot(0, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ImageCenterLabel", "Image Center"))
				.ToolTipText(LOCTEXT("ImageCenterTooltip", "Normalized Center in the range [0, 1], with (0, 0) representing the top left corner of the image"))
				.MinDesiredWidth(100.0f)
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(1, 0)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (bCouldEvaluateImageCenter)
					{
						const FText CxText = FText::AsNumber(CachedImageCenter.PrincipalPoint.X, &FloatOptions);
						const FText CyText = FText::AsNumber(CachedImageCenter.PrincipalPoint.Y, &FloatOptions);
						return FText::Format(LOCTEXT("PrincipalPointValue", "({0}, {1})"), CxText, CyText);
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
			]
			+ SGridPanel::Slot(0, 1)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FxFyLabel", "FxFy"))
				.ToolTipText(LOCTEXT("FxFyTooltip", "Normalized values representing the camera focal length divided by the sensor/image size. The ratio of Fx to Fy should roughly equal the camera's aspect ratio"))
				.MinDesiredWidth(100.0f)
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(1, 1)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (bCouldEvaluateFocalLength)
					{
						const FText FxText = FText::AsNumber(CachedFocalLengthInfo.FxFy.X, &FloatOptions);
						const FText FyText = FText::AsNumber(CachedFocalLengthInfo.FxFy.Y, &FloatOptions);
						return FText::Format(LOCTEXT("FxFyValue", "({0}, {1})"), FxText, FyText);
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
			]
		];
}

TSharedRef<SWidget> SLensEvaluation::MakeNodalOffsetWidget() const
{
	FNumberFormattingOptions FloatOptions;
	FloatOptions.MinimumFractionalDigits = 2;
	FloatOptions.MaximumFractionalDigits = 2;

	const FRotator CachedRotationOffset = CachedNodalOffset.RotationOffset.Rotator();

	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 5.0f)
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NodalOffsetSection", "Nodal Point Offset"))
			.ToolTipText(LOCTEXT("NodalOffsetTooltip", "The offset required to go from the tracked camera transform to the nodal point of the physical lens"))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(SGridPanel)

			+ SGridPanel::Slot(0, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LocationOffsetLabel", "Location"))
				.MinDesiredWidth(75.0f)
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(1, 0)
			[
				SNew(STextBlock)
				.MinDesiredWidth(15.0f)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (bCouldEvaluateNodalOffset)
					{
						const FText LocXText = FText::AsNumber(CachedNodalOffset.LocationOffset.X, &FloatOptions);
						const FText LocYText = FText::AsNumber(CachedNodalOffset.LocationOffset.Y, &FloatOptions);
						const FText LocZText = FText::AsNumber(CachedNodalOffset.LocationOffset.Z, &FloatOptions);
						return FText::Format(LOCTEXT("LocationOffsetValue", "({0}, {1}, {2})"), LocXText, LocYText, LocZText);
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
			]

			+ SGridPanel::Slot(0, 1)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RotationOffsetLabel", "Rotation"))
				.MinDesiredWidth(75.0f)
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]
			+ SGridPanel::Slot(1, 1)
			[
				SNew(STextBlock)
				.MinDesiredWidth(15.0f)
				.Text(MakeAttributeLambda([this, FloatOptions]
				{
					if (bCouldEvaluateNodalOffset)
					{
						const FText RotXText = FText::AsNumber(CachedNodalOffset.RotationOffset.Rotator().GetComponentForAxis(EAxis::X), &FloatOptions);
						const FText RotYText = FText::AsNumber(CachedNodalOffset.RotationOffset.Rotator().GetComponentForAxis(EAxis::Y), &FloatOptions);
						const FText RotZText = FText::AsNumber(CachedNodalOffset.RotationOffset.Rotator().GetComponentForAxis(EAxis::Z), &FloatOptions);
						return FText::Format(LOCTEXT("RotationOffsetValue", "({0}, {1}, {2})"), RotXText, RotYText, RotZText);
					}
					return LOCTEXT("UndefinedValue", "N/A");
				}))
			]
		];
}

#undef LOCTEXT_NAMESPACE /* LensDataViewer */