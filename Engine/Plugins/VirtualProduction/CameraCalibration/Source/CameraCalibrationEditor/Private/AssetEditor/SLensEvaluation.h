// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Camera/CameraActor.h"
#include "SLensFilePanel.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class FCameraCalibrationStepsController;
class SVerticalBox;
class ULensFile;
class ULensModel;

/**
 * Widget using lens file evaluation inputs to evaluate lens file and show resulting data
 */
class SLensEvaluation : public SCompoundWidget
{
	using Super = SCompoundWidget;
public:
	SLATE_BEGIN_ARGS(SLensEvaluation)
	{}
	SLATE_END_ARGS()

	~SLensEvaluation();

	void Construct(const FArguments& InArgs, TWeakPtr<FCameraCalibrationStepsController> StepsController, ULensFile* InLensFile);

	//~ Begin SCompoundWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SCompoundWidget interface

	/** Returns last raw and evaluated FIZ data */
	FCachedFIZData GetLastEvaluatedData() const { return CachedFIZData; }

private:

	/** Caches latest evaluation inputs and evaluates LensFile for FIZ */
	void CacheLensFileEvaluationInputs();

	/** Evaluates LensFile using tracking data */
	void CacheLensFileData();

	/** Get the name of the selected camera actor */
	FText GetTrackedCameraLabel() const;

	/** Get the text color for the tracked camera text */
	FSlateColor GetTrackedCameraLabelColor() const;

	/** Get the name of the lens component driving the selected camera */
	FText GetLensComponentLabel() const;

	/** Get the text color for the lens component text */
	FSlateColor GetLensComponentLabelColor() const;

	/** Make widget to display selected camera and lens component */
	TSharedRef<SWidget> MakeTrackingWidget();

	/** Make Raw FIZ widget */
	TSharedRef<SWidget> MakeRawInputFIZWidget() const;

	/** Make Evaluated FIZ widget */
	TSharedRef<SWidget> MakeEvaluatedFIZWidget() const;

	/** Make widget showing distortion data evaluated from LensFile */
	void MakeDistortionWidget(const TSubclassOf<ULensModel>& LensModel);

	/** Make widget showing Intrinsic data evaluated from LensFile */
	TSharedRef<SWidget> MakeIntrinsicsWidget() const;

	/** Make widget showing NodalOffset data evaluated from LensFile */
	TSharedRef<SWidget> MakeNodalOffsetWidget() const;

private:

	/** LensFile being edited */
	TStrongObjectPtr<ULensFile> LensFile;

	/** Flags used to know what data is valid or not and adjust UI */
	bool bCouldEvaluateDistortion = false;
	bool bCouldEvaluateFocalLength = false;
	bool bCouldEvaluateImageCenter = false;
	bool bCouldEvaluateNodalOffset = false;

	/** Calibration steps controller */
	TWeakPtr<FCameraCalibrationStepsController> WeakStepsController;

	/** Cached LensFile evaluated data */
	FCachedFIZData CachedFIZData;
	FDistortionInfo CachedDistortionInfo;
	FFocalLengthInfo CachedFocalLengthInfo;
	FImageCenterInfo CachedImageCenter;
	FNodalPointOffset CachedNodalOffset;

	/** Distortion Widget that displays the evaluated distortion parameters */
	TSharedPtr<SVerticalBox> DistortionWidget;
};
