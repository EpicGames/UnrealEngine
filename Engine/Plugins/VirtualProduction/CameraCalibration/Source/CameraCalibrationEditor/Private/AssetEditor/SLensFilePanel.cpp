// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLensFilePanel.h"

#include "IStructureDetailsView.h"
#include "LensFile.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SLensDataViewer.h"
#include "UObject/StructOnScope.h"


void SLensFilePanel::Construct(const FArguments& InArgs, ULensFile* InLensFile, const TSharedRef<FCameraCalibrationStepsController>& InCalibrationStepsController)
{
	LensFile = TStrongObjectPtr<ULensFile>(InLensFile);
	CachedFIZ = InArgs._CachedFIZData;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		[
			SNew(SLensDataViewer, InLensFile, InCalibrationStepsController)
			.CachedFIZData(InArgs._CachedFIZData)
		]
	];
}
