// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CameraCalibrationEditorCommon.h"
#include "UObject/StrongObjectPtr.h"

class ULensFile;
class FCameraCalibrationStepsController;


/**
 * Main panel of lens file editor
 */
class SLensFilePanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLensFilePanel) {}
	
		/** FIZ data */
		SLATE_ATTRIBUTE(FCachedFIZData, CachedFIZData)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, ULensFile* InLensFile, const TSharedRef<FCameraCalibrationStepsController>& InCalibrationStepsController);

private:

	/** Holds the preset asset. */
	TStrongObjectPtr<ULensFile> LensFile;

	/** Evaluated FIZ for the current frame */
	TAttribute<FCachedFIZData> CachedFIZ;
};
