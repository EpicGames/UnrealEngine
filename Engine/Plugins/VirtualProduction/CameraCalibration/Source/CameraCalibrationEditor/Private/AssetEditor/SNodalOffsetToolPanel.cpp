// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNodalOffsetToolPanel.h"

#include "AssetRegistry/AssetData.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationSubsystem.h"
#include "CameraNodalOffsetAlgo.h"
#include "Dialog/SCustomDialog.h"
#include "EditorFontGlyphs.h"
#include "Engine/Engine.h"
#include "Styling/AppStyle.h"
#include "Engine/Selection.h"
#include "LensFile.h"
#include "NodalOffsetTool.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STreeView.h"


#define LOCTEXT_NAMESPACE "NodalOffsetTool"


void SNodalOffsetToolPanel::Construct(const FArguments& InArgs, UNodalOffsetTool* InNodalOffsetTool)
{
	NodalOffsetTool = InNodalOffsetTool;

	// This will be the widget wrapper of the custom algo UI.
	NodalOffsetUI = SNew(SVerticalBox);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot() // Right toolbar
		.FillWidth(0.25f)
		[ 
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Algo picker
			.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
			[ FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("NodalOffsetAlgo", "Nodal Offset Algo"), BuildNodalOffsetAlgoPickerWidget())]

			+ SVerticalBox::Slot() // Algo UI
			.AutoHeight()
			[ BuildNodalOffsetUIWrapper() ]

			+ SVerticalBox::Slot() // Import dataset
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0, 20)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton).Text(LOCTEXT("ImportCalibrationDataset", "Import Nodal Offset Dataset"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([this]() -> FReply
					{
						if (UNodalOffsetTool* Tool = NodalOffsetTool.Get())
						{
							Tool->ImportCalibrationDataset();

							// After importing, the tool may have switched the active algo, so redraw the UI accordingly
							UCameraNodalOffsetAlgo* Algo = Tool->GetNodalOffsetAlgo();
							NodalOffsetUI->ClearChildren();
							NodalOffsetUI->AddSlot()[Algo->BuildUI()];

							for (const TSharedPtr<FString>& AlgoString : CurrentAlgos)
							{
								if (AlgoString->Equals(Algo->FriendlyName().ToString()))
								{
									AlgosComboBox->SetSelectedItem(AlgoString);
									break;
								}
							}
						}
						return FReply::Handled();
					})
					.Visibility_Lambda([]() -> EVisibility
					{
						if (GetDefault<UCameraCalibrationSettings>()->IsCalibrationDatasetImportExportEnabled())
						{
							return EVisibility::Visible;
						}
						return EVisibility::Collapsed;
					})
				]
			]

			+ SVerticalBox::Slot() // Save Offset
			.AutoHeight()
			.Padding(0, 20)
			[
				SNew(SButton).Text(LOCTEXT("AddToNodalOffsetLUT", "Add To Nodal Offset Calibration"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([WeakTool = NodalOffsetTool]() -> FReply
				{
					if (UNodalOffsetTool* Tool = WeakTool.Get())
					{
						Tool->OnSaveCurrentNodalOffset();
					}
					return FReply::Handled();
				})
			]
		]
	];
}

TSharedRef<SWidget> SNodalOffsetToolPanel::BuildNodalOffsetUIWrapper()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Algo's Widget
		.AutoHeight()
		[ NodalOffsetUI.ToSharedRef() ];
}

void SNodalOffsetToolPanel::UpdateNodalOffsetUI()
{
	check(AlgosComboBox.IsValid());

	// Get current algo to later compare with new one
	const UCameraNodalOffsetAlgo* OldAlgo = NodalOffsetTool->GetNodalOffsetAlgo();

	// Set new algo by name
	const FName AlgoName(*AlgosComboBox->GetSelectedItem());
	NodalOffsetTool->SetNodalOffsetAlgo(AlgoName);

	// Get the new algo
	UCameraNodalOffsetAlgo* Algo = NodalOffsetTool->GetNodalOffsetAlgo();

	// nullptr may indicate that it was unregistered, so refresh combobox options.
	if (!Algo)
	{
		UpdateAlgosOptions();
		return;
	}

	// If we didn't change the algo, we're done here.
	if (Algo == OldAlgo)
	{
		return;
	}

	// Remove old UI
	check(NodalOffsetUI.IsValid());
	NodalOffsetUI->ClearChildren();

	// Assign GUI
	NodalOffsetUI->AddSlot() [Algo->BuildUI()];
}

void SNodalOffsetToolPanel::UpdateAlgosOptions()
{
	CurrentAlgos.Empty();

	// Ask the subsystem for the list of registered Algos

	UCameraCalibrationSubsystem* Subsystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	check(Subsystem);

	for (FName& Name : Subsystem->GetCameraNodalOffsetAlgos())
	{
		CurrentAlgos.Add(MakeShared<FString>(Name.ToString()));
	}

	// Ask the ComboBox to refresh its options from its source (that we just updated)
	AlgosComboBox->RefreshOptions();
}

TSharedRef<SWidget> SNodalOffsetToolPanel::BuildNodalOffsetAlgoPickerWidget()
{
	// Create ComboBox widget

	AlgosComboBox = SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&CurrentAlgos)
		.OnSelectionChanged_Lambda([&](TSharedPtr<FString> NewValue, ESelectInfo::Type Type) -> void
		{
			// Replace the custom algo widget
			UpdateNodalOffsetUI();
		})
		.OnGenerateWidget_Lambda([&](TSharedPtr<FString> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock).Text(FText::FromString(*InOption));
		})
		.InitiallySelectedItem(nullptr)
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				if (AlgosComboBox.IsValid() && AlgosComboBox->GetSelectedItem().IsValid())
				{
					return FText::FromString(*AlgosComboBox->GetSelectedItem());
				}

				return LOCTEXT("InvalidComboOption", "Invalid");
			})
		];

	// Update the object holding this combobox's options source
	UpdateAlgosOptions();

	// Pick the first option by default (if available)
	if (CurrentAlgos.Num())
	{
		AlgosComboBox->SetSelectedItem(CurrentAlgos[0]);
	}
	else
	{
		AlgosComboBox->SetSelectedItem(nullptr);
	}

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot() // algo picker
		[
			AlgosComboBox.ToSharedRef()
		]

		+ SHorizontalBox::Slot() // Help button
		.AutoWidth()
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("ShowHelp_Tip", "Help about this algo"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked_Lambda([&]() -> FReply
			{
				if (!NodalOffsetTool.IsValid())
				{
					return FReply::Handled();
				}

				UCameraNodalOffsetAlgo* Algo = NodalOffsetTool->GetNodalOffsetAlgo();

				if (!Algo)
				{
					return FReply::Handled();
				}

				TSharedRef<SCustomDialog> AlgoHelpWindow =
					SNew(SCustomDialog)
					.Title(FText::FromName(NodalOffsetTool->FriendlyName()))
					.Content()
					[
						Algo->BuildHelpWidget()
					]
					.Buttons({ SCustomDialog::FButton(LOCTEXT("Ok", "Ok")) });

				AlgoHelpWindow->Show();

				return FReply::Handled();
			})
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.12"))
				.Text(FEditorFontGlyphs::Info_Circle)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
	;
}


#undef LOCTEXT_NAMESPACE
