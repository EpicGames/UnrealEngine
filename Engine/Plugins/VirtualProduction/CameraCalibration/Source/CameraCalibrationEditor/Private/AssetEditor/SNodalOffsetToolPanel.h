// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/StrongObjectPtr.h"

class FString;
class ULensFile;
class UNodalOffsetTool;

template<typename OptionType>
class SComboBox;

struct FArguments;

/**
 * UI for the nodal offset calibration.
 * It also holds the UI given by the selected nodal offset algorithm.
 */
class SNodalOffsetToolPanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNodalOffsetToolPanel) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UNodalOffsetTool* InNodalOffsetTool);

private:

	/** Builds the wrapper for the currently selected nodal offset UI */
	TSharedRef<SWidget> BuildNodalOffsetUIWrapper();

	/** Builds the UI for the nodal offset algorithm picker */
	TSharedRef<SWidget> BuildNodalOffsetAlgoPickerWidget();

	/** Updates the UI so that it matches the selected nodal offset algorithm (if necessary) */
	void UpdateNodalOffsetUI();

	/** Refreshes the list of available nodal offset algorithms shown in the AlgosComboBox */
	void UpdateAlgosOptions();

private:

	/** The nodal offset tool controller object */
	TWeakObjectPtr<class UNodalOffsetTool> NodalOffsetTool;

	/** The box containing the UI given by the selected nodal offset algorithm */
	TSharedPtr<class SVerticalBox> NodalOffsetUI;
	
private:

	/** Options source for the AlgosComboBox. Lists the currently available nodal offset algorithms */
	TArray<TSharedPtr<FString>> CurrentAlgos;

	/** The combobox that presents the available nodal offset algorithms */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> AlgosComboBox;
};
