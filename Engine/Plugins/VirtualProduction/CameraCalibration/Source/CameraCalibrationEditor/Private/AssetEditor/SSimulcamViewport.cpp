// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSimulcamViewport.h"
#include "Engine/Texture.h"
#include "Viewport/SSimulcamEditorViewport.h"


void SSimulcamViewport::Construct(const FArguments& InArgs)
{
	ViewActor = InArgs._ViewActor;
	FilmbackAspectRatio = InArgs._FilmbackAspectRatio;
	MediaOverlayTexture = InArgs._MediaOverlayTexture;
	MediaOverlayOpacity = InArgs._MediaOverlayOpacity;
	ToolOverlayMaterial = InArgs._ToolOverlayMaterial;
	UserOverlayMaterial = InArgs._UserOverlayMaterial;
	OverrideTexture = TStrongObjectPtr<UTexture>(InArgs._OverrideTexture);
	OnSimulcamViewportClicked = InArgs._OnSimulcamViewportClicked;
	OnSimulcamViewportInputKey = InArgs._OnSimulcamViewportInputKey;
	OnSimulcamViewportMarqueeSelect = InArgs._OnSimulcamViewportMarqueeSelect;

	TextureViewport = SNew(SSimulcamEditorViewport, SharedThis(this), InArgs._WithZoom.Get(), InArgs._WithPan.Get());

	ChildSlot
		[
			TextureViewport.ToSharedRef()
		];
}

ACameraActor* SSimulcamViewport::GetCameraActor() const
{
	return ViewActor.Get(nullptr);
}

float SSimulcamViewport::GetCameraFeedAspectRatio() const
{
	return FilmbackAspectRatio.Get(1.0f);
}

UTexture* SSimulcamViewport::GetMediaOverlayTexture() const
{
	return MediaOverlayTexture.Get(nullptr);
}

float SSimulcamViewport::GetMediaOverlayOpacity() const
{
	return MediaOverlayOpacity.Get(1.0f);
}

UMaterialInterface* SSimulcamViewport::GetToolOverlayMaterial() const
{
	return ToolOverlayMaterial.Get(nullptr);
}

UMaterialInterface* SSimulcamViewport::GetUserOverlayMaterial() const
{
	return UserOverlayMaterial.Get(nullptr);
}

UTexture* SSimulcamViewport::GetOverrideTexture() const
{
	return OverrideTexture.Get();
}

bool SSimulcamViewport::OnViewportInputKey(const FKey& Key, const EInputEvent& Event)
{ 
	if (OnSimulcamViewportInputKey.IsBound())
	{
		return OnSimulcamViewportInputKey.Execute(Key, Event);
	}
	return false;
}
