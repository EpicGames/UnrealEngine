// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Engine/EngineBaseTypes.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"
#include "Viewport/SimulcamEditorViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UTexture;

struct FGeometry;
struct FKey;
struct FPointerEvent;

class SSimulcamEditorViewport;

/** Delegate to be executed when the viewport area is clicked */
DECLARE_DELEGATE_TwoParams(FSimulcamViewportClickedEventHandler, const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);

/** Delegate to be executed when the viewport area receives a key event */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FSimulcamViewportInputKeyEventHandler, const FKey& Key, const EInputEvent& Event);

/** Delegate to be executed when the user performs a marquee selection on the viewport area */
DECLARE_DELEGATE_TwoParams(FSimulcamViewportMarqueeSelectEventHandler, FVector2D StartPosition, FVector2D EndPosition);

/**
 * Viewport that displays a level render from the perspective of a camera actor overlaid with a media texture overlay
 */
class SSimulcamViewport : public SCompoundWidget, public FSimulcamViewportElementsProvider
{
public:
	SLATE_BEGIN_ARGS(SSimulcamViewport)
		:
		_ViewActor(nullptr),
		_FilmbackAspectRatio(1.0f),
		_MediaOverlayTexture(nullptr),
		_MediaOverlayOpacity(0.5f),
		_ToolOverlayMaterial(nullptr),
		_UserOverlayMaterial(nullptr),
		_OverrideTexture(nullptr),
		_WithZoom(true),
		_WithPan(true)
	{}
		SLATE_ATTRIBUTE(ACameraActor*, ViewActor)
		SLATE_ATTRIBUTE(float, FilmbackAspectRatio)
		SLATE_ATTRIBUTE(UTexture*, MediaOverlayTexture)
		SLATE_ATTRIBUTE(float, MediaOverlayOpacity)
		SLATE_ATTRIBUTE(UMaterialInterface*, ToolOverlayMaterial)
		SLATE_ATTRIBUTE(UMaterialInterface*, UserOverlayMaterial)
		SLATE_ARGUMENT(UTexture*, OverrideTexture)
		SLATE_EVENT(FSimulcamViewportClickedEventHandler, OnSimulcamViewportClicked)
		SLATE_EVENT(FSimulcamViewportInputKeyEventHandler, OnSimulcamViewportInputKey)
		SLATE_EVENT(FSimulcamViewportMarqueeSelectEventHandler, OnSimulcamViewportMarqueeSelect)
		SLATE_ATTRIBUTE(bool, WithZoom)
		SLATE_ATTRIBUTE(bool, WithPan)
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs);

	//~ Begin FSimulcamViewportElementsProvider interface
	virtual  ACameraActor* GetCameraActor() const override;
	virtual float GetCameraFeedAspectRatio() const override;
	virtual UTexture* GetMediaOverlayTexture() const override;
	virtual float GetMediaOverlayOpacity() const override;
	virtual UMaterialInterface* GetToolOverlayMaterial() const override;
	virtual UMaterialInterface* GetUserOverlayMaterial() const override;
	virtual UTexture* GetOverrideTexture() const override;
	virtual void OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent) override { OnSimulcamViewportClicked.ExecuteIfBound(MyGeometry, PointerEvent); }
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InInputEvent) override;
	virtual void OnMarqueeSelect(const FVector2D& InStartPosition, const FVector2D& InEndPosition) override { OnSimulcamViewportMarqueeSelect.ExecuteIfBound(InStartPosition, InEndPosition); }
	//~ End FSimulcamViewportElementsProvider interface

private:
	/** The camera actor that the scene is rendering from */
	TAttribute<ACameraActor*> ViewActor;

	/** The aspect ratio of the camera's filmback */
	TAttribute<float> FilmbackAspectRatio;

	/** The media texture to overlay onto the scene render */
	TAttribute<UTexture*> MediaOverlayTexture;

	/** The opacity of the media overlay */
	TAttribute<float> MediaOverlayOpacity;

	/** The material to use to draw the tool overlay */
	TAttribute<UMaterialInterface*> ToolOverlayMaterial;

	/** The material to use to draw the user overlay */
	TAttribute<UMaterialInterface*> UserOverlayMaterial;

	/** An override texture that is displayed instead of the scene render and media overlay */
	TStrongObjectPtr<UTexture> OverrideTexture;

	/** Delegate to be executed when the viewport area is clicked */
	FSimulcamViewportClickedEventHandler OnSimulcamViewportClicked;

	/** Delegate to be executed when the viewport receives input key presses */
	FSimulcamViewportInputKeyEventHandler OnSimulcamViewportInputKey;

	/** Delegate to be executed when the user performs a marquee selection on the viewport area */
	FSimulcamViewportMarqueeSelectEventHandler OnSimulcamViewportMarqueeSelect;

	TSharedPtr<SSimulcamEditorViewport> TextureViewport;
};

