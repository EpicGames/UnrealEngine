// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditor/Viewport/SSimulcamEditorViewport.h"

#include "AssetEditor/SSimulcamViewport.h"
#include "AssetEditor/Viewport/SimulcamEditorViewportClient.h"
#include "Engine/Texture.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SceneViewport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "STextureEditorViewport"

void SSimulcamEditorViewport::Construct(const FArguments& InArgs, const TSharedRef<SSimulcamViewport>& InSimulcamViewport, const bool bWithZoom, const bool bWithPan)
{
	bIsRenderingEnabled = true;
	SimulcamViewportWeakPtr = InSimulcamViewport;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SNew(SOverlay)
					// viewport canvas
					+ SOverlay::Slot()
					[
						SAssignNew(ViewportWidget, SViewport)
						.EnableGammaCorrection(false)
						.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
						.ShowEffectWhenDisabled(false)
						.EnableBlending(true)
						.ToolTip(SNew(SToolTip).Text(this, &SSimulcamEditorViewport::GetDisplayedResolution))
					]

					// tool bar
					+ SOverlay::Slot()
					.Padding(2.0f)
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(4.0f, 0.0f)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SSimulcamEditorViewport::HandleZoomPercentageText)
							.Visibility(bWithZoom ? EVisibility::Visible : EVisibility::Hidden)
						]
					]
				]
			]
		]

	];

	ViewportClient = MakeShared<FSimulcamEditorViewportClient>(InSimulcamViewport, bWithZoom, bWithPan);
	Viewport = FSceneViewport::Create(ViewportClient, ViewportWidget);
}

void SSimulcamEditorViewport::EnableRendering()
{
	bIsRenderingEnabled = true;
}

void SSimulcamEditorViewport::DisableRendering()
{
	bIsRenderingEnabled = false;
}

TSharedPtr<FSceneViewport> SSimulcamEditorViewport::GetViewport() const
{
	return Viewport;
}

TSharedPtr<SViewport> SSimulcamEditorViewport::GetViewportWidget() const
{
	return ViewportWidget;
}

void SSimulcamEditorViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bIsRenderingEnabled)
	{
		Viewport->Invalidate();
	}
}

FText SSimulcamEditorViewport::GetDisplayedResolution() const
{
	return ViewportClient->GetDisplayedResolution();
}

FText SSimulcamEditorViewport::HandleZoomPercentageText() const
{
	UTexture* Texture = nullptr;
	if (SimulcamViewportWeakPtr.IsValid())
	{
		Texture = SimulcamViewportWeakPtr.Pin()->GetOverrideTexture();

		if (!Texture)
		{
			Texture = SimulcamViewportWeakPtr.Pin()->GetMediaOverlayTexture();
		}
	}
	
	if (Texture)
	{
		FText FormattedText = FText::FromString(TEXT("{0}: {1}"));
		double DisplayedZoomLevel = ViewportClient->GetMediaOverlayScale();

		return FText::Format(FormattedText, FText::FromName(Texture->GetFName()), FText::AsPercent(DisplayedZoomLevel));
	}
	else
	{
		return LOCTEXT("InvalidTexture", "Invalid Texture");
	}
}

#undef LOCTEXT_NAMESPACE
