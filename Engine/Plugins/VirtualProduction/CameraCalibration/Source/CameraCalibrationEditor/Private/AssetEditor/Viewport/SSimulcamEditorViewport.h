// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "SimulcamEditorViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
class FSceneViewport;
class SViewport;
class SSimulcamViewport;
struct FPointerEvent;

/**
 * Implements the texture editor's view port.
 */
class SSimulcamEditorViewport
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimulcamEditorViewport) { }
	SLATE_END_ARGS()
public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<SSimulcamViewport>& InSimulcamViewport, const bool bWithZoom, const bool bWithPan);
	
	/** Enable viewport rendering */
	void EnableRendering();
	/** Disable viewport rendering */
	void DisableRendering();

	/** Get the current SceneViewport */
	TSharedPtr<FSceneViewport> GetViewport( ) const;

	/** Get The SViewport Widget using this editor */
	TSharedPtr<SViewport> GetViewportWidget( ) const;

	//~ Begin SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	//~ End SWidget interface

protected:
	/**
	 * Gets the displayed textures resolution as a string.
	 *
	 * @return Texture resolution string.
	 */
	FText GetDisplayedResolution() const;

private:

	// Callback for getting the zoom percentage text.
	FText HandleZoomPercentageText( ) const;

private:
	TWeakPtr<SSimulcamViewport> SimulcamViewportWeakPtr;
	
	// Level viewport client.
	TSharedPtr<FSimulcamEditorViewportClient> ViewportClient;
	// Slate viewport for rendering and IO.
	TSharedPtr<FSceneViewport> Viewport;
	// Viewport widget.
	TSharedPtr<SViewport> ViewportWidget;

	// Is rendering currently enabled? (disabled when reimporting a texture)
	bool bIsRenderingEnabled;
};
