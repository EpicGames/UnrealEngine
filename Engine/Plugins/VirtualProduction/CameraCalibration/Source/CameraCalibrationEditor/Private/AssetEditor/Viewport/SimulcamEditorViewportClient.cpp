// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditor/Viewport/SimulcamEditorViewportClient.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "CineCameraComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "ImageUtils.h"
#include "SceneView.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"
#include "Camera/CameraActor.h"
#include "Materials/MaterialInstanceDynamic.h"

FSimulcamEditorViewportClient::FSimulcamEditorViewportClient(TSharedPtr<FSimulcamViewportElementsProvider> InElementsProvider, const bool bInWithZoom, const bool bInWithPan)
	: FLevelEditorViewportClient(nullptr)
	, ElementsProvider(InElementsProvider)
	, bWithZoom(bInWithZoom)
	, bWithPan(bInWithPan)
{
	bIsRealtime = true;

	VisibilityDelegate.BindLambda([] {return true; });

	EngineShowFlags = ESFIM_Game;
	EngineShowFlags.SetFog(false);
	
	FSoftObjectPath CropToViewRectMaterialPath("/CameraCalibration/Materials/M_CropToViewRect.M_CropToViewRect");
	if (UMaterialInterface* CropToViewRectMaterial = Cast<UMaterialInterface>(CropToViewRectMaterialPath.TryLoad()))
	{
		CropToViewRectMaterialInstance = TStrongObjectPtr<UMaterialInstanceDynamic>(UMaterialInstanceDynamic::Create(CropToViewRectMaterial, nullptr));
	}
}

void FSimulcamEditorViewportClient::Tick(float DeltaSeconds)
{
	ACameraActor* ViewActor = ElementsProvider.IsValid() ? ElementsProvider.Pin()->GetCameraActor() : nullptr;
	SetActorLock(ViewActor);
	
	FLevelEditorViewportClient::Tick(DeltaSeconds);

	// FLevelEditorViewportClient will update the post-process blends from the controlling actor every tick,
	// so append the crop to view rect pass afterward
	FPostProcessSettings CropToViewRectPPSettings;
	CropToViewRectPPSettings.AddBlendable(CropToViewRectMaterialInstance.Get(), 1.0f);
	
	ControllingActorExtraPostProcessBlends.Add(CropToViewRectPPSettings);
	ControllingActorExtraPostProcessBlendWeights.Add(1.0f);
}

void FSimulcamEditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{	
	UTexture* OverrideTexture = ElementsProvider.IsValid() ? ElementsProvider.Pin()->GetOverrideTexture() : nullptr;
	if (OverrideTexture && OverrideTexture->GetResource())
	{
		ViewportSize = FVector2D(InViewport->GetSizeXY());
		
		const FVector2D TextureSize(OverrideTexture->GetSurfaceWidth(), OverrideTexture->GetSurfaceHeight());
		const float RawZoom = GetCurrentZoom();
		const float TextureScale = FMath::Min(ViewportSize.X / TextureSize.X, ViewportSize.Y / TextureSize.Y);
		const FVector2D TextureOffset = 0.5 * (ViewportSize - TextureSize * TextureScale);
		
		FCanvasTileItem TileItem(TextureOffset * RawZoom - ViewportSize * RawZoom * ViewRect.Min, OverrideTexture->GetResource(), TextureSize * TextureScale * RawZoom, FLinearColor::White);
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_Translucent;
		Canvas->DrawItem(TileItem);
		return;
	}
	
	PreDraw(InViewport, Canvas);
	FEditorViewportClient::Draw(InViewport, Canvas);
}

void FSimulcamEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FLevelEditorViewportClient::Draw(View, PDI);
}

void FSimulcamEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	TSharedPtr<FSimulcamViewportElementsProvider> PinnedElementsProvider = ElementsProvider.Pin();
	if (!PinnedElementsProvider.IsValid())
	{
		return;
	}

	const FBox2D ConstrainedRect(FVector2D(View.UnscaledViewRect.Min), FVector2D(View.UnscaledViewRect.Max));
	const float RawZoom = GetCurrentZoom();
	
	UTexture* MediaOverlayTexture = PinnedElementsProvider->GetMediaOverlayTexture();
	if (MediaOverlayTexture && MediaOverlayTexture->GetResource())
	{
		const FLinearColor Color = FLinearColor::White * PinnedElementsProvider->GetMediaOverlayOpacity();

		MediaOverlayTexturePosition = ConstrainedRect.Min * (1.0 - RawZoom) + MediaOverlayTextureOffset * RawZoom - ConstrainedRect.GetSize() * RawZoom * ViewRect.Min;
		
		FCanvasTileItem TileItem(MediaOverlayTexturePosition, MediaOverlayTexture->GetResource(), MediaOverlayTextureScaledSize * RawZoom, Color);
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_Translucent;
		Canvas.DrawItem(TileItem);
	}
	
	UMaterialInterface* ToolOverlayMaterial = PinnedElementsProvider->GetToolOverlayMaterial();
	if (ToolOverlayMaterial && ToolOverlayMaterial->GetRenderProxy())
	{
		FCanvasTileItem TileItem(MediaOverlayTexturePosition, ToolOverlayMaterial->GetRenderProxy(), MediaOverlayTextureScaledSize * RawZoom);
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_Translucent;
		Canvas.DrawItem(TileItem);
	}

	UMaterialInterface* UserOverlayMaterial = PinnedElementsProvider->GetUserOverlayMaterial();
	if (UserOverlayMaterial && UserOverlayMaterial->GetRenderProxy())
	{
		FCanvasTileItem TileItem(MediaOverlayTexturePosition, UserOverlayMaterial->GetRenderProxy(), MediaOverlayTextureScaledSize * RawZoom);
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_Translucent;
		Canvas.DrawItem(TileItem);
	}

	// Draw black bars to cover up areas of the overlay textures that extend beyond the bounds of the unscaled view rect
	if (ConstrainedRect.Min.X > 0.0)
	{
		FCanvasTileItem LeftBar(FVector2D::ZeroVector, GBlackTexture, FVector2D(ConstrainedRect.Min.X, ViewportSize.Y), FLinearColor::White);
		Canvas.DrawItem(LeftBar);
	}
	
	if (ConstrainedRect.Max.X < ViewportSize.X)
	{
		FCanvasTileItem RightBar(FVector2D(ConstrainedRect.Max.X, 0.0), GBlackTexture, FVector2D(ViewportSize.X - ConstrainedRect.Max.X, ViewportSize.Y), FLinearColor::White);
		Canvas.DrawItem(RightBar);
	}
		
	if (ConstrainedRect.Min.Y > 0.0)
	{
		FCanvasTileItem TopBar(FVector2D::ZeroVector, GBlackTexture, FVector2D(ViewportSize.X, ConstrainedRect.Min.Y), FLinearColor::White);
		Canvas.DrawItem(TopBar);
	}
	
	if (ConstrainedRect.Max.Y < ViewportSize.Y)
	{
		FCanvasTileItem BottomBar(FVector2D(0.0, ConstrainedRect.Max.Y), GBlackTexture, FVector2D(ViewportSize.X, ViewportSize.Y - ConstrainedRect.Max.Y), FLinearColor::White);
		Canvas.DrawItem(BottomBar);
	}
	
	// If the user is current doing a marquee select, draw the marquee selection box
	if (bIsMarqueeSelecting)
	{
		FCanvasBoxItem BoxItem(SelectionStartCanvas, SelectionBoxSize);
		Canvas.DrawItem(BoxItem);
	}
}

void FSimulcamEditorViewportClient::MouseMove(FViewport* InViewport, int32 X, int32 Y)
{
	MousePosition = FIntPoint(X, Y);
}

bool FSimulcamEditorViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	TSharedPtr<FSimulcamViewportElementsProvider> PinnedElementsProvider = ElementsProvider.Pin();
	if (!PinnedElementsProvider.IsValid())
	{
		return false;
	}
	
	if (InEventArgs.Event == IE_Pressed)
	{
		const bool bIsCtrlDown = FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys().IsControlDown();
		const bool bIsAltDown = FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys().IsAltDown();

		const FVector2D LocalMouse = FVector2D(InEventArgs.Viewport->GetMouseX(), InEventArgs.Viewport->GetMouseY());
		MousePosition.X = FMath::Floor(LocalMouse.X);
		MousePosition.Y = FMath::Floor(LocalMouse.Y);

		if (InEventArgs.Key == EKeys::LeftMouseButton || InEventArgs.Key == EKeys::MiddleMouseButton || InEventArgs.Key == EKeys::RightMouseButton)
		{
			// check if we are under the viewport, otherwise the capture system will blindly trigger the PointerEvent
			if (LocalMouse.ComponentwiseAllGreaterOrEqual(FVector2D(0, 0)) &&
				LocalMouse.ComponentwiseAllLessThan(InEventArgs.Viewport->GetSizeXY()))
			{
				// create fake geometry and mouseposition
				const FVector2D FakeMousePosition = GetMousePositionInMediaOverlaySpace();
				const FIntPoint TextureSize = MediaOverlayTextureRawSize;

				// check for meaningful position
				if (FakeMousePosition.X >= 0 && FakeMousePosition.Y >= 0 && FakeMousePosition.X < TextureSize.X && FakeMousePosition.Y < TextureSize.Y)
				{
					if (bIsCtrlDown && bIsAltDown && !bIsMarqueeSelecting)
					{
						// The user is initiating a marquee select
						bIsMarqueeSelecting = true;
						SelectionStartCanvas = LocalMouse;
						SelectionStartTexture = FakeMousePosition;
						SelectionBoxSize = FVector2D(0);
					}
					else 
					{
						// The user is performing some other mouse click event
						const FGeometry FakeGeometry = FGeometry::MakeRoot(FVector2D(TextureSize), FSlateLayoutTransform());
						FPointerEvent PointerEvent(
							FSlateApplicationBase::CursorPointerIndex,
							FakeMousePosition,
							FakeMousePosition,
							TSet<FKey>(),
							InEventArgs.Key,
							0,
							FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys());

						PinnedElementsProvider->OnViewportClicked(FakeGeometry, PointerEvent);
					}
				}
			}
		}

		if (bWithZoom)
		{
			if (InEventArgs.Key == EKeys::MouseScrollUp || (InEventArgs.Key == EKeys::Add && bIsCtrlDown))
			{
				ChangeZoom(InEventArgs.Viewport, MousePosition, ZoomIncrement);
				return true;
			}

			if (InEventArgs.Key == EKeys::MouseScrollDown || (InEventArgs.Key == EKeys::Subtract && bIsCtrlDown))
			{
				ChangeZoom(InEventArgs.Viewport, MousePosition, -ZoomIncrement);
				return true;
			}

			if ((InEventArgs.Key == EKeys::Zero || InEventArgs.Key == EKeys::NumPadZero) && bIsCtrlDown)
			{
				ResetZoom();
				return true;
			}
		}

		return PinnedElementsProvider->OnViewportInputKey(InEventArgs.Key, InEventArgs.Event);
	}
	else if (InEventArgs.Event == IE_Released)
	{
		if (bIsMarqueeSelecting)
		{
			// The user has finished the marquee select
			bIsMarqueeSelecting = false;

			// Calculate where the selection ended in texture coordinates based on the latest selection box size
			const double CurrentZoom = GetCurrentZoom() * MediaOverlayTextureScale;
			FVector2D SelectionEndTexture = (SelectionStartTexture + (SelectionBoxSize / CurrentZoom));

			// Clamp end position to the texture size
			const FIntPoint TextureSize = MediaOverlayTextureRawSize;
			SelectionEndTexture.X = FMath::Clamp(SelectionEndTexture.X, 0, TextureSize.X);
			SelectionEndTexture.Y = FMath::Clamp(SelectionEndTexture.Y, 0, TextureSize.Y);

			const FVector2D TopLeftPoint = FIntPoint(FMath::Min(SelectionStartTexture.X, SelectionEndTexture.X), FMath::Min(SelectionStartTexture.Y, SelectionEndTexture.Y));
			const FVector2D BottomRightPoint = FIntPoint(FMath::Max(SelectionStartTexture.X, SelectionEndTexture.X), FMath::Max(SelectionStartTexture.Y, SelectionEndTexture.Y));

			PinnedElementsProvider->OnMarqueeSelect(TopLeftPoint, BottomRightPoint);
		}

		return PinnedElementsProvider->OnViewportInputKey(InEventArgs.Key, InEventArgs.Event);
	}
	else if (InEventArgs.Event == IE_Repeat)
	{
		return PinnedElementsProvider->OnViewportInputKey(InEventArgs.Key, InEventArgs.Event);
	}

	return false;
}

float FSimulcamEditorViewportClient::GetCurrentZoom() const
{
	return 1.0 / GetCurrentInvZoom();
}

float FSimulcamEditorViewportClient::GetCurrentInvZoom() const
{
	return ViewRect.Max.X - ViewRect.Min.X;
}

float FSimulcamEditorViewportClient::GetMediaOverlayScale() const
{
	return GetCurrentZoom() * MediaOverlayTextureScale;
}

#define SAFE_DIVIDE(Dividend, Divisor, Default) (Divisor != 0.0f) ? (Dividend / Divisor) : Default

void FSimulcamEditorViewportClient::PreDraw(FViewport* InViewport, FCanvas* Canvas)
{
	TSharedPtr<FSimulcamViewportElementsProvider> PinnedElementsProvider = ElementsProvider.Pin();
	if (!PinnedElementsProvider.IsValid())
	{
		return;
	}

	bool bViewportSizeChanged = false;
	if (ViewportSize != InViewport->GetSizeXY())
	{
		ViewportSize = InViewport->GetSizeXY();
		bViewportSizeChanged = true;
	}
	
	if (UTexture* MediaOverlayTexture = PinnedElementsProvider->GetMediaOverlayTexture())
	{
		const FIntPoint TextureSize = FIntPoint(MediaOverlayTexture->GetSurfaceWidth(), MediaOverlayTexture->GetSurfaceHeight());
		bool bOverlayTextureSizeChanged = false;
		if (TextureSize != MediaOverlayTextureRawSize)
		{
			MediaOverlayTextureRawSize = TextureSize;
			bOverlayTextureSizeChanged = true;
		}

		if (bViewportSizeChanged || bOverlayTextureSizeChanged)
		{
			MediaOverlayTextureScale = FMath::Min(SAFE_DIVIDE(ViewportSize.X, MediaOverlayTextureRawSize.X, 1.0f), SAFE_DIVIDE(ViewportSize.Y, MediaOverlayTextureRawSize.Y, 1.0f));
			MediaOverlayTextureScaledSize = FVector2D(MediaOverlayTextureRawSize) * MediaOverlayTextureScale;
			MediaOverlayTextureOffset = 0.5 * (ViewportSize - MediaOverlayTextureScaledSize);

			ResetZoom();
		}
	}
	else
	{
		MediaOverlayTextureRawSize = FIntPoint::ZeroValue;
		MediaOverlayTextureScale = 1.0f;
		MediaOverlayTextureScaledSize = FVector2D::ZeroVector;
		MediaOverlayTextureOffset = FVector2D::ZeroVector;
	}
	
	const float ViewportAspectRatio = SAFE_DIVIDE(ViewportSize.X, ViewportSize.Y, 1.0f);
	const float CGAspectRatio = bUseControllingActorViewInfo ? ControllingActorViewInfo.AspectRatio : ViewportAspectRatio;
	const float FilmbackAspectRatio = PinnedElementsProvider->GetCameraFeedAspectRatio();
	const float MediaOverlayAspectRatio = SAFE_DIVIDE(MediaOverlayTextureScaledSize.X, MediaOverlayTextureScaledSize.Y, 1.0f);

	// Compute the rectangle that the CG layer is intended to inhabit within the full viewport, as it will be aspect ratio constrained inside the media overlay texture
	CGViewportSize = CGAspectRatio > MediaOverlayAspectRatio ?
		FVector2D(MediaOverlayTextureScaledSize.X, SAFE_DIVIDE(MediaOverlayTextureScaledSize.X, CGAspectRatio, 0.0f)) :
		FVector2D(MediaOverlayTextureScaledSize.Y * CGAspectRatio, MediaOverlayTextureScaledSize.Y);

	// Need to modify the CG viewport size to match the aspect ratio of the viewport being rendered, as the viewport size is what determines the backbuffer
	// size of the render
	if (ViewportAspectRatio > CGAspectRatio)
	{
		CGViewportSize.X = CGViewportSize.Y * ViewportAspectRatio;
	}
	else
	{
		CGViewportSize.Y = SAFE_DIVIDE(CGViewportSize.X, ViewportAspectRatio, 0.0f);
	}

	if (FilmbackAspectRatio < CGAspectRatio)
	{
		CGViewportSize.X *= SAFE_DIVIDE(FilmbackAspectRatio, CGAspectRatio, 0.0f);
	}
	else
	{
		CGViewportSize.Y *= SAFE_DIVIDE(CGAspectRatio, FilmbackAspectRatio, 0.0f);
	}
	
	// TODO: Ideally we would want to unconstrain the aspect ratio so that we can make full use of the viewport's space. However, any change to the aspect ratio
	// here will change how lens distortion gets applied to the CG image, as the UV coordinates will change as the aspect ratio changes. In theory, this could be
	// mitigated by something like asymmetric overscan, but the lens distortion scene view extension has not been updated to account for asymmetric overscan yet.
	//ControllingActorViewInfo.bConstrainAspectRatio = false;
	//ControllingActorViewInfo.AspectRatioAxisConstraint = ViewportAspectRatio > CGAspectRatio ? EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV : EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV;
	
	if (CropToViewRectMaterialInstance.IsValid())
	{
		// If the CG viewport size is zero, assume that either the media texture size is zero/invalid or an aspect ratio is invalid. Either way, skip
		// setting the view rect and bounds in the crop material so that a normal viewport is rendered
		if (CGViewportSize.X <= KINDA_SMALL_NUMBER || CGViewportSize.Y <= KINDA_SMALL_NUMBER)
		{
			CropToViewRectMaterialInstance->ClearParameterValues();
			return;
		}
		
		float Overscan = 1.0f;
		if (ElementsProvider.IsValid())
		{
			if (ACameraActor* Camera = ElementsProvider.Pin()->GetCameraActor())
			{
				if (UCineCameraComponent* CineCamera = Cast<UCineCameraComponent>(Camera->GetCameraComponent()))
				{
					// Only apply overscan if the render pipeline will crop it automatically
					if (CineCamera->bCropOverscan)
					{
						Overscan = CineCamera->Overscan + 1.0f;

						// This would be unexpected but best to keep it safe.
						if (Overscan <= KINDA_SMALL_NUMBER)
						{
							Overscan = 1.0f;
						}
					}
					// When bCropOverscan is false, Overscan remains 1.0f
				}
			}
		}

		// Compute the view rect from the CG viewport size and the current zoom/pan
		const FBox2D CGViewRect((ViewRect.Min - 0.5) * ViewportSize / CGViewportSize + 0.5, (ViewRect.Max - 0.5) * ViewportSize / CGViewportSize + 0.5);
		
		// Apply overscan transformation (when Overscan = 1.0, this is a no-op)
		// The equation below is derived from combining the overscan scaling equation (rect = (rect - center) * overscan + center) with
		// the equation to transform to overscanned coordinate space (rect = (rect - 0.5) / overscan + 0.5)
		const FBox2D OverscannedViewRect = CGViewRect.ShiftBy((CGViewRect.GetCenter() - 0.5) * (1.0 / Overscan - 1.0));
		
		// Compute the overscan bounds (when Overscan = 1.0, this yields (0,0) to (1,1))
		const FBox2D OverscanBounds = FBox2D(-FVector2D(0.5, 0.5) / Overscan + 0.5, FVector2D(0.5, 0.5) / Overscan + 0.5);
		
		CropToViewRectMaterialInstance->SetVectorParameterValue("ViewRect", FVector4(OverscannedViewRect.Min.X, OverscannedViewRect.Min.Y, OverscannedViewRect.Max.X, OverscannedViewRect.Max.Y));
		CropToViewRectMaterialInstance->SetVectorParameterValue("ViewBounds", FVector4(OverscanBounds.Min.X, OverscanBounds.Min.Y, OverscanBounds.Max.X, OverscanBounds.Max.Y));
	}
}

FVector2D FSimulcamEditorViewportClient::GetMousePositionInMediaOverlaySpace() const
{
	return (FVector2D(MousePosition) - MediaOverlayTexturePosition) * GetCurrentInvZoom() / MediaOverlayTextureScale;
}

FVector2D FSimulcamEditorViewportClient::GetScaledMediaOverlayTextureSize() const
{
	return MediaOverlayTextureScaledSize * GetCurrentZoom();
}

void FSimulcamEditorViewportClient::ResetZoom()
{
	ViewRect = FBox2D(FVector2D(0.0, 0.0), FVector2D(1.0, 1.0));
}

bool FSimulcamEditorViewportClient::InputChar(FViewport* InViewport, int32 ControllerId, TCHAR Character)
{
	if (!bWithZoom)
	{
		return false;
	}

	const bool bIsCtrlDown = FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys().IsControlDown();

	if (Character == TCHAR('+') && bIsCtrlDown)
	{
		ChangeZoom(InViewport, MousePosition, ZoomIncrement);
		return true;
	}

	if (Character == TCHAR('-') && bIsCtrlDown)
	{
		ChangeZoom(InViewport, MousePosition, -ZoomIncrement);
		return true;
	}

	return false;
}

bool FSimulcamEditorViewportClient::InputAxis(const FInputKeyEventArgs& InEventArgs)
{
	const FKey& Key = InEventArgs.Key;
	const float MouseDelta = InEventArgs.AmountDepressed;
	FViewport* InViewport = InEventArgs.Viewport;
	
	if (Key == EKeys::MouseX || Key == EKeys::MouseY)
	{
		if (ShouldUseMousePanning(InViewport))
		{
			if (Key == EKeys::MouseY)
			{
				ViewRect = ViewRect.ShiftBy(FVector2D(0.0, GetClampedVerticalPanDelta(MouseDelta)));
			}
			else if (Key == EKeys::MouseX)
			{
				ViewRect = ViewRect.ShiftBy(FVector2D(GetClampedHorizontalPanDelta(MouseDelta), 0.0));
			}
		}

		// Update the marquee selection box size based on the current mouse position
		if (bIsMarqueeSelecting)
		{
			FVector2D LocalMouse = FVector2D(InViewport->GetMouseX(), InViewport->GetMouseY());
			SelectionBoxSize = LocalMouse - SelectionStartCanvas;
			ClampSelectionBoxSizeToTextureSize();
		}

		return true;
	}

	return false;
}

void FSimulcamEditorViewportClient::ClampSelectionBoxSizeToTextureSize()
{
	const FIntPoint TextureSize = MediaOverlayTextureRawSize;
	const double CurrentZoom = GetCurrentZoom() * MediaOverlayTextureScale;

	const FVector2D MinBoxSize = (FVector2D(0.0f, 0.0f) - SelectionStartTexture) * CurrentZoom;
	const FVector2D MaxBoxSize = (FVector2D(TextureSize) - SelectionStartTexture) * CurrentZoom;

	SelectionBoxSize.X = FMath::Clamp(SelectionBoxSize.X, MinBoxSize.X, MaxBoxSize.X);
	SelectionBoxSize.Y = FMath::Clamp(SelectionBoxSize.Y, MinBoxSize.Y, MaxBoxSize.Y);
}

bool FSimulcamEditorViewportClient::ShouldUseMousePanning(FViewport* InViewport) const
{
	return bWithPan && InViewport->KeyState(EKeys::RightMouseButton);
}

float FSimulcamEditorViewportClient::GetClampedHorizontalPanDelta(float MouseDelta)
{
	const float ViewportWidth = Viewport->GetSizeXY().X;
	const float ScaledMouseDelta = MouseDelta / ViewportWidth;

	// Mouse moved left, which means a pan to the right
	if (MouseDelta < 0)
	{
		return FMath::Min(-ScaledMouseDelta, 1.0 - ViewRect.Max.X);
	}

	// Otherwise, mouse moved right, which means pan left
	return FMath::Max(-ScaledMouseDelta, -ViewRect.Min.X);
}

float FSimulcamEditorViewportClient::GetClampedVerticalPanDelta(float MouseDelta)
{
	const float ViewportHeight = Viewport->GetSizeXY().Y;
	const float ScaledMouseDelta = MouseDelta / ViewportHeight;

	// Mouse moved up, which means a pan down
	if (MouseDelta > 0)
	{
		return FMath::Min(ScaledMouseDelta, 1.0 - ViewRect.Max.Y);
	}

	// Otherwise, mouse moved down, which means pan up
	return FMath::Max(ScaledMouseDelta, -ViewRect.Min.Y);
}

void FSimulcamEditorViewportClient::ChangeZoom(FViewport* InViewport, const FIntPoint& InMousePosition, float ZoomDelta)
{
	const float CurrentZoom = GetCurrentZoom();
	const float NewZoom = FMath::Clamp(CurrentZoom + ZoomDelta, 1.0, MaxZoom);
	
	ViewRect.Min = ViewRect.Min + FVector2D(InMousePosition) / FVector2D(InViewport->GetSizeXY()) * ( 1.0 / CurrentZoom - 1.0 / NewZoom);
	ViewRect.Max = ViewRect.Min + 1.0 / NewZoom;

	// Correct zoom rect back towards the 0-1 range in case the zoom change caused an overshoot
	FVector2D Correction = FVector2D::ZeroVector;
	if (ViewRect.Max.X > 1.0)
	{
		Correction.X = 1.0 - ViewRect.Max.X;
	}
	else if (ViewRect.Min.X < 0.0)
	{
		Correction.X = -ViewRect.Min.X;
	}

	if (ViewRect.Max.Y > 1.0)
	{
		Correction.Y = 1.0 - ViewRect.Max.Y;
	}
	else if (ViewRect.Min.Y < 0.0)
	{
		Correction.Y = -ViewRect.Min.Y;
	}

	ViewRect = ViewRect.ShiftBy(Correction);
}

EMouseCursor::Type FSimulcamEditorViewportClient::GetCursor(FViewport* InViewport, int32 X, int32 Y)
{
	return ShouldUseMousePanning(InViewport) ? EMouseCursor::GrabHandClosed : EMouseCursor::Default;
}

FText FSimulcamEditorViewportClient::GetDisplayedResolution() const
{
	TSharedPtr<FSimulcamViewportElementsProvider> PinnedElementsProvider = ElementsProvider.Pin();
	if (!PinnedElementsProvider.IsValid())
	{
		return FText::GetEmpty();
	}

	UTexture* Texture = PinnedElementsProvider->GetOverrideTexture();
	const bool bIsOverrideTexture = Texture != nullptr;
	
	if (!bIsOverrideTexture)
	{
		Texture = PinnedElementsProvider->GetMediaOverlayTexture();
	}

	if (!Texture)
	{
		return FText::GetEmpty();
	}

	FVector2D TextureSize;
	FVector2D ScaledTextureSize;
	float Zoom;
	FVector2D MouseTexturePosition;

	if (bIsOverrideTexture)
	{
		TextureSize = FVector2D(Texture->GetSurfaceWidth(), Texture->GetSurfaceHeight());
		Zoom = GetCurrentZoom() * FMath::Min(ViewportSize.X / TextureSize.X, ViewportSize.Y / TextureSize.Y);
		ScaledTextureSize = TextureSize * Zoom;
		MouseTexturePosition = (FVector2D(MousePosition) - 0.5 * (ViewportSize - ScaledTextureSize * GetCurrentInvZoom())) / Zoom;
	}
	else
	{
		TextureSize = MediaOverlayTextureRawSize;
		Zoom = GetCurrentZoom() * MediaOverlayTextureScale;
		ScaledTextureSize = GetScaledMediaOverlayTextureSize();
		MouseTexturePosition = GetMousePositionInMediaOverlaySpace();
	}

	if (MouseTexturePosition.X < 0)
	{
		MouseTexturePosition.X = 0.0;
	}
	else if (MouseTexturePosition.X > TextureSize.X)
	{
		MouseTexturePosition.X = TextureSize.X;
	}

	if (MouseTexturePosition.Y < 0)
	{
		MouseTexturePosition.Y = 0.0;
	}
	else if (MouseTexturePosition.Y > TextureSize.Y)
	{
		MouseTexturePosition.Y = TextureSize.Y;
	}

	return FText::Format(
		FText::FromString("Displayed: {0}x{1}\nTextureSize: {2}x{3}\nTexturePosition: {4}x{5}\nZoom: {6}\n\nCtrl+Alt+Drag: Marquee ROI selection"),
		FText::AsNumber(ScaledTextureSize.X),
		FText::AsNumber(ScaledTextureSize.Y),
		FText::AsNumber(TextureSize.X),
		FText::AsNumber(TextureSize.Y),
		FText::AsNumber((int32)MouseTexturePosition.X),
		FText::AsNumber((int32)MouseTexturePosition.Y),
		FText::AsNumber(Zoom)
	);
}
