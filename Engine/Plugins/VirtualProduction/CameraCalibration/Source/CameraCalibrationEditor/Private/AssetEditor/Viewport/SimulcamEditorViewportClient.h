// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "LevelEditorViewport.h"

class FCanvas;
class ITextureEditorToolkit;
class STextureEditorViewport;
class UTexture2D;
class SSimulcamViewport;
class SSimulcamEditorViewport;

/** Interface used by the simulcam viewport client to retrieve any elements used for rendering the simulcam viewport */
class FSimulcamViewportElementsProvider
{
public:
	virtual ACameraActor* GetCameraActor() const { return nullptr; }
	virtual float GetCameraFeedAspectRatio() const { return 1.0f; }
	virtual UTexture* GetMediaOverlayTexture() const { return nullptr; }
	virtual float GetMediaOverlayOpacity() const { return 1.0f; }
	virtual UMaterialInterface* GetToolOverlayMaterial() const { return nullptr;}
	virtual UMaterialInterface* GetUserOverlayMaterial() const { return nullptr;}
	virtual UTexture* GetOverrideTexture() const { return nullptr; }

	virtual void OnViewportClicked(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) { }
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InInputEvent) { return false; }
	virtual void OnMarqueeSelect(const FVector2D& InStartPosition, const FVector2D& InEndPosition) { }
};

class FSimulcamEditorViewportClient
	: public FLevelEditorViewportClient
{
public:
	static constexpr float MaxZoom = 16.0f;
	static constexpr float ZoomIncrement = 0.1f;
	
public:
	/** Constructor */
	FSimulcamEditorViewportClient(TSharedPtr<FSimulcamViewportElementsProvider> InElementsProvider, const bool bInWithZoom, const bool bInWithPan);

	/** Begin FViewportClient interface */
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override;
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual bool InputKey(const FInputKeyEventArgs& InEventArgs) override;
	virtual bool InputChar(FViewport* InViewport, int32 ControllerId, TCHAR Character) override;
	virtual bool InputAxis(const FInputKeyEventArgs& InEventArgs) override;
	virtual EMouseCursor::Type GetCursor(FViewport* InViewport, int32 X, int32 Y) override;
	virtual void MouseMove(FViewport* InViewport, int32 X, int32 Y) override;
	/** End FViewportClient interface */

	/** Returns a string representation of the currently displayed textures resolution */
	FText GetDisplayedResolution() const;

	/** Gets the current zoom level of the view rectangle */
	float GetCurrentZoom() const;

	/** Gets the current inverse zoom level of the view rectangle */
	float GetCurrentInvZoom() const;

	/** Gets the amount that the media overlay texture is being scaled by, including zoom scaling */
	float GetMediaOverlayScale() const;

private:
	/** Performs any necessary processing needed before the viewport client renders */
	void PreDraw(FViewport* InViewport, FCanvas* Canvas);
	
	/** Gets the current mouse position in the pixel coordinates of the media overlay texture */
	FVector2D GetMousePositionInMediaOverlaySpace() const;

	/** Gets the size the media overlay texture is being scaled to when displayed in the viewport */
	FVector2D GetScaledMediaOverlayTextureSize() const;
	
	/** TRUE if right clicking and dragging for panning a texture 2D */
	bool ShouldUseMousePanning(FViewport* InViewport) const;

	/** Converts a horizontal mouse delta in pixels and converts it to relative coordinates, clamping to ensure that the view rect stays bounded between 0.0 and 1.0 */
	float GetClampedHorizontalPanDelta(float MouseDelta);

	/** Converts a vertical mouse delta in pixels and converts it to relative coordinates, clamping to ensure that the view rect stays bounded between 0.0 and 1.0 */
	float GetClampedVerticalPanDelta(float MouseDelta);

	void ChangeZoom(FViewport* InViewport, const FIntPoint& InMousePosition, float ZoomDelta);

	/** Zoom to fit the texture in the viewport */
	void ResetZoom();

	/** Clamp the selection box size to only be large as the texture size */
	void ClampSelectionBoxSizeToTextureSize();

private:
	/** An interface that supplies all the necessary assets needed to render the simulcam viewport, and has handlers for various input events */
	TWeakPtr<FSimulcamViewportElementsProvider> ElementsProvider;

	/** Post process material to crop the viewport's output to the zoom/pan view rect */
	TStrongObjectPtr<UMaterialInstanceDynamic> CropToViewRectMaterialInstance;

	/** The position of the mouse within the viewport */
	FIntPoint MousePosition = FIntPoint::ZeroValue;

	/** Cached viewport size, to keep track of when the viewport size has changed so various values can be recomputed */
	FVector2D ViewportSize = FVector2D::ZeroVector;

	/** The size of the CG viewport when it is scaled down to fit within the media overlay */
	FVector2D CGViewportSize = FVector2D::ZeroVector;
	
	/** Box that represents the portion of the texture and viewport the user is zoomed and panned to, in relative coordinates ((0, 0) is top left, (1, 1) is bottom right) */
	FBox2D ViewRect = FBox2D(FVector2D(0.0, 0.0), FVector2D(1.0, 1.0));

	/** The offset from the viewport's top left corner that the media overlay texture must be placed to center it in the viewport */
	FVector2D MediaOverlayTextureOffset = FVector2D::ZeroVector;
	
	/** The position in pixel coordinates of the viewport at which the media overlay texture was last drawn */
	FVector2D MediaOverlayTexturePosition = FVector2D::ZeroVector;

	/** The raw size of the media overlay texture */
	FIntPoint MediaOverlayTextureRawSize = FIntPoint::ZeroValue;

	/** The size of the media overlay texture when scaled to fit inside the viewport */
	FVector2D MediaOverlayTextureScaledSize = FVector2D::ZeroVector;
	
	/** The scale applied to the media overlay texture to match the rendered viewport size when the media overlay texture was last drawn */
	double MediaOverlayTextureScale = 1.0;

	bool bWithZoom;
	bool bWithPan;

	bool bIsMarqueeSelecting = false;
	FVector2D SelectionStartTexture = FVector2D(0);
	FVector2D SelectionStartCanvas = FVector2D(0);
	FVector2D SelectionBoxSize = FVector2D(0);
};
