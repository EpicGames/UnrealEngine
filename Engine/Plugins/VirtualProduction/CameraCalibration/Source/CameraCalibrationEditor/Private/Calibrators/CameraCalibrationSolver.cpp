// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationSolver.h"

#include "CameraCalibrationUtils.h"
#include "HAL/IConsoleManager.h"
#include "Models/AnamorphicLensModel.h"
#include "Models/SphericalLensModel.h"
#include "OpenCVHelper.h"

#if WITH_OPENCV

#include "PreOpenCVHeaders.h"
#include "opencv2/calib3d.hpp"
#include "PostOpenCVHeaders.h"

#endif	// WITH_OPENCV

#define LOCTEXT_NAMESPACE "CameraCalibrationSolver"

static TAutoConsoleVariable<int> CVarUseLegacySphericalSolver(TEXT("CameraCalibration.UseLegacySphericalSolver"), 0, TEXT("If set, the legacy OpenCV spherical solver will be used"));
static TAutoConsoleVariable<bool> CVarConstrainWithCameraMotion(TEXT("CameraCalibration.ConstrainWithCameraMotion"), false, TEXT("If true, the solver will only solve for the camera pose of the first image, and will use the input tracked camera poses to transform the solve pose for subsequent images."));
static TAutoConsoleVariable<float> CVarRotationStepValue(TEXT("CameraCalibration.RotationStepValue"), 0.05f, TEXT("The value of the initial step size to use when finding an optimal nodal offset rotation that minimizes reprojection error."));
static TAutoConsoleVariable<float> CVarLocationStepValue(TEXT("CameraCalibration.LocationStepValue"), 0.5f, TEXT("The value of the initial step size to use when finding an optimal nodal offset location that minimizes reprojection error."));

DEFINE_LOG_CATEGORY_STATIC(LogCameraCalibrationSolver, Log, All);

bool ULensDistortionSolver::GetStatusText(FText& OutStatusText) 
{ 
	OutStatusText = StatusText;
	if (bHasStatusChanged)
	{
		bHasStatusChanged = false;
		return true;
	}
	return false;
}

void ULensDistortionSolver::SetStatusText(FText InStatusText) 
{ 
	StatusText = InStatusText;
	bHasStatusChanged = true;
}

FText ULensDistortionSolverOpenCV::GetDisplayName_Implementation() const
{
	return LOCTEXT("OpenCVSolverDisplayName", "OpenCV Solver");
}

FDistortionCalibrationResult ULensDistortionSolverOpenCV::Solve_Implementation(
	const TArray<FObjectPoints>& ObjectPointArray,
	const TArray<FImagePoints>& ImagePointArray,
	const FIntPoint ImageSize,
	const FVector2D& FocalLength,
	const FVector2D& ImageCenter,
	const TArray<float>& DistortionParameters,
	const TArray<FTransform>& CameraPoses,
	const TArray<FTransform>& TargetPoses,
	TSubclassOf<ULensModel> LensModel,
	double PixelAspect,
	ECalibrationFlags SolverFlags)
{
	FDistortionCalibrationResult Result;

#if WITH_OPENCV
	const int NumImages = ObjectPointArray.Num();

	// Create an array to store the number of points in each image
	cv::Mat NumPointsMat = cv::Mat(1, NumImages, CV_32S);

	int NumTotalPoints = 0;
	int MaxPoints = 0;
	for (int ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		const int NumPointsInImage = ObjectPointArray[ImageIndex].Points.Num();

		NumPointsMat.at<int>(ImageIndex) = NumPointsInImage;
		NumTotalPoints += NumPointsInImage;

		MaxPoints = MAX(MaxPoints, NumPointsInImage);
	}

	cv::Mat ObjectPointsMat = cv::Mat(1, NumTotalPoints, CV_64FC3);
	cv::Mat ImagePointsMat = cv::Mat(1, NumTotalPoints, CV_64FC2);

	// Reorganize the 3D and 2D points from the input arrays or arrays to be laid out linearly in memory in two cv::Mat objects
	GatherPoints(ObjectPointArray, ImagePointArray, ObjectPointsMat, ImagePointsMat);

	TArray<int32> CameraPoseIndices;
	TArray<FTransform> UniqueCameraPoses;
	if ((EnumHasAnyFlags(SolverFlags, ECalibrationFlags::GroupCameraPoses)))
	{
		// Find the set of unique camera poses to reduce the number of poses we need to solve for
		FindUniqueCameraPoses(CameraPoses, UniqueCameraPoses, CameraPoseIndices);
	}
	else
	{
		// Do not group camera poses, and consider each one to be unique
		for (int32 PoseIndex = 0; PoseIndex < CameraPoses.Num(); ++PoseIndex)
		{
			UniqueCameraPoses.Add(CameraPoses[PoseIndex]);
			CameraPoseIndices.Add(PoseIndex);
		}
	}

	double RMSE = 0.0;

	cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);

	CameraMatrix.at<double>(0, 0) = FocalLength.X;
	CameraMatrix.at<double>(1, 1) = FocalLength.Y;
	CameraMatrix.at<double>(0, 2) = ImageCenter.X;
	CameraMatrix.at<double>(1, 2) = ImageCenter.Y;

	const int NumDistortionCoefficients = LensModel->GetDefaultObject<ULensModel>()->GetNumParameters();
	cv::Mat DistCoeffs = cv::Mat(1, NumDistortionCoefficients, CV_64F);

	// If input parameters were provided, they can be used to initialize the solver. Otherwise, the solver will be initialized with the default values for the input model.
	if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::UseIntrinsicGuess) && (DistortionParameters.Num() == NumDistortionCoefficients))
	{
		for (int CoeffIndex = 0; CoeffIndex < NumDistortionCoefficients; ++CoeffIndex)
		{
			DistCoeffs.at<double>(CoeffIndex) = DistortionParameters[CoeffIndex];
		}
	}
	else
	{
		TArray<float> DefaultCoefficients;
		LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(DefaultCoefficients);

		for (int CoeffIndex = 0; CoeffIndex < NumDistortionCoefficients; ++CoeffIndex)
		{
			DistCoeffs.at<double>(CoeffIndex) = DefaultCoefficients[CoeffIndex];
		}
	}

	if (LensModel == UAnamorphicLensModel::StaticClass())
	{
		DistCoeffs.at<double>(0) = PixelAspect;
	}
	else if (LensModel == USphericalLensModel::StaticClass())
	{
		// The spherical distortion coefficients in our model are in a different order than they appear in the solver
		// so the initial values need to be rearranged before initializing the solver
		const double K3 = DistCoeffs.at<double>(2);
		const double P1 = DistCoeffs.at<double>(3);
		const double P2 = DistCoeffs.at<double>(4);
		DistCoeffs.at<double>(2) = P1;
		DistCoeffs.at<double>(3) = P2;
		DistCoeffs.at<double>(4) = K3;
	}

	cv::Size CvImageSize = cv::Size(ImageSize.X, ImageSize.Y);

	// If the cvar is set, and the lens model is spherical, use the legacy version of the solver which simply calls cv::calibrateCamera()
	// Note: this can be removed in the future after some testing shows that the hand-written version of the solver works identically (if not better) compared to the legacy version
	if (CVarUseLegacySphericalSolver.GetValueOnAnyThread() && LensModel == USphericalLensModel::StaticClass())
	{
		std::vector<cv::Mat> Rvecs;
		std::vector<cv::Mat> Tvecs;

		std::vector<std::vector<cv::Point2f>> Samples2d;
		std::vector<std::vector<cv::Point3f>> Samples3d;

		Samples2d.reserve(ImagePointArray.Num());
		Samples3d.reserve(ObjectPointArray.Num());

		for (const FObjectPoints& PointsInImage : ObjectPointArray)
		{
			std::vector<cv::Point3f> Points3d;
			Points3d.reserve(PointsInImage.Points.Num());

			for (const FVector& Point3d : PointsInImage.Points)
			{
				const FVector Point3dCV = FOpenCVHelper::ConvertUnrealToOpenCV(Point3d);
				Points3d.push_back(cv::Point3f(Point3dCV.X, Point3dCV.Y, Point3dCV.Z));
			}

			Samples3d.push_back(Points3d);
		}

		for (const FImagePoints& PointsInImage : ImagePointArray)
		{
			std::vector<cv::Point2f> Points2d;
			Points2d.reserve(PointsInImage.Points.Num());

			for (const FVector2D& Point2d : PointsInImage.Points)
			{
				Points2d.push_back(cv::Point2f(Point2d.X, Point2d.Y));
			}

			Samples2d.push_back(Points2d);
		}

		int LegacyFlags = 0;
		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::UseIntrinsicGuess))
		{
			LegacyFlags |= cv::CALIB_USE_INTRINSIC_GUESS;
		}
		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixFocalLength))
		{
			LegacyFlags |= cv::CALIB_FIX_FOCAL_LENGTH;
		}
		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixPrincipalPoint))
		{
			LegacyFlags |= cv::CALIB_FIX_PRINCIPAL_POINT;
		}
		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixAspectRatio))
		{
			LegacyFlags |= cv::CALIB_FIX_ASPECT_RATIO;
		}

		RMSE = cv::calibrateCamera(
			Samples3d,
			Samples2d,
			CvImageSize,
			CameraMatrix,
			DistCoeffs,
			Rvecs,
			Tvecs,
			cv::noArray(),
			cv::noArray(),
			cv::noArray(),
			LegacyFlags
		);

		// Set the output intrinsics and distortion parameters to the final values calculated by the solver
		Result.FocalLength.FxFy.X = CameraMatrix.at<double>(0, 0);
		Result.FocalLength.FxFy.Y = CameraMatrix.at<double>(1, 1);
		Result.ImageCenter.PrincipalPoint.X = CameraMatrix.at<double>(0, 2);
		Result.ImageCenter.PrincipalPoint.Y = CameraMatrix.at<double>(1, 2);

		// The spherical distortion coefficients in our model are in a different order than they appear in the solver, so the results need to be rearranged
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(0));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(1));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(4));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(2));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(3));

		Result.ReprojectionError = RMSE;

		Result.CameraPoses.Reserve(NumImages);
		for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
		{
			FTransform LegacyCameraPose;
			FOpenCVHelper::MakeCameraPoseFromObjectVectors(Rvecs[ImageIndex], Tvecs[ImageIndex], LegacyCameraPose);

			Result.CameraPoses.Add(LegacyCameraPose);
		}

		return Result;
	}

	// Compute canonical plane detection once to avoid redundant calculations
	const bool bUseCanonicalPlaneInit = CanUseHomographyInitialization(ObjectPointsMat);
	
	// If the flag to use a starting guess for the camera intrinsics is not set, calculate some initial values for the intrinsic parameters
	if (!(EnumHasAnyFlags(SolverFlags, ECalibrationFlags::UseIntrinsicGuess)))
	{
		InitCameraIntrinsics(ObjectPointsMat, ImagePointsMat, NumPointsMat, CvImageSize, CameraMatrix, bUseCanonicalPlaneInit);
	}

	// Initialize the solver with the number of parameters to solve, and the maximum number of iterations to run
	int NumIntrinsics = NumDistortionCoefficients + 4; // Includes Fx, Fy, Cx, and Cy
	const int NumExtrinsics = 6; // 3 for rotation vector, 3 for translation vector

	if ((EnumHasAnyFlags(SolverFlags, ECalibrationFlags::SolveTargetOffset)))
	{
		const int NumObjectOffsetParams = 6; // 3 for rotation vector, 3 for translation vector
		NumIntrinsics += NumObjectOffsetParams;
	}

	// If using a guess for the extrinsic parameters, then the solver will be constrained to solve only one camera pose, and needs only one set of extrinsic parameters.
	// Otherwise, the solver needs one set of extrinsic parameters per image.
	const bool bConstrainWithCameraMotion = CVarConstrainWithCameraMotion.GetValueOnAnyThread();

	int NumPosesToSolve = 0;
	if (bConstrainWithCameraMotion)
	{
		NumPosesToSolve = 1;
	}
	else
	{
		NumPosesToSolve = UniqueCameraPoses.Num();
	}

	const int NumParamsToSolve = NumIntrinsics + (NumPosesToSolve * NumExtrinsics);

	constexpr int NumErrors = 0;
	constexpr int MaxIterations = 30;

	FLevMarqSolver Solver(NumParamsToSolve, NumErrors, MaxIterations);

	// Initialize the solver with the starting values for the intrinsic parameters and distortion parameters
	double* SolverParams = Solver.Params.ptr<double>();
	SolverParams[0] = CameraMatrix.at<double>(0, 0);
	SolverParams[1] = CameraMatrix.at<double>(1, 1);
	SolverParams[2] = CameraMatrix.at<double>(0, 2);
	SolverParams[3] = CameraMatrix.at<double>(1, 2);

	// Initialize the target offset rvec and tvec to zero
	if ((EnumHasAnyFlags(SolverFlags, ECalibrationFlags::SolveTargetOffset)))
	{
		SolverParams[4] = 0.0f;
		SolverParams[5] = 0.0f;
		SolverParams[6] = 0.0f;
		SolverParams[7] = 0.0f;
		SolverParams[8] = 0.0f;
		SolverParams[9] = 0.0f;
	}

	// Initialize the solver with the starting values for the distortion coefficients
	const int32 DistortionParameterOffset = NumIntrinsics - NumDistortionCoefficients;
	FMemory::Memcpy(SolverParams + DistortionParameterOffset, DistCoeffs.ptr<double>(), NumDistortionCoefficients * sizeof(double));

	// Instruct the solver to ignore some parameters when running its solve
	uchar* Mask = Solver.Mask.ptr<uchar>();
	if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixAspectRatio))
	{
		Mask[0] = 0;
	}
	if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixFocalLength))
	{
		Mask[0] = 0;
		Mask[1] = 0;
	}
	if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixPrincipalPoint))
	{
		Mask[2] = 0;
		Mask[3] = 0;
	}
	if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixExtrinsics))
	{
		for (int ParamIndex = NumIntrinsics; ParamIndex < NumParamsToSolve; ++ParamIndex)
		{
			Mask[ParamIndex] = 0;
		}
	}
	if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixDistortion))
	{
		for (int ParamIndex = DistortionParameterOffset; ParamIndex < DistortionParameterOffset + NumDistortionCoefficients; ++ParamIndex)
		{
			Mask[ParamIndex] = 0;
		}
	}

	if (LensModel == UAnamorphicLensModel::StaticClass())
	{
		Mask[DistortionParameterOffset] = 0; // We do not want to solve for pixel aspect
	}

	// Initialize the starting guess for the camera's extrinsic parameters. 
	// If using a guess for the extrinsic parameters, then the initial guess is just the first input camera pose
	// Otherwise, a starting pose will need to be computed for each image
	int ObjectPointsIndex = 0;
	for (int ImageIndex = 0; ImageIndex < NumPosesToSolve; ImageIndex++)
	{
		// Get a view to parameters used by the solver for the rotation and translation vectors for this image
		const int ExtrinsicOffset = NumIntrinsics + (ImageIndex * NumExtrinsics);
		cv::Mat Rotation = Solver.Params.rowRange(ExtrinsicOffset, ExtrinsicOffset + 3);
		cv::Mat Translation = Solver.Params.rowRange(ExtrinsicOffset + 3, ExtrinsicOffset + 6);

		if ((EnumHasAnyFlags(SolverFlags, ECalibrationFlags::UseExtrinsicGuess)))
		{
			FOpenCVHelper::MakeObjectVectorsFromCameraPose(UniqueCameraPoses[ImageIndex], Rotation, Translation);
		}
		else
		{
			const int NumImagePoints = NumPointsMat.at<int>(ImageIndex);

			// Get the object and image points for this image
			cv::Mat ObjectPointsInImage = ObjectPointsMat.colRange(ObjectPointsIndex, ObjectPointsIndex + NumImagePoints);
			cv::Mat ImagePointsInImage = ImagePointsMat.colRange(ObjectPointsIndex, ObjectPointsIndex + NumImagePoints);

			ObjectPointsIndex += NumImagePoints;

			InitCameraExtrinsics(LensModel, ObjectPointsInImage, ImagePointsInImage, CameraMatrix, DistCoeffs, CvImageSize, Rotation, Translation, SolverFlags, bUseCanonicalPlaneInit);
		}
	}

	// If using a guess for the extrinsic parameters, compute the transformation from the first camera pose to each subsequent pose
	// which represents how the camera moved between each image. The constrained solver, which only solves one camera pose, will use 
	// this transformation to offset the pose used in the projection of points for each image. 
	TArray<FTransform> CameraMovements;
	if (bConstrainWithCameraMotion)
	{
		CameraMovements.Reserve(CameraPoses.Num());

		for (const FTransform& Pose : CameraPoses)
		{
			const FTransform CameraMovement = CameraPoses[0].Inverse() * Pose;
			CameraMovements.Add(CameraMovement);
		}
	}

	double ReprojectionError = 0.0;

	cv::Mat Jacobian(MaxPoints * 2, NumExtrinsics + NumIntrinsics, CV_64FC1, cv::Scalar(0));
	cv::Mat Diffs(MaxPoints * 2, 1, CV_64FC1);

	int32 LoopCounter = 0;
	while (true)
	{
		// Update the solver
		bool bShouldProceed = Solver.UpdateAlt() && IsRunning();
		bool bComputeJacobian = Solver.State == FLevMarqSolver::ESolverState::ComputeJacobian;

		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixAspectRatio))
		{
			SolverParams[0] = SolverParams[1] * PixelAspect;
		}

		// Update the camera matrix and distortion parameters with the latest values of the parameters from the solver
		CameraMatrix.at<double>(0, 0) = SolverParams[0];
		CameraMatrix.at<double>(1, 1) = SolverParams[1];
		CameraMatrix.at<double>(0, 2) = SolverParams[2];
		CameraMatrix.at<double>(1, 2) = SolverParams[3];

		cv::Mat TargetOffsetRotation = cv::Mat::zeros(3, 1, CV_64F);;
		cv::Mat TargetOffsetTranslation = cv::Mat::zeros(3, 1, CV_64F);;
		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::SolveTargetOffset))
		{
			TargetOffsetRotation = Solver.Params.rowRange(4, 7);
			TargetOffsetTranslation = Solver.Params.rowRange(7, 10);
		}

		FMemory::Memcpy(DistCoeffs.ptr<double>(), SolverParams + DistortionParameterOffset, NumDistortionCoefficients * sizeof(double));

		// If the solver determined that it no longer needs to proceed, break out of the loop
		if (!bShouldProceed)
		{
			break;
		}

		// If using a guess for the extrinsic parameters, cache the current solver pose as a FTransform to more easily offset the camera pose for each image
		FTransform CurrentSolverPose;
		if (bConstrainWithCameraMotion)
		{
			cv::Mat Rotation = Solver.Params.rowRange(NumIntrinsics, NumIntrinsics + 3);
			cv::Mat Translation = Solver.Params.rowRange(NumIntrinsics + 3, NumIntrinsics + 6);

			FOpenCVHelper::MakeCameraPoseFromObjectVectors(Rotation, Translation, CurrentSolverPose);
		}

		ReprojectionError = 0;

		ObjectPointsIndex = 0;
		for (int ImageIndex = 0; ImageIndex < NumImages; ImageIndex++)
		{
			int NumImagePoints = NumPointsMat.at<int>(ImageIndex);

			// Get the object and image points for this image
			cv::Mat ObjectPointsInImage = cv::Mat(ObjectPointsMat.colRange(ObjectPointsIndex, ObjectPointsIndex + NumImagePoints));
			cv::Mat ImagePointsInImage = cv::Mat(ImagePointsMat.colRange(ObjectPointsIndex, ObjectPointsIndex + NumImagePoints));

			FTransform TargetPose = FTransform::Identity;
			if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::SolveTargetOffset))
			{
				if (TargetPoses.IsValidIndex(ImageIndex))
				{
					TargetPose = TargetPoses[ImageIndex];
				}
			}

			ObjectPointsIndex += NumImagePoints;

			int ExtrinsicOffset = 0;
			if (bConstrainWithCameraMotion)
			{
				ExtrinsicOffset = NumIntrinsics;
			}
			else
			{
				ExtrinsicOffset = NumIntrinsics + (CameraPoseIndices[ImageIndex] * NumExtrinsics);
			}

			cv::Mat Rotation;
			cv::Mat Translation;
			if (bConstrainWithCameraMotion)
			{
				// Transform the solver's current camera pose by the camera movement to get the pose for this image
				FTransform ImagePose = CurrentSolverPose * CameraMovements[ImageIndex];
				FOpenCVHelper::MakeObjectVectorsFromCameraPose(ImagePose, Rotation, Translation);
			}
			else
			{
				// Get the rotation and translation vectors for this image
				Rotation = Solver.Params.rowRange(ExtrinsicOffset, ExtrinsicOffset + 3);
				Translation = Solver.Params.rowRange(ExtrinsicOffset + 3, ExtrinsicOffset + 6);
			}

			Jacobian.resize(NumImagePoints * 2);
			Diffs.resize(NumImagePoints * 2);

			cv::Mat ProjectedPoints = cv::Mat(Diffs.reshape(2, 1));

			// Project the 3D object points to 2D using the camera extrinsics, intrinsics, and distortion parameters, optionally also solving for the Jacobian matrix
			if (bComputeJacobian)
			{
				ProjectPoints(LensModel, ObjectPointsInImage, TargetPose, TargetOffsetRotation, TargetOffsetTranslation, Rotation, Translation, CameraMatrix, DistCoeffs, CvImageSize, ProjectedPoints, Jacobian, SolverFlags);
			}
			else
			{
				ProjectPoints(LensModel, ObjectPointsInImage, TargetPose, TargetOffsetRotation, TargetOffsetTranslation, Rotation, Translation, CameraMatrix, DistCoeffs, CvImageSize, ProjectedPoints);
			}

			// Compute the difference between the input image points and the projected points
			ProjectedPoints = ProjectedPoints - ImagePointsInImage;

			// Compute the jacobian matrices used by the solver for its next update
			if (bComputeJacobian)
			{
				cv::Mat JacobianExtrinsics = Jacobian.colRange(0, NumExtrinsics);
				cv::Mat JacobianIntrinsics = Jacobian.colRange(NumExtrinsics, NumExtrinsics + NumIntrinsics);

				Solver.JacTJac(cv::Rect(0, 0, NumIntrinsics, NumIntrinsics)) += JacobianIntrinsics.t() * JacobianIntrinsics;
				Solver.JacTJac(cv::Rect(ExtrinsicOffset, ExtrinsicOffset, 6, 6)) += JacobianExtrinsics.t() * JacobianExtrinsics;
				Solver.JacTJac(cv::Rect(ExtrinsicOffset, 0, 6, NumIntrinsics)) += JacobianIntrinsics.t() * JacobianExtrinsics;

				Solver.JacTErr.rowRange(0, NumIntrinsics) += JacobianIntrinsics.t() * Diffs;
				Solver.JacTErr.rowRange(ExtrinsicOffset, ExtrinsicOffset + 6) += JacobianExtrinsics.t() * Diffs;
			}

			const double ViewError = cv::norm(Diffs, cv::NORM_L2SQR);

			ReprojectionError += ViewError;
		}

		// Update the solver's error with the latest reprojection error
		Solver.ErrorNorm = ReprojectionError;

		const double CurrentRMSE = FMath::Sqrt(ReprojectionError / NumTotalPoints);
		SetStatusText(FText::Format(LOCTEXT("ReprojectionError", "Reprojection Error: {0} pixels for loop {1}"), CurrentRMSE, LoopCounter));

		++LoopCounter;
	}

	RMSE = FMath::Sqrt(ReprojectionError / NumTotalPoints);

	// Set the output intrinsics and distortion parameters to the final values calculated by the solver
	Result.FocalLength.FxFy.X = CameraMatrix.at<double>(0, 0);
	Result.FocalLength.FxFy.Y = CameraMatrix.at<double>(1, 1);
	Result.ImageCenter.PrincipalPoint.X = CameraMatrix.at<double>(0, 2);
	Result.ImageCenter.PrincipalPoint.Y = CameraMatrix.at<double>(1, 2);

	Result.CameraPoses.Reserve(NumImages);
	for (int ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		// Get the rotation and translation vectors for this image
		int ExtrinsicOffset = NumIntrinsics + (CameraPoseIndices[ImageIndex] * NumExtrinsics);
		cv::Mat Rotation = Solver.Params.rowRange(ExtrinsicOffset, ExtrinsicOffset + 3);
		cv::Mat Translation = Solver.Params.rowRange(ExtrinsicOffset + 3, ExtrinsicOffset + 6);

		// Convert rotation and translation vectors to an FTransform in the UE coordinate space
		FTransform CameraPoseForImage;
		FOpenCVHelper::MakeCameraPoseFromObjectVectors(Rotation, Translation, CameraPoseForImage);

		Result.CameraPoses.Add(CameraPoseForImage);
	}

	// If the solver computed exactly one calibrated camera pose, we can compute a tracking offset based on the input camera pose
	if (UniqueCameraPoses.Num() == 1)
	{
		const FTransform& TrackedCameraPose = CameraPoses[0];
		const FTransform& CalibratedCameraPose = Result.CameraPoses[0];
		const FTransform TargetOffset = CalibratedCameraPose * TrackedCameraPose.Inverse();

		Result.NodalOffset.LocationOffset = TargetOffset.GetLocation();
		Result.NodalOffset.RotationOffset = TargetOffset.GetRotation();
	}

	if (LensModel == UAnamorphicLensModel::StaticClass())
	{
		for (int CoeffIndex = 0; CoeffIndex < NumDistortionCoefficients; ++CoeffIndex)
		{
			Result.Parameters.Parameters.Add(DistCoeffs.at<double>(CoeffIndex));
		}
	}
	else
	{
		// The spherical distortion coefficients in our model are in a different order than they appear in the solver, so the results need to be rearranged
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(0));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(1));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(4));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(2));
		Result.Parameters.Parameters.Add(DistCoeffs.at<double>(3));
	}

	Result.ReprojectionError = RMSE;

	return Result;
#else
	Result.ErrorMessage = LOCTEXT("OpenCVNotSupportedError", "OpenCV is not supported");
	return Result;
#endif // WITH_OPENCV
}

double FCameraCalibrationSolver::OptimizeNodalOffset(
	const TArray<TArray<FVector>>& InObjectPoints,
	const TArray<TArray<FVector2f>>& InImagePoints,
	const FVector2D& InFocalLength,
	const FVector2D& InImageCenter,
	const TArray<FTransform>& InCameraPoses,
	FTransform& InOutNodalOffset)
{
#if WITH_OPENCV
	const int32 NumViews = InObjectPoints.Num();
	if (NumViews < 1)
	{
		return -1.0;
	}

	cv::Ptr<cv::DownhillSolver> Solver = cv::DownhillSolver::create();

	const FQuat InitialRotation = InOutNodalOffset.GetRotation();
	const FVector InitialLocation = InOutNodalOffset.GetLocation();

	constexpr int32 NumRotationParameters = 4; // FQuat
	constexpr int32 NumLocationParameters = 3; // FVector
	cv::Mat WorkingSolution = cv::Mat(1, NumRotationParameters + NumLocationParameters, CV_64FC1);
	WorkingSolution.at<double>(0, 0) = InitialRotation.X;
	WorkingSolution.at<double>(0, 1) = InitialRotation.Y;
	WorkingSolution.at<double>(0, 2) = InitialRotation.Z;
	WorkingSolution.at<double>(0, 3) = InitialRotation.W;
	WorkingSolution.at<double>(0, 4) = InitialLocation.X;
	WorkingSolution.at<double>(0, 5) = InitialLocation.Y;
	WorkingSolution.at<double>(0, 6) = InitialLocation.Z;

	// NOTE: These step sizes may need further testing and refinement, but tests so far have shown them to be decent
	const double RotationStep = CVarRotationStepValue.GetValueOnGameThread();
	const double LocationStep = CVarLocationStepValue.GetValueOnGameThread();
	cv::Mat Step = cv::Mat(1, NumRotationParameters + NumLocationParameters, CV_64FC1);
	Step.at<double>(0, 0) = RotationStep;
	Step.at<double>(0, 1) = RotationStep;
	Step.at<double>(0, 2) = RotationStep;
	Step.at<double>(0, 3) = RotationStep;
	Step.at<double>(0, 4) = LocationStep;
	Step.at<double>(0, 5) = LocationStep;
	Step.at<double>(0, 6) = LocationStep;

	Solver->setInitStep(Step);

	cv::Ptr<FOptimizeNodalOffsetSolver> SolverFunction = cv::makePtr<FOptimizeNodalOffsetSolver>();
	Solver->setFunction(SolverFunction);

	SolverFunction->FocalLength = InFocalLength;
	SolverFunction->ImageCenter = InImageCenter;

	SolverFunction->CameraPoses.Reserve(NumViews);
	SolverFunction->Points3d.Reserve(NumViews);
	SolverFunction->Points2d.Reserve(NumViews);

	for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
	{
		SolverFunction->CameraPoses.Add(InCameraPoses[ViewIndex]);
		SolverFunction->Points3d.Add(InObjectPoints[ViewIndex]);
		SolverFunction->Points2d.Add(InImagePoints[ViewIndex]);
	}

	const double Error = Solver->minimize(WorkingSolution);

	const FQuat FinalRotation = FQuat(WorkingSolution.at<double>(0, 0), WorkingSolution.at<double>(0, 1), WorkingSolution.at<double>(0, 2), WorkingSolution.at<double>(0, 3)).GetNormalized();
	const FVector FinalLocation = FVector(WorkingSolution.at<double>(0, 4), WorkingSolution.at<double>(0, 5), WorkingSolution.at<double>(0, 6));

	InOutNodalOffset.SetRotation(FinalRotation);
	InOutNodalOffset.SetLocation(FinalLocation);

	return Error;
#else
	return -1.0;
#endif // WITH_OPENCV
}

#if WITH_OPENCV
void ULensDistortionSolverOpenCV::InitCameraIntrinsics(
	const cv::Mat& ObjectPoints,
	const cv::Mat& ImagePoints,
	const cv::Mat& NumPoints,
	const cv::Size ImageSize,
	cv::Mat& CameraMatrix,
	bool bUseCanonicalPlaneInit)
{	
	if (bUseCanonicalPlaneInit)
	{
		UE_LOGF(LogCameraCalibrationSolver, Verbose, "Using homography-based initialization for objects point in a canonical plane");
		InitIntrinsicsFromCanonicalPlane(ObjectPoints, ImagePoints, NumPoints, ImageSize, CameraMatrix);
	}
	else
	{
		UE_LOGF(LogCameraCalibrationSolver, Verbose, "Using 3D point initialization for object points in general world coordinates");
		InitIntrinsicsFrom3DPoints(ObjectPoints, ImagePoints, NumPoints, ImageSize, CameraMatrix);
	}
}

void ULensDistortionSolverOpenCV::InitIntrinsicsFromCanonicalPlane(
	const cv::Mat& ObjectPoints,
	const cv::Mat& ImagePoints,
	const cv::Mat& NumPoints,
	const cv::Size ImageSize,
	cv::Mat& CameraMatrix)
{
	// Use homography-based method for 2D plane points (checkerboard, dummy points)
	const int NumImages = NumPoints.cols;

	const double Cx = (ImageSize.width - 1) * 0.5;
	const double Cy = (ImageSize.height - 1) * 0.5;

	// Collect valid homography constraint rows to avoid uninitialized matrix data
	std::vector<double> ValidMatrixARows;
	std::vector<double> ValidVecBRows;
	ValidMatrixARows.reserve(2 * NumImages * 2); // Each image contributes 2 rows with 2 columns
	ValidVecBRows.reserve(2 * NumImages);

	// extract vanishing points in order to obtain initial value for the focal length
	int ObjectPointsIndex = 0;
	int ValidImageCount = 0;
	for (int ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		const int NumImagePoints = NumPoints.ptr<int>()[ImageIndex];

		const cv::Mat ObjectPointsInImage = ObjectPoints.colRange(ObjectPointsIndex, ObjectPointsIndex + NumImagePoints);
		const cv::Mat ImagePointsInImage = ImagePoints.colRange(ObjectPointsIndex, ObjectPointsIndex + NumImagePoints);

		ObjectPointsIndex += NumImagePoints;

		cv::Mat HomographyMat = cv::findHomography(ObjectPointsInImage, ImagePointsInImage);

		if (HomographyMat.empty() || HomographyMat.rows != 3 || HomographyMat.cols != 3 || HomographyMat.type() != CV_64F)
		{
			UE_LOGF(LogCameraCalibrationSolver, Warning, "findHomography failed or returned invalid matrix for image %d", ImageIndex);
			continue;
		}

		double* Homography = HomographyMat.ptr<double>();

		Homography[0] -= Homography[6] * Cx;
		Homography[1] -= Homography[7] * Cx;
		Homography[2] -= Homography[8] * Cx;
		Homography[3] -= Homography[6] * Cy;
		Homography[4] -= Homography[7] * Cy;
		Homography[5] -= Homography[8] * Cy;

		double H[3];
		double V[3];
		double D1[3];
		double D2[3];
		double N[4] = { 0 };

		for (int Index = 0; Index < 3; ++Index)
		{
			double T0 = Homography[Index * 3];
			double T1 = Homography[Index * 3 + 1];

			H[Index] = T0;
			V[Index] = T1;

			D1[Index] = (T0 + T1) * 0.5;
			D2[Index] = (T0 - T1) * 0.5;

			N[0] += H[Index] * H[Index];
			N[1] += V[Index] * V[Index];
			N[2] += D1[Index] * D1[Index];
			N[3] += D2[Index] * D2[Index];
		}

		N[0] = FMath::InvSqrt(N[0]);
		N[1] = FMath::InvSqrt(N[1]);
		N[2] = FMath::InvSqrt(N[2]);
		N[3] = FMath::InvSqrt(N[3]);

		for (int32 Index = 0; Index < 3; ++Index)
		{
			H[Index] *= N[0];
			V[Index] *= N[1];

			D1[Index] *= N[2];
			D2[Index] *= N[3];
		}

		// Store valid constraint rows
		ValidMatrixARows.push_back(H[0] * V[0]);
		ValidMatrixARows.push_back(H[1] * V[1]);
		ValidMatrixARows.push_back(D1[0] * D2[0]);
		ValidMatrixARows.push_back(D1[1] * D2[1]);

		ValidVecBRows.push_back(-H[2] * V[2]);
		ValidVecBRows.push_back(-D1[2] * D2[2]);

		ValidImageCount++;
	}

	// Check if we have enough valid homographies for focal length estimation
	if (ValidImageCount < 2)
	{
		UE_LOGF(LogCameraCalibrationSolver, Warning, "InitIntrinsicsFromCanonicalPlane: Insufficient valid homographies (%d) for focal length estimation. Need at least 2.", ValidImageCount);

		// Fallback to basic initialization
		const double Fx = ImageSize.width;
		const double Fy = ImageSize.width;

		CameraMatrix.at<double>(0, 0) = Fx;
		CameraMatrix.at<double>(1, 1) = Fy;
		CameraMatrix.at<double>(0, 2) = Cx;
		CameraMatrix.at<double>(1, 2) = Cy;
		return;
	}

	// Build matrices only with valid data
	cv::Mat MatrixA = cv::Mat(2 * ValidImageCount, 2, CV_64F, ValidMatrixARows.data());
	cv::Mat VecB = cv::Mat(2 * ValidImageCount, 1, CV_64F, ValidVecBRows.data());

	double FxFy[2] = { 0 };
	cv::Mat FxFyMat = cv::Mat(2, 1, CV_64F, FxFy);

	cv::solve(MatrixA, VecB, FxFyMat, cv::DECOMP_NORMAL + cv::DECOMP_SVD);

	constexpr double MinFocalLength = 1e-6;

	if (FxFy[0] < MinFocalLength || FxFy[1] < MinFocalLength)
	{
		UE_LOGF(LogCameraCalibrationSolver, Warning, "InitIntrinsicsFromPlane2d: FxFy did not have reasonable values: [%f, %f]. ",
			FxFy[0], FxFy[1]
		);

		FxFy[0] = FMath::Max(FxFy[0], MinFocalLength);
		FxFy[1] = FMath::Max(FxFy[1], MinFocalLength);
	}

	const double Fx = FMath::Sqrt(FMath::Abs(1.0 / FxFy[0]));
	const double Fy = FMath::Sqrt(FMath::Abs(1.0 / FxFy[1]));

	CameraMatrix.at<double>(0, 0) = Fx;
	CameraMatrix.at<double>(1, 1) = Fy;
	CameraMatrix.at<double>(0, 2) = Cx;
	CameraMatrix.at<double>(1, 2) = Cy;
}

void ULensDistortionSolverOpenCV::InitCameraExtrinsics(
	const TSubclassOf<ULensModel> LensModel,
	const cv::Mat& ObjectPoints,
	const cv::Mat& ImagePoints,
	const cv::Mat& CameraMatrix,
	const cv::Mat& DistCoeffs,
	const cv::Size ImageSize,
	cv::Mat& Rotation,
	cv::Mat& Translation,
	const ECalibrationFlags SolverFlags,
	bool bUseCanonicalPlaneInit)
{
	const int NumPoints = ObjectPoints.total();

	// Normalize image points (unapply the intrinsic matrix transformation)
	cv::Mat ZeroDistortion = cv::Mat::zeros(5, 1, CV_64F);
	cv::Mat UndistortedPoints = cv::Mat(1, NumPoints, CV_64FC2);
	cv::undistortPoints(ImagePoints, UndistortedPoints, CameraMatrix, ZeroDistortion);

	// Average the X, Y, and Z channels for every object point in the set
	cv::Mat ObjectPointAvg = cv::Mat(1, 3, CV_64F, cv::mean(ObjectPoints).val);

	// Reshape the ObjectPoints mat into a 1-channel (NumPoints x 3) matrix, where each row is [ X Y Z ]
	cv::Mat ObjectPoints3Column = ObjectPoints.reshape(1, NumPoints);

	// Multiply the ObjectPoints Transpose by itself, subtracting the average values that were computed
	constexpr bool bTransposeFirstMatrix = true;
	cv::Mat MM = cv::Mat(3, 3, CV_64F);

	cv::mulTransposed(ObjectPoints3Column, MM, bTransposeFirstMatrix, ObjectPointAvg);

	// Perform singular value decomposition on the resulting matrix
	double V[9] = { 0 };
	cv::Mat MatrixV = cv::Mat(3, 3, CV_64F, V);

	double W[3] = { 0 };
	cv::Mat MatrixW = cv::Mat(3, 1, CV_64F, W);

	cv::SVDecomp(MM, MatrixW, cv::noArray(), MatrixV, cv::SVD::MODIFY_A + cv::SVD::FULL_UV);

	// Use the same canonical plane detection as intrinsics initialization
	if (bUseCanonicalPlaneInit)
	{
		cv::Mat RotationMatrix = MatrixV;

		if ((V[2] * V[2] + V[5] * V[5]) < 1e-10)
		{
			RotationMatrix = cv::Mat::eye(3, 3, CV_64F);
		}

		if (cv::determinant(RotationMatrix) < 0)
		{
			RotationMatrix = RotationMatrix.mul(-1);
		}

		Translation = -RotationMatrix * ObjectPointAvg.t();

		cv::Mat TransformedPoints = cv::Mat(1, NumPoints, CV_64FC2);

		for (int PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			const cv::Point3d& ObjectPoint = ObjectPoints.at<cv::Point3d>(PointIndex);
			cv::Point2d& TransformedPoint = TransformedPoints.at<cv::Point2d>(PointIndex);

			const double* RPtr = RotationMatrix.ptr<double>();
			const double* TPtr = Translation.ptr<double>();

			TransformedPoint.x = (ObjectPoint.x * RPtr[0]) + (ObjectPoint.y * RPtr[1]) + (ObjectPoint.z * RPtr[2]) + TPtr[0];
			TransformedPoint.y = (ObjectPoint.x * RPtr[3]) + (ObjectPoint.y * RPtr[4]) + (ObjectPoint.z * RPtr[5]) + TPtr[1];
		}

		cv::Mat Homography = cv::findHomography(TransformedPoints, UndistortedPoints);

		if (cv::checkRange(Homography, true))
		{
			cv::Mat H1 = Homography.col(0);
			cv::Mat H2 = Homography.col(1);
			cv::Mat H3 = Homography.col(2);

			const double H1Norm = FMath::Max(cv::norm(H1), DBL_EPSILON);
			const double H2Norm = FMath::Max(cv::norm(H2), DBL_EPSILON);

			H1 = H1.mul(1.0 / H1Norm);
			H2 = H2.mul(1.0 / H2Norm);
			cv::Mat H3Norm = H3.mul(2.0 / (H1Norm + H2Norm));

			H3 = H1.cross(H2);

			RotationMatrix = Homography * RotationMatrix;

			Translation = Homography * Translation;
			Translation += H3Norm;
		}
		else
		{
			RotationMatrix = cv::Mat::eye(3, 3, CV_64F);
			Translation.setTo(cv::Scalar::all(0));
		}

		cv::Rodrigues(RotationMatrix, Rotation);
	}
	else // Case of not being in a 2d canonical plane
	{
		UE_LOGF(LogCameraCalibrationSolver, Verbose, "Initializing extrinsics for non-planar 3D points using solvePnP");
		
		// For 3D world coordinates, use cv::solvePnP to get initial pose estimate
		// Convert ObjectPoints format: from (3 x NumPoints) to (NumPoints x 3)
		cv::Mat ObjectPoints3x1 = ObjectPoints.reshape(1, NumPoints);
		
		// Convert ImagePoints format: from (1 x NumPoints, 2-channel) to (NumPoints x 1, 2-channel)  
		cv::Mat ImagePoints2x1 = ImagePoints.reshape(2, NumPoints);
		
		try
		{
			bool bSolveSuccess = cv::solvePnP(
				ObjectPoints3x1,    // (NumPoints x 3) object points
				ImagePoints2x1,     // (NumPoints x 1, 2-channel) image points
				CameraMatrix,       // 3x3 camera matrix
				cv::Mat::zeros(5, 1, CV_64F), // No distortion for initial estimate
				Rotation,           // Output: 3x1 rotation vector
				Translation,        // Output: 3x1 translation vector
				false,              // useExtrinsicGuess = false
				cv::SOLVEPNP_ITERATIVE
			);
			
			if (!bSolveSuccess)
			{
				UE_LOGF(LogCameraCalibrationSolver, Warning, "solvePnP failed, using identity pose");
				cv::Mat RotationMatrix = cv::Mat::eye(3, 3, CV_64F);
				cv::Rodrigues(RotationMatrix, Rotation);
				Translation.setTo(cv::Scalar::all(0));
			}
		}
		catch (const cv::Exception& e)
		{
			UE_LOGF(LogCameraCalibrationSolver, Warning, "solvePnP exception: %ls, using identity pose", ANSI_TO_TCHAR(e.what()));
			cv::Mat RotationMatrix = cv::Mat::eye(3, 3, CV_64F);
			cv::Rodrigues(RotationMatrix, Rotation);
			Translation.setTo(cv::Scalar::all(0));
		}
	}

	// Refine extrinsic parameters
	constexpr int NumExtrinsicParams = 6;
	const int NumErrors = NumPoints * 2;
	constexpr int MaxIterations = 30;

	FLevMarqSolver Solver(NumExtrinsicParams, NumErrors, MaxIterations, FLevMarqSolver::ESymmetryMode::UpperToLower);

	double* SolverParams = Solver.Params.ptr<double>();

	SolverParams[0] = Rotation.at<double>(0);
	SolverParams[1] = Rotation.at<double>(1);
	SolverParams[2] = Rotation.at<double>(2);

	SolverParams[3] = Translation.at<double>(0);
	SolverParams[4] = Translation.at<double>(1);
	SolverParams[5] = Translation.at<double>(2);

	while (true)
	{
		bool bShouldProceed = Solver.Update();
		bool bComputeJacobian = Solver.State == FLevMarqSolver::ESolverState::ComputeJacobian;

		Rotation.at<double>(0) = SolverParams[0];
		Rotation.at<double>(1) = SolverParams[1];
		Rotation.at<double>(2) = SolverParams[2];

		Translation.at<double>(0) = SolverParams[3];
		Translation.at<double>(1) = SolverParams[4];
		Translation.at<double>(2) = SolverParams[5];

		if (!bShouldProceed)
		{
			break;
		}

		cv::Mat ProjectedPoints = Solver.Error.reshape(2, 1);

		// When initializing the extrinsics, the target offset is not used
		FTransform TargetPose = FTransform::Identity;
		cv::Mat TargetOffsetRotation = cv::Mat::zeros(3, 1, CV_64F);
		cv::Mat TargetOffsetTranslation = cv::Mat::zeros(3, 1, CV_64F);

		if (bComputeJacobian)
		{
			ProjectPoints(LensModel, ObjectPoints, TargetPose, TargetOffsetRotation, TargetOffsetTranslation, Rotation, Translation, CameraMatrix, DistCoeffs, ImageSize, ProjectedPoints, Solver.Jacobian, SolverFlags);
		}
		else
		{
			ProjectPoints(LensModel, ObjectPoints, TargetPose, TargetOffsetRotation, TargetOffsetTranslation, Rotation, Translation, CameraMatrix, DistCoeffs, ImageSize, ProjectedPoints);
		}

		ProjectedPoints = ProjectedPoints - ImagePoints;
	}
}

void ULensDistortionSolverOpenCV::ProjectPoints(
	const TSubclassOf<ULensModel> LensModel,
	const cv::Mat& ObjectPoints,
	const FTransform& TargetPose,
	const cv::Mat& TargetOffsetRotation,
	const cv::Mat& TargetOffsetTranslation,
	const cv::Mat& Rotation,
	const cv::Mat& Translation,
	const cv::Mat& CameraMatrix,
	const cv::Mat& DistCoeffs,
	const cv::Size ImageSize,
	cv::Mat& ProjectedPoints)
{
	cv::Mat Jacobian;
	const ECalibrationFlags SolverFlags = ECalibrationFlags::None;
	ProjectPoints(LensModel, ObjectPoints, TargetPose, TargetOffsetRotation, TargetOffsetTranslation, Rotation, Translation, CameraMatrix, DistCoeffs, ImageSize, ProjectedPoints, Jacobian, SolverFlags);
}

void ULensDistortionSolverOpenCV::ProjectPoints(
	const TSubclassOf<ULensModel> LensModel,
	const cv::Mat& ObjectPoints,
	const FTransform& TargetPose,
	const cv::Mat& TargetOffsetRotation,
	const cv::Mat& TargetOffsetTranslation,
	const cv::Mat& Rotation,
	const cv::Mat& Translation,
	const cv::Mat& CameraMatrix,
	const cv::Mat& DistCoeffs,
	const cv::Size ImageSize,
	cv::Mat& ProjectedPoints,
	cv::Mat& Jacobian,
	ECalibrationFlags SolverFlags)
{
	// Project the 3D points to 2D using the transformation matrix, camera matrix, and distortion equation
	if (LensModel == USphericalLensModel::StaticClass())
	{
		ProjectPointsSpherical(
			ObjectPoints,
			TargetPose,
			TargetOffsetRotation,
			TargetOffsetTranslation,
			Rotation,
			Translation,
			CameraMatrix,
			DistCoeffs,
			ProjectedPoints,
			Jacobian,
			SolverFlags);
	}
	else if (LensModel == UAnamorphicLensModel::StaticClass())
	{
		ProjectPointsAnamorphic(
			ObjectPoints,
			Rotation,
			Translation,
			CameraMatrix,
			DistCoeffs,
			ImageSize,
			ProjectedPoints,
			Jacobian,
			SolverFlags);
	}
}

void ULensDistortionSolverOpenCV::ProjectPointsAnamorphic(
	const cv::Mat& ObjectPoints,
	const cv::Mat& Rotation,
	const cv::Mat& Translation,
	const cv::Mat& CameraMatrix,
	const cv::Mat& DistCoeffs,
	const cv::Size ImageSize,
	cv::Mat& ProjectedPoints,
	cv::Mat& Jacobian,
	ECalibrationFlags SolverFlags)
{
	const bool bComputeJacobian = !Jacobian.empty();

	// Get the rotation vector/matrix into the correct format and also compute the matrix of partial derivatives
	double R[9];
	cv::Mat RotationMatrix = cv::Mat(3, 3, CV_64F, R);

	double JacR[27];
	cv::Mat RotationJacobian = cv::Mat(3, 9, CV_64F, JacR);

	cv::Rodrigues(Rotation, RotationMatrix, RotationJacobian);

	const double* TranslationVector = Translation.ptr<double>();

	// Extract Camera Intrinsics
	const double Fx = CameraMatrix.at<double>(0, 0);
	const double Fy = CameraMatrix.at<double>(1, 1);
	const double Cx = CameraMatrix.at<double>(0, 2);
	const double Cy = CameraMatrix.at<double>(1, 2);

	// Extract Distortion Parameters
	const double* DistortionParams = DistCoeffs.ptr<double>();

	const double PixelAspect = DistortionParams[0];
	const double CX02 = DistortionParams[1];
	const double CX04 = DistortionParams[2];
	const double CX22 = DistortionParams[3];
	const double CX24 = DistortionParams[4];
	const double CX44 = DistortionParams[5];
	const double CY02 = DistortionParams[6];
	const double CY04 = DistortionParams[7];
	const double CY22 = DistortionParams[8];
	const double CY24 = DistortionParams[9];
	const double CY44 = DistortionParams[10];
	const double SqueezeX = DistortionParams[11];
	const double SqueezeY = DistortionParams[12];
	const double LensRotationDeg = DistortionParams[13];

	// Pre-compute values that will be re-used in the distortion equation for every point
	const double LensRotationRad = LensRotationDeg * (PI / 180.0);
	const double CosLensRotation = FMath::Cos(LensRotationRad);
	const double SinLensRotation = FMath::Sin(LensRotationRad);

	const double FilmbackWidth = ImageSize.width;
	const double FilmbackHeight = ImageSize.height;

	const double FilmbackRadius = (0.5) * FMath::Sqrt((FilmbackWidth * FilmbackWidth) + (FilmbackHeight * FilmbackHeight));

	const int NumPoints = ObjectPoints.total();

	for (int PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		const cv::Point3d& WorldPoint = ObjectPoints.at<cv::Point3d>(PointIndex);
		cv::Point2d& ProjectedPoint = ProjectedPoints.at<cv::Point2d>(PointIndex);

		// Transform the 3D point from the world coordinate system to our camera coordinate system
		cv::Point3d CameraPoint;
		CameraPoint.x = (WorldPoint.x * R[0]) + (WorldPoint.y * R[1]) + (WorldPoint.z * R[2]) + TranslationVector[0];
		CameraPoint.y = (WorldPoint.x * R[3]) + (WorldPoint.y * R[4]) + (WorldPoint.z * R[5]) + TranslationVector[1];
		CameraPoint.z = (WorldPoint.x * R[6]) + (WorldPoint.y * R[7]) + (WorldPoint.z * R[8]) + TranslationVector[2];

		// Divide x and y by the z-coordinate to normalize the point before projection
		CameraPoint.z = CameraPoint.z ? (1.0 / CameraPoint.z) : 1.0;

		CameraPoint.x *= CameraPoint.z;
		CameraPoint.y *= CameraPoint.z;

		// Project the point onto the image plane using our camera intrinsics
		cv::Point2d ImagePoint;
		ImagePoint.x = CameraPoint.x * Fx + Cx;
		ImagePoint.y = CameraPoint.y * Fy + Cy;

		// Compute the diagonally normalized point using the image center and the filmback radius
		cv::Point2d DiagNormPoint;
		DiagNormPoint.x = (ImagePoint.x - Cx) / FilmbackRadius;
		DiagNormPoint.y = (ImagePoint.y - Cy) / FilmbackRadius;

		// Rotate the point using the lens rotation model parameter
		cv::Point2d RotatedPoint;
		RotatedPoint.x = (DiagNormPoint.x * CosLensRotation) + (DiagNormPoint.y * SinLensRotation);
		RotatedPoint.y = (DiagNormPoint.y * CosLensRotation) - (DiagNormPoint.x * SinLensRotation);

		// Divide the x-coordinate by the pixel aspect
		RotatedPoint.x = RotatedPoint.x / PixelAspect;

		// The anamorphic model is intended to undistort points. Here we use it to distort points, so we run an iterative solution to reverse the model
		cv::Point2d DistortedPoint;
		DistortedPoint.x = RotatedPoint.x;
		DistortedPoint.y = RotatedPoint.y;

		double RadiusSq = 0.0;
		double Radius4th = 0.0;
		double Angle = 0.0;
		double DistortionX = 0.0;
		double DistortionY = 0.0;
		double CosAngle2 = 0.0;
		double CosAngle4 = 0.0;
		for (int Index = 0; Index < 10; ++Index)
		{
			// Compute the polar coordinates (radius and angle) for the current point
			RadiusSq = (DistortedPoint.x * DistortedPoint.x) + (DistortedPoint.y * DistortedPoint.y);
			Radius4th = RadiusSq * RadiusSq;
			Angle = FMath::Atan2(DistortedPoint.y, DistortedPoint.x);
			CosAngle2 = FMath::Cos(Angle * 2);
			CosAngle4 = FMath::Cos(Angle * 4);

			// Compute the distortion contribution in the x and y directions
			DistortionX = 1 + (CX02 * RadiusSq) + (CX04 * Radius4th) + (CX22 * RadiusSq * CosAngle2) + (CX24 * Radius4th * CosAngle2) + (CX44 * Radius4th * CosAngle4);
			DistortionY = 1 + (CY02 * RadiusSq) + (CY04 * Radius4th) + (CY22 * RadiusSq * CosAngle2) + (CY24 * Radius4th * CosAngle2) + (CY44 * Radius4th * CosAngle4);

			// Distort the undistorted point
			DistortedPoint.x = (RotatedPoint.x / DistortionX) / SqueezeX;
			DistortedPoint.y = (RotatedPoint.y / DistortionY) / SqueezeY;
		}

		// Undo the model's lens rotation that we applied earlier
		cv::Point2d DistortedPointRot;
		DistortedPointRot.x = (DistortedPoint.x * CosLensRotation) - (DistortedPoint.y * SinLensRotation);
		DistortedPointRot.y = (DistortedPoint.y * CosLensRotation) + (DistortedPoint.x * SinLensRotation);

		// Re-multiply the x-coordinate by the pixel aspect
		DistortedPointRot.x = DistortedPointRot.x * PixelAspect;

		// Go back from diagonally normalized coordinates to pixels
		ProjectedPoint.x = (DistortedPointRot.x * FilmbackRadius) + Cx;
		ProjectedPoint.y = (DistortedPointRot.y * FilmbackRadius) + Cy;

		if (bComputeJacobian)
		{
			const int JacobianStep = Jacobian.cols;

			cv::Mat JacRotation;
			cv::Mat JacTranslation;
			cv::Mat JacFocalLength;
			cv::Mat JacImageCenter;
			cv::Mat JacTargetOffsetRotation;
			cv::Mat JacTargetOffsetTranslation;
			cv::Mat JacDistortion;
			SubdivideJacobian(Jacobian, JacRotation, JacTranslation, JacFocalLength, JacImageCenter, JacTargetOffsetRotation, JacTargetOffsetTranslation, JacDistortion, SolverFlags);

			if (!JacDistortion.empty())
			{
				const double DDistortedPointXDCX02 = (RotatedPoint.x * (-1.0 / (DistortionX * DistortionX)) * (RadiusSq)) / SqueezeX;
				const double DDistortedPointXDCX04 = (RotatedPoint.x * (-1.0 / (DistortionX * DistortionX)) * (Radius4th)) / SqueezeX;
				const double DDistortedPointXDCX22 = (RotatedPoint.x * (-1.0 / (DistortionX * DistortionX)) * (RadiusSq * CosAngle2)) / SqueezeX;
				const double DDistortedPointXDCX24 = (RotatedPoint.x * (-1.0 / (DistortionX * DistortionX)) * (Radius4th * CosAngle2)) / SqueezeX;
				const double DDistortedPointXDCX44 = (RotatedPoint.x * (-1.0 / (DistortionX * DistortionX)) * (Radius4th * CosAngle4)) / SqueezeX;

				const double DDistortedPointYDCY02 = (RotatedPoint.y * (-1.0 / (DistortionY * DistortionY)) * (RadiusSq)) / SqueezeY;
				const double DDistortedPointYDCY04 = (RotatedPoint.y * (-1.0 / (DistortionY * DistortionY)) * (Radius4th)) / SqueezeY;
				const double DDistortedPointYDCY22 = (RotatedPoint.y * (-1.0 / (DistortionY * DistortionY)) * (RadiusSq * CosAngle2)) / SqueezeY;
				const double DDistortedPointYDCY24 = (RotatedPoint.y * (-1.0 / (DistortionY * DistortionY)) * (Radius4th * CosAngle2)) / SqueezeY;
				const double DDistortedPointYDCY44 = (RotatedPoint.y * (-1.0 / (DistortionY * DistortionY)) * (Radius4th * CosAngle4)) / SqueezeY;

				double DDistortedPointXDSX = -(RotatedPoint.x / DistortionX) / (SqueezeX * SqueezeX);
				double DDistortedPointYDSY = -(RotatedPoint.y / DistortionY) / (SqueezeY * SqueezeY);

				double DRotatedPointXDRot = ((DiagNormPoint.x * -SinLensRotation) + (DiagNormPoint.y * CosLensRotation)) * (PI / 180.0) / PixelAspect;
				double DRotatedPointYDRot = ((DiagNormPoint.y * -SinLensRotation) - (DiagNormPoint.x * CosLensRotation)) * (PI / 180.0);

				double DDistortedPointRotXDRot = ((DRotatedPointXDRot * CosLensRotation - SinLensRotation * (PI / 180.0) * DistortedPoint.x) - (DRotatedPointYDRot * SinLensRotation + CosLensRotation * (PI / 180.0) * DistortedPoint.y)) * PixelAspect;
				double DDistortedPointRotYDRot = (DRotatedPointYDRot * CosLensRotation - SinLensRotation * (PI / 180.0) * DistortedPoint.y) + (DRotatedPointXDRot * SinLensRotation + CosLensRotation * (PI / 180.0) * DistortedPoint.x);

				double* JacDistortionPtr = JacDistortion.ptr<double>(PointIndex * 2);

				JacDistortionPtr[0] = 0;
				JacDistortionPtr[1] = FilmbackRadius * CosLensRotation * PixelAspect * DDistortedPointXDCX02;
				JacDistortionPtr[2] = FilmbackRadius * CosLensRotation * PixelAspect * DDistortedPointXDCX04;
				JacDistortionPtr[3] = FilmbackRadius * CosLensRotation * PixelAspect * DDistortedPointXDCX22;
				JacDistortionPtr[4] = FilmbackRadius * CosLensRotation * PixelAspect * DDistortedPointXDCX24;
				JacDistortionPtr[5] = FilmbackRadius * CosLensRotation * PixelAspect * DDistortedPointXDCX44;
				JacDistortionPtr[6] = 0;
				JacDistortionPtr[7] = 0;
				JacDistortionPtr[8] = 0;
				JacDistortionPtr[9] = 0;
				JacDistortionPtr[10] = 0;
				JacDistortionPtr[11] = FilmbackRadius * CosLensRotation * PixelAspect * DDistortedPointXDSX;
				JacDistortionPtr[12] = 0;
				JacDistortionPtr[13] = FilmbackRadius * DDistortedPointRotXDRot;

				JacDistortionPtr[JacobianStep + 0] = 0;
				JacDistortionPtr[JacobianStep + 1] = 0;
				JacDistortionPtr[JacobianStep + 2] = 0;
				JacDistortionPtr[JacobianStep + 3] = 0;
				JacDistortionPtr[JacobianStep + 4] = 0;
				JacDistortionPtr[JacobianStep + 5] = 0;
				JacDistortionPtr[JacobianStep + 6] = FilmbackRadius * CosLensRotation * DDistortedPointYDCY02;
				JacDistortionPtr[JacobianStep + 7] = FilmbackRadius * CosLensRotation * DDistortedPointYDCY04;
				JacDistortionPtr[JacobianStep + 8] = FilmbackRadius * CosLensRotation * DDistortedPointYDCY22;
				JacDistortionPtr[JacobianStep + 9] = FilmbackRadius * CosLensRotation * DDistortedPointYDCY24;
				JacDistortionPtr[JacobianStep + 10] = FilmbackRadius * CosLensRotation * DDistortedPointYDCY44;
				JacDistortionPtr[JacobianStep + 11] = 0;
				JacDistortionPtr[JacobianStep + 12] = FilmbackRadius * CosLensRotation * DDistortedPointYDSY;
				JacDistortionPtr[JacobianStep + 13] = FilmbackRadius * DDistortedPointRotYDRot;
			}

			if (!JacImageCenter.empty())
			{
				double* JacImageCenterPtr = JacImageCenter.ptr<double>(PointIndex * 2);

				JacImageCenterPtr[0] = 1.0;
				JacImageCenterPtr[1] = 0.0;

				JacImageCenterPtr[JacobianStep + 0] = 0.0;
				JacImageCenterPtr[JacobianStep + 1] = 1.0;
			}

			if (!JacFocalLength.empty())
			{
				// Compute the derivative of the undistorted image point with respect to Fx and Fy
				const cv::Point2d DImagePointXDFxFy = { CameraPoint.x, 0.0 };
				const cv::Point2d DImagePointYDFxFy = { 0.0, CameraPoint.y };

				// Compute the derivative of the diagonally normalized point with respect to Fx and Fy
				const cv::Point2d DDiagNormPointXDFxFy = (1.0 / FilmbackRadius) * DImagePointXDFxFy;
				const cv::Point2d DDiagNormPointYDFxFy = (1.0 / FilmbackRadius) * DImagePointYDFxFy;

				// Compute the derivative of the rotated point with respect to Fx and Fy
				const cv::Point2d DRotatedPointXDFxFy = (DDiagNormPointXDFxFy * CosLensRotation + DDiagNormPointYDFxFy * SinLensRotation) / PixelAspect;
				const cv::Point2d DRotatedPointYDFxFy = (DDiagNormPointYDFxFy * CosLensRotation + DDiagNormPointXDFxFy * SinLensRotation);

				// Compute the derivative of the distorted point with respect to Fx and Fy
				const cv::Point2d DDistortedPointXDFxFy = (DRotatedPointXDFxFy * CosLensRotation - DRotatedPointYDFxFy * SinLensRotation) * PixelAspect;
				const cv::Point2d DDistortedPointYDFxFy = (DRotatedPointYDFxFy * CosLensRotation + DRotatedPointXDFxFy * SinLensRotation);
				
				// Add derivatives to the Jacobian matrix
				double* JacFocalLengthPtr = JacFocalLength.ptr<double>(PointIndex * 2);

				JacFocalLengthPtr[0] = FilmbackRadius * DDistortedPointXDFxFy.x;
				JacFocalLengthPtr[1] = FilmbackRadius * DDistortedPointXDFxFy.y;

				JacFocalLengthPtr[JacobianStep + 0] = FilmbackRadius * DDistortedPointYDFxFy.x;
				JacFocalLengthPtr[JacobianStep + 1] = FilmbackRadius * DDistortedPointYDFxFy.y;
			}

			if (!JacTranslation.empty())
			{
				// Compute the derivative of the undistorted image point with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DImagePointXDT = { Fx * CameraPoint.z, 0.0,                Fx * (-CameraPoint.x * CameraPoint.z) };
				const cv::Point3d DImagePointYDT = { 0.0,                Fy * CameraPoint.z, Fy * (-CameraPoint.y * CameraPoint.z) };

				// Compute the derivative of the diagonally normalized point with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DDiagNormPointXDT = (1.0 / FilmbackRadius) * DImagePointXDT;
				const cv::Point3d DDiagNormPointYDT = (1.0 / FilmbackRadius) * DImagePointYDT;

				// Compute the derivative of the rotated point with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DRotatedPointXDT = (DDiagNormPointXDT * CosLensRotation + DDiagNormPointYDT * SinLensRotation) / PixelAspect;
				const cv::Point3d DRotatedPointYDT = (DDiagNormPointYDT * CosLensRotation + DDiagNormPointXDT * SinLensRotation);

				// Compute the derivative of the distorted point with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DDistortedPointXDT = (DRotatedPointXDT * CosLensRotation - DRotatedPointYDT * SinLensRotation) * PixelAspect;
				const cv::Point3d DDistortedPointYDT = (DRotatedPointYDT * CosLensRotation + DRotatedPointXDT * SinLensRotation);

				// Add derivatives to the Jacobian matrix
				double* JacTranslationPtr = JacTranslation.ptr<double>(PointIndex * 2);

				JacTranslationPtr[0] = FilmbackRadius * DDistortedPointXDT.x;
				JacTranslationPtr[1] = FilmbackRadius * DDistortedPointXDT.y;
				JacTranslationPtr[2] = FilmbackRadius * DDistortedPointXDT.z;

				JacTranslationPtr[JacobianStep + 0] = FilmbackRadius * DDistortedPointYDT.x;
				JacTranslationPtr[JacobianStep + 1] = FilmbackRadius * DDistortedPointYDT.y;
				JacTranslationPtr[JacobianStep + 2] = FilmbackRadius * DDistortedPointYDT.z;
			}

			if (!JacRotation.empty())
			{
				// Compute the derivatives of the camera points with respect to the rotation vector [ R0  R1  R2]
				// This requires the use of the rotation jacobian matrix that relates the rotation vector in Rodrigues form to the rotation matrix
				cv::Point3d DCameraPointXDR;
				DCameraPointXDR.x = WorldPoint.x * JacR[0] + WorldPoint.y * JacR[1] + WorldPoint.z * JacR[2];
				DCameraPointXDR.y = WorldPoint.x * JacR[9] + WorldPoint.y * JacR[10] + WorldPoint.z * JacR[11];
				DCameraPointXDR.z = WorldPoint.x * JacR[18] + WorldPoint.y * JacR[19] + WorldPoint.z * JacR[20];

				cv::Point3d DCameraPointYDR;
				DCameraPointYDR.x = WorldPoint.x * JacR[3] + WorldPoint.y * JacR[4] + WorldPoint.z * JacR[5];
				DCameraPointYDR.y = WorldPoint.x* JacR[12] + WorldPoint.y * JacR[13] + WorldPoint.z * JacR[14];
				DCameraPointYDR.z = WorldPoint.x* JacR[21] + WorldPoint.y * JacR[22] + WorldPoint.z * JacR[23];

				cv::Point3d DCameraPointZDR;
				DCameraPointZDR.x = WorldPoint.x* JacR[6] + WorldPoint.y * JacR[7] + WorldPoint.z * JacR[8];
				DCameraPointZDR.y = WorldPoint.x* JacR[15] + WorldPoint.y * JacR[16] + WorldPoint.z * JacR[17];
				DCameraPointZDR.z = WorldPoint.x* JacR[24] + WorldPoint.y * JacR[25] + WorldPoint.z * JacR[26];

				// Compute the derivative of the undistorted image point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DImagePointXDR = Fx * CameraPoint.z * (DCameraPointXDR - CameraPoint.x * DCameraPointZDR);
				const cv::Point3d DImagePointYDR = Fy * CameraPoint.z * (DCameraPointYDR - CameraPoint.y * DCameraPointZDR);

				// Compute the derivative of the diagonally normalized point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DDiagNormPointXDR = (1.0 / FilmbackRadius) * DImagePointXDR;
				const cv::Point3d DDiagNormPointYDR = (1.0 / FilmbackRadius) * DImagePointYDR;

				// Compute the derivative of the rotated point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DRotatedPointXDR = (DDiagNormPointXDR * CosLensRotation + DDiagNormPointYDR * SinLensRotation) / PixelAspect;
				const cv::Point3d DRotatedPointYDR = (DDiagNormPointYDR * CosLensRotation + DDiagNormPointXDR * SinLensRotation);

				// Compute the derivative of the distorted point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DDistortedPointXDR = (DRotatedPointXDR * CosLensRotation - DRotatedPointYDR * SinLensRotation) * PixelAspect;
				const cv::Point3d DDistortedPointYDR = (DRotatedPointYDR * CosLensRotation + DRotatedPointXDR * SinLensRotation);

				// Add derivatives to the Jacobian matrix
				double* JacRotationPtr = JacRotation.ptr<double>(PointIndex * 2);

				JacRotationPtr[0] = FilmbackRadius * DDistortedPointXDR.x;
				JacRotationPtr[1] = FilmbackRadius * DDistortedPointXDR.y;
				JacRotationPtr[2] = FilmbackRadius * DDistortedPointXDR.z;

				JacRotationPtr[JacobianStep + 0] = FilmbackRadius * DDistortedPointYDR.x;
				JacRotationPtr[JacobianStep + 1] = FilmbackRadius * DDistortedPointYDR.y;
				JacRotationPtr[JacobianStep + 2] = FilmbackRadius * DDistortedPointYDR.z;
			}
		}
	}
}

void ULensDistortionSolverOpenCV::ProjectPointsSpherical(
	const cv::Mat& ObjectPoints,
	const FTransform& TargetPose,
	const cv::Mat& TargetOffsetRotation,
	const cv::Mat& TargetOffsetTranslation,
	const cv::Mat& Rotation,
	const cv::Mat& Translation,
	const cv::Mat& CameraMatrix,
	const cv::Mat& DistCoeffs,
	cv::Mat& ProjectedPoints,
	cv::Mat& Jacobian,
	ECalibrationFlags SolverFlags)
{
	const bool bComputeJacobian = !Jacobian.empty();

	// Get the rotation vector/matrix into the correct format and also compute the matrix of partial derivatives
	double R[9];
	cv::Mat RotationMatrix = cv::Mat(3, 3, CV_64F, R);

	double JacR[27];
	cv::Mat RotationJacobian = cv::Mat(3, 9, CV_64F, JacR);

	cv::Rodrigues(Rotation, RotationMatrix, RotationJacobian);

	const double* TranslationVector = Translation.ptr<double>();

	// Extract Camera Intrinsics
	const double Fx = CameraMatrix.at<double>(0, 0);
	const double Fy = CameraMatrix.at<double>(1, 1);
	const double Cx = CameraMatrix.at<double>(0, 2);
	const double Cy = CameraMatrix.at<double>(1, 2);

	// Extract Distortion Parameters
	const double* DistortionParams = DistCoeffs.ptr<double>();
	const double K1 = DistortionParams[0];
	const double K2 = DistortionParams[1];
	const double P1 = DistortionParams[2];
	const double P2 = DistortionParams[3];
	const double K3 = DistortionParams[4];

	// Convert the target offset rvec into rodrigues format
	double TargetOffsetR[9];
	cv::Mat TargetOffsetRotationMatrix = cv::Mat(3, 3, CV_64F, TargetOffsetR);

	double TargetOffsetJacR[27];
	cv::Mat TargetOffsetRotationJacobian = cv::Mat(3, 9, CV_64F, TargetOffsetJacR);

	cv::Rodrigues(TargetOffsetRotation, TargetOffsetRotationMatrix, TargetOffsetRotationJacobian);

	const double* TargetOffsetT = TargetOffsetTranslation.ptr<double>();

	// Convert the target pose transform into rvecs and tvecs
	cv::Mat TargetPoseRvec;
	cv::Mat TargetPoseTvec;
	FOpenCVHelper::MakeObjectVectorsFromCameraPose(TargetPose, TargetPoseRvec, TargetPoseTvec);

	// Convert the target pose rvec into rodrigues format
	double TrackerR[9];
	cv::Mat TrackerRotationMatrix = cv::Mat(3, 3, CV_64F, TrackerR);

	double TrackerJacR[27];
	cv::Mat TrackerRotationJacobian = cv::Mat(3, 9, CV_64F, TrackerJacR);

	cv::Rodrigues(TargetPoseRvec, TrackerRotationMatrix, TrackerRotationJacobian);

	const double* TrackerT = TargetPoseTvec.ptr<double>();

	// Convert the inverse target pose transform into rvecs and tvecs
	cv::Mat InvTargetPoseRvec;
	cv::Mat InvTargetPoseTvec;
	FOpenCVHelper::MakeObjectVectorsFromCameraPose(TargetPose.Inverse(), InvTargetPoseRvec, InvTargetPoseTvec);

	// Convert the inverse target pose rvec into rodrigues format
	double InvTrackerR[9];
	cv::Mat InvTrackerRotationMatrix = cv::Mat(3, 3, CV_64F, InvTrackerR);

	double InvTrackerJacR[27];
	cv::Mat InvTrackerRotationJacobian = cv::Mat(3, 9, CV_64F, InvTrackerJacR);

	cv::Rodrigues(InvTargetPoseRvec, InvTrackerRotationMatrix, InvTrackerRotationJacobian);

	const double* InvTrackerT = InvTargetPoseTvec.ptr<double>();

	const int NumPoints = ObjectPoints.total();

	for (int PointIndex = 0; PointIndex < NumPoints; PointIndex++)
	{
		const cv::Point3d& InputPoint = ObjectPoints.at<cv::Point3d>(PointIndex);

		// Transform the input point by the original target pose
		cv::Point3d TargetPoint;
		TargetPoint.x = TrackerR[0] * InputPoint.x + TrackerR[1] * InputPoint.y + TrackerR[2] * InputPoint.z + TrackerT[0];
		TargetPoint.y = TrackerR[3] * InputPoint.x + TrackerR[4] * InputPoint.y + TrackerR[5] * InputPoint.z + TrackerT[1];
		TargetPoint.z = TrackerR[6] * InputPoint.x + TrackerR[7] * InputPoint.y + TrackerR[8] * InputPoint.z + TrackerT[2];

		// Transform the input point by the target offset
		cv::Point3d OffsetPoint;
		OffsetPoint.x = TargetOffsetR[0] * TargetPoint.x + TargetOffsetR[1] * TargetPoint.y + TargetOffsetR[2] * TargetPoint.z + TargetOffsetT[0];
		OffsetPoint.y = TargetOffsetR[3] * TargetPoint.x + TargetOffsetR[4] * TargetPoint.y + TargetOffsetR[5] * TargetPoint.z + TargetOffsetT[1];
		OffsetPoint.z = TargetOffsetR[6] * TargetPoint.x + TargetOffsetR[7] * TargetPoint.y + TargetOffsetR[8] * TargetPoint.z + TargetOffsetT[2];

		// Transform the input point by the inverse of the target pose
		cv::Point3d WorldPoint;
		WorldPoint.x = InvTrackerR[0] * OffsetPoint.x + InvTrackerR[1] * OffsetPoint.y + InvTrackerR[2] * OffsetPoint.z + InvTrackerT[0];
		WorldPoint.y = InvTrackerR[3] * OffsetPoint.x + InvTrackerR[4] * OffsetPoint.y + InvTrackerR[5] * OffsetPoint.z + InvTrackerT[1];
		WorldPoint.z = InvTrackerR[6] * OffsetPoint.x + InvTrackerR[7] * OffsetPoint.y + InvTrackerR[8] * OffsetPoint.z + InvTrackerT[2];
		
		cv::Point3d CameraPoint;
		CameraPoint.x = R[0] * WorldPoint.x + R[1] * WorldPoint.y + R[2] * WorldPoint.z + TranslationVector[0];
		CameraPoint.y = R[3] * WorldPoint.x + R[4] * WorldPoint.y + R[5] * WorldPoint.z + TranslationVector[1];
		CameraPoint.z = R[6] * WorldPoint.x + R[7] * WorldPoint.y + R[8] * WorldPoint.z + TranslationVector[2];

		// Divide x and y by the z-coordinate to normalize the point before projection
		CameraPoint.z = CameraPoint.z ? (1.0 / CameraPoint.z) : 1.0;

		CameraPoint.x *= CameraPoint.z;
		CameraPoint.y *= CameraPoint.z;

		const double R2 = CameraPoint.x * CameraPoint.x + CameraPoint.y * CameraPoint.y;
		const double R4 = R2 * R2;
		const double R6 = R4 * R2;

		const double A1 = 2 * CameraPoint.x * CameraPoint.y;
		const double A2 = R2 + 2 * CameraPoint.x * CameraPoint.x;
		const double A3 = R2 + 2 * CameraPoint.y * CameraPoint.y;

		const double Radial = 1 + K1 * R2 + K2 * R4 + K3 * R6;
		const double TangentialX = (P1 * A1) + (P2 * A2);
		const double TangentialY = (P1 * A3) + (P2 * A1);

		cv::Point2d DistortedPoint;
		DistortedPoint.x = (CameraPoint.x * Radial) + TangentialX;
		DistortedPoint.y = (CameraPoint.y * Radial) + TangentialY;

		cv::Point2d& ProjectedPoint = ProjectedPoints.at<cv::Point2d>(PointIndex);
		ProjectedPoint.x = DistortedPoint.x * Fx + Cx;
		ProjectedPoint.y = DistortedPoint.y * Fy + Cy;

		if (bComputeJacobian)
		{
			const int JacobianStep = Jacobian.cols;

			cv::Mat JacRotation;
			cv::Mat JacTranslation;
			cv::Mat JacFocalLength;
			cv::Mat JacImageCenter;
			cv::Mat JacTargetOffsetRotation;
			cv::Mat JacTargetOffsetTranslation;
			cv::Mat JacDistortion;
			SubdivideJacobian(Jacobian, JacRotation, JacTranslation, JacFocalLength, JacImageCenter, JacTargetOffsetRotation, JacTargetOffsetTranslation, JacDistortion, SolverFlags);

			if (!JacImageCenter.empty())
			{
				double* JacImageCenterPtr = JacImageCenter.ptr<double>(PointIndex * 2);

				JacImageCenterPtr[0] = 1;
				JacImageCenterPtr[1] = 0;

				JacImageCenterPtr[JacobianStep + 0] = 0;
				JacImageCenterPtr[JacobianStep + 1] = 1;
			}

			if (!JacFocalLength.empty())
			{
				double* JacFocalLengthPtr = JacFocalLength.ptr<double>(PointIndex * 2);

				if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixAspectRatio))
				{
					double PixelAspect = Fx / Fy;
					JacFocalLengthPtr[0] = 0;
					JacFocalLengthPtr[1] = DistortedPoint.x * PixelAspect;
				}
				else
				{
					JacFocalLengthPtr[0] = DistortedPoint.x;
					JacFocalLengthPtr[1] = 0;
				}

				JacFocalLengthPtr[JacobianStep + 0] = 0;
				JacFocalLengthPtr[JacobianStep + 1] = DistortedPoint.y;
			}

			if (!JacDistortion.empty())
			{
				double* JacDistortionPtr = JacDistortion.ptr<double>(PointIndex * 2);

				JacDistortionPtr[0] = Fx * CameraPoint.x * R2;
				JacDistortionPtr[1] = Fx * CameraPoint.x * R4;
				JacDistortionPtr[2] = Fx * A1;
				JacDistortionPtr[3] = Fx * A2;
				JacDistortionPtr[4] = Fx * CameraPoint.x * R6;

				JacDistortionPtr[JacobianStep + 0] = Fy * CameraPoint.y * R2;
				JacDistortionPtr[JacobianStep + 1] = Fy * CameraPoint.y * R4;
				JacDistortionPtr[JacobianStep + 2] = Fy * A3;
				JacDistortionPtr[JacobianStep + 3] = Fy * A1;
				JacDistortionPtr[JacobianStep + 4] = Fy * CameraPoint.y * R6;
			}

			if (!JacTranslation.empty())
			{
				// Compute the derivative of the undistorted image point with respect to the translation vector [ T0  T1  T2 ]
				cv::Point3d DCameraPointXDT;
				DCameraPointXDT.x = CameraPoint.z;
				DCameraPointXDT.y = 0;
				DCameraPointXDT.z = -CameraPoint.x * CameraPoint.z;

				cv::Point3d DCameraPointYDT;
				DCameraPointYDT.x = 0;
				DCameraPointYDT.y = CameraPoint.z;
				DCameraPointYDT.z = -CameraPoint.y * CameraPoint.z;

				// Compute the derivative of the R-Squared term with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DR2DT = (2 * CameraPoint.x * DCameraPointXDT) + (2 * CameraPoint.y * DCameraPointYDT);

				// Compute the derivative of the radial distortion term with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DRadialDT = (K1 * DR2DT) + (2 * K2 * R2 * DR2DT) + (3 * K3 * R4 * DR2DT);

				// Compute the derivative of the A1 term with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DA1DT = 2 * (CameraPoint.x * DCameraPointYDT + CameraPoint.y * DCameraPointXDT);

				// Compute the derivative of distorted point with respect to the translation vector [ T0  T1  T2 ]
				const cv::Point3d DDistortedPointXDT = (DCameraPointXDT * Radial) + (CameraPoint.x * DRadialDT) + (P1 * DA1DT) + (P2 * (DR2DT + 4 * CameraPoint.x * DCameraPointXDT));
				const cv::Point3d DDistortedPointYDT = (DCameraPointYDT * Radial) + (CameraPoint.y * DRadialDT) + (P2 * DA1DT) + (P1 * (DR2DT + 4 * CameraPoint.y * DCameraPointYDT));

				// Add derivatives to the Jacobian matrix
				double* JacTranslationPtr = JacTranslation.ptr<double>(PointIndex * 2);

				JacTranslationPtr[0] = Fx * DDistortedPointXDT.x;
				JacTranslationPtr[1] = Fx * DDistortedPointXDT.y;
				JacTranslationPtr[2] = Fx * DDistortedPointXDT.z;

				JacTranslationPtr[JacobianStep + 0] = Fy * DDistortedPointYDT.x;
				JacTranslationPtr[JacobianStep + 1] = Fy * DDistortedPointYDT.y;
				JacTranslationPtr[JacobianStep + 2] = Fy * DDistortedPointYDT.z;
			}

			if (!JacRotation.empty())
			{
				// Compute the derivatives of the camera points with respect to the rotation vector [ R0  R1  R2]
				// This requires the use of the rotation jacobian matrix that relates the rotation vector in Rodrigues form to the rotation matrix
				cv::Point3d DCameraPointXDR;
				DCameraPointXDR.x = WorldPoint.x * JacR[0] + WorldPoint.y * JacR[1] + WorldPoint.z * JacR[2];
				DCameraPointXDR.y = WorldPoint.x * JacR[9] + WorldPoint.y * JacR[10] + WorldPoint.z * JacR[11];
				DCameraPointXDR.z = WorldPoint.x * JacR[18] + WorldPoint.y * JacR[19] + WorldPoint.z * JacR[20];

				cv::Point3d DCameraPointYDR;
				DCameraPointYDR.x = WorldPoint.x * JacR[3] + WorldPoint.y * JacR[4] + WorldPoint.z * JacR[5];
				DCameraPointYDR.y = WorldPoint.x * JacR[12] + WorldPoint.y * JacR[13] + WorldPoint.z * JacR[14];
				DCameraPointYDR.z = WorldPoint.x * JacR[21] + WorldPoint.y * JacR[22] + WorldPoint.z * JacR[23];

				cv::Point3d DCameraPointZDR;
				DCameraPointZDR.x = WorldPoint.x * JacR[6] + WorldPoint.y * JacR[7] + WorldPoint.z * JacR[8];
				DCameraPointZDR.y = WorldPoint.x * JacR[15] + WorldPoint.y * JacR[16] + WorldPoint.z * JacR[17];
				DCameraPointZDR.z = WorldPoint.x * JacR[24] + WorldPoint.y * JacR[25] + WorldPoint.z * JacR[26];

				// Compute the derivative of the undistorted image point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DImagePointXDR = CameraPoint.z * (DCameraPointXDR - CameraPoint.x * DCameraPointZDR);
				const cv::Point3d DImagePointYDR = CameraPoint.z * (DCameraPointYDR - CameraPoint.y * DCameraPointZDR);

				// Compute the derivative of the R-Squared term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DR2DR = 2 * ((CameraPoint.x * DImagePointXDR) + (CameraPoint.y * DImagePointYDR));

				// Compute the derivative of the radial distortion term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DRadialDR = DR2DR * (K1 + (2 * K2 * R2) + (3 * K3 * R4));

				// Compute the derivative of the A1 term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DA1DR = 2 * ((CameraPoint.x * DImagePointYDR) + (CameraPoint.y * DImagePointXDR));
				
				// Compute the derivative of distorted point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DDistortedPointXDR = (DImagePointXDR * Radial) + (CameraPoint.x * DRadialDR) + (P1 * DA1DR) + (P2 * (DR2DR + 4 * CameraPoint.x * DImagePointXDR));
				const cv::Point3d DDistortedPointYDR = (DImagePointYDR * Radial) + (CameraPoint.y * DRadialDR) + (P2 * DA1DR) + (P1 * (DR2DR + 4 * CameraPoint.y * DImagePointYDR));

				// Add derivatives to the Jacobian matrix
				double* JacRotationPtr = JacRotation.ptr<double>(PointIndex * 2);

				JacRotationPtr[0] = Fx * DDistortedPointXDR.x;
				JacRotationPtr[1] = Fx * DDistortedPointXDR.y;
				JacRotationPtr[2] = Fx * DDistortedPointXDR.z;

				JacRotationPtr[JacobianStep + 0] = Fy * DDistortedPointYDR.x;
				JacRotationPtr[JacobianStep + 1] = Fy * DDistortedPointYDR.y;
				JacRotationPtr[JacobianStep + 2] = Fy * DDistortedPointYDR.z;
			}

			if (!JacTargetOffsetTranslation.empty())
			{
				cv::Point3d DWorldPointXDT;
				DWorldPointXDT.x = InvTrackerR[0];
				DWorldPointXDT.y = InvTrackerR[1];
				DWorldPointXDT.z = InvTrackerR[2];

				cv::Point3d DWorldPointYDT;
				DWorldPointYDT.x = InvTrackerR[3];
				DWorldPointYDT.y = InvTrackerR[4];
				DWorldPointYDT.z = InvTrackerR[5];

				cv::Point3d DWorldPointZDT;
				DWorldPointZDT.x = InvTrackerR[6];
				DWorldPointZDT.y = InvTrackerR[7];
				DWorldPointZDT.z = InvTrackerR[8];

				cv::Point3d DCameraPointXDT = R[0] * DWorldPointXDT + R[1] * DWorldPointYDT + R[2] * DWorldPointZDT;;
				cv::Point3d DCameraPointYDT = R[3] * DWorldPointXDT + R[4] * DWorldPointYDT + R[5] * DWorldPointZDT;
				cv::Point3d DCameraPointZDT = R[6] * DWorldPointXDT + R[7] * DWorldPointYDT + R[8] * DWorldPointZDT;

				// Compute the derivative of the undistorted image point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DImagePointXDT = CameraPoint.z * (DCameraPointXDT - CameraPoint.x * DCameraPointZDT);
				const cv::Point3d DImagePointYDT = CameraPoint.z * (DCameraPointYDT - CameraPoint.y * DCameraPointZDT);

				// Compute the derivative of the R-Squared term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DR2DT= 2 * ((CameraPoint.x * DImagePointXDT) + (CameraPoint.y * DImagePointYDT));

				// Compute the derivative of the radial distortion term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DRadialDT = DR2DT * (K1 + (2 * K2 * R2) + (3 * K3 * R4));

				// Compute the derivative of the A1 term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DA1DT = 2 * ((CameraPoint.x * DImagePointYDT) + (CameraPoint.y * DImagePointXDT));

				// Compute the derivative of distorted point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DDistortedPointXDT = (DImagePointXDT * Radial) + (CameraPoint.x * DRadialDT) + (P1 * DA1DT) + (P2 * (DR2DT + 4 * CameraPoint.x * DImagePointXDT));
				const cv::Point3d DDistortedPointYDT = (DImagePointYDT * Radial) + (CameraPoint.y * DRadialDT) + (P2 * DA1DT) + (P1 * (DR2DT + 4 * CameraPoint.y * DImagePointYDT));

				// Add derivatives to the Jacobian matrix
				double* JacTargetOffsetTranslationPtr = JacTargetOffsetTranslation.ptr<double>(PointIndex * 2);

				JacTargetOffsetTranslationPtr[0] = Fx * DDistortedPointXDT.x;
				JacTargetOffsetTranslationPtr[1] = Fx * DDistortedPointXDT.y;
				JacTargetOffsetTranslationPtr[2] = Fx * DDistortedPointXDT.z;

				JacTargetOffsetTranslationPtr[JacobianStep + 0] = Fy * DDistortedPointYDT.x;
				JacTargetOffsetTranslationPtr[JacobianStep + 1] = Fy * DDistortedPointYDT.y;
				JacTargetOffsetTranslationPtr[JacobianStep + 2] = Fy * DDistortedPointYDT.z;
			}

			if (!JacTargetOffsetRotation.empty())
			{
				cv::Point3d DOffsetPointXDR;
				DOffsetPointXDR.x = TargetPoint.x * TargetOffsetJacR[0] + TargetPoint.y * TargetOffsetJacR[1] + TargetPoint.z * TargetOffsetJacR[2];
				DOffsetPointXDR.y = TargetPoint.x * TargetOffsetJacR[9] + TargetPoint.y * TargetOffsetJacR[10] + TargetPoint.z * TargetOffsetJacR[11];
				DOffsetPointXDR.z = TargetPoint.x * TargetOffsetJacR[18] + TargetPoint.y * TargetOffsetJacR[19] + TargetPoint.z * TargetOffsetJacR[20];

				cv::Point3d DOffsetPointYDR;
				DOffsetPointYDR.x = TargetPoint.x * TargetOffsetJacR[3] + TargetPoint.y * TargetOffsetJacR[4] + TargetPoint.z * TargetOffsetJacR[5];
				DOffsetPointYDR.y = TargetPoint.x * TargetOffsetJacR[12] + TargetPoint.y * TargetOffsetJacR[13] + TargetPoint.z * TargetOffsetJacR[14];
				DOffsetPointYDR.z = TargetPoint.x * TargetOffsetJacR[21] + TargetPoint.y * TargetOffsetJacR[22] + TargetPoint.z * TargetOffsetJacR[23];

				cv::Point3d DOffsetPointZDR;
				DOffsetPointZDR.x = TargetPoint.x * TargetOffsetJacR[6] + TargetPoint.y * TargetOffsetJacR[7] + TargetPoint.z * TargetOffsetJacR[8];
				DOffsetPointZDR.y = TargetPoint.x * TargetOffsetJacR[15] + TargetPoint.y * TargetOffsetJacR[16] + TargetPoint.z * TargetOffsetJacR[17];
				DOffsetPointZDR.z = TargetPoint.x * TargetOffsetJacR[24] + TargetPoint.y * TargetOffsetJacR[25] + TargetPoint.z * TargetOffsetJacR[26];

				cv::Point3d DWorldPointXDR = InvTrackerR[0] * DOffsetPointXDR + InvTrackerR[1] * DOffsetPointYDR + InvTrackerR[2] * DOffsetPointZDR;
				cv::Point3d DWorldPointYDR = InvTrackerR[3] * DOffsetPointXDR + InvTrackerR[4] * DOffsetPointYDR + InvTrackerR[5] * DOffsetPointZDR;
				cv::Point3d DWorldPointZDR = InvTrackerR[6] * DOffsetPointXDR + InvTrackerR[7] * DOffsetPointYDR + InvTrackerR[8] * DOffsetPointZDR;

				// Compute the derivatives of the camera points with respect to the rotation vector [ R0  R1  R2]
				// This requires the use of the rotation jacobian matrix that relates the rotation vector in Rodrigues form to the rotation matrix
				cv::Point3d DCameraPointXDR = R[0] * DWorldPointXDR + R[1] * DWorldPointYDR + R[2] * DWorldPointZDR;
				cv::Point3d DCameraPointYDR = R[3] * DWorldPointXDR + R[4] * DWorldPointYDR + R[5] * DWorldPointZDR;
				cv::Point3d DCameraPointZDR = R[6] * DWorldPointXDR + R[7] * DWorldPointYDR + R[8] * DWorldPointZDR;

				// Compute the derivative of the undistorted image point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DImagePointXDR = CameraPoint.z * (DCameraPointXDR - CameraPoint.x * DCameraPointZDR);
				const cv::Point3d DImagePointYDR = CameraPoint.z * (DCameraPointYDR - CameraPoint.y * DCameraPointZDR);

				// Compute the derivative of the R-Squared term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DR2DR = 2 * ((CameraPoint.x * DImagePointXDR) + (CameraPoint.y * DImagePointYDR));

				// Compute the derivative of the radial distortion term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DRadialDR = DR2DR * (K1 + (2 * K2 * R2) + (3 * K3 * R4));

				// Compute the derivative of the A1 term with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DA1DR = 2 * ((CameraPoint.x * DImagePointYDR) + (CameraPoint.y * DImagePointXDR));

				// Compute the derivative of distorted point with respect to the rotation vector [ R0  R1  R2 ]
				const cv::Point3d DDistortedPointXDR = (DImagePointXDR * Radial) + (CameraPoint.x * DRadialDR) + (P1 * DA1DR) + (P2 * (DR2DR + 4 * CameraPoint.x * DImagePointXDR));
				const cv::Point3d DDistortedPointYDR = (DImagePointYDR * Radial) + (CameraPoint.y * DRadialDR) + (P2 * DA1DR) + (P1 * (DR2DR + 4 * CameraPoint.y * DImagePointYDR));

				// Add derivatives to the Jacobian matrix
				double* JacTargetOffsetRotationPtr = JacTargetOffsetRotation.ptr<double>(PointIndex * 2);

				JacTargetOffsetRotationPtr[0] = Fx * DDistortedPointXDR.x;
				JacTargetOffsetRotationPtr[1] = Fx * DDistortedPointXDR.y;
				JacTargetOffsetRotationPtr[2] = Fx * DDistortedPointXDR.z;

				JacTargetOffsetRotationPtr[JacobianStep + 0] = Fy * DDistortedPointYDR.x;
				JacTargetOffsetRotationPtr[JacobianStep + 1] = Fy * DDistortedPointYDR.y;
				JacTargetOffsetRotationPtr[JacobianStep + 2] = Fy * DDistortedPointYDR.z;
			}
		}
	}
}

void ULensDistortionSolverOpenCV::GatherPoints(
	const TArray<FObjectPoints>& InObjectPointsArray,
	const TArray<FImagePoints>& InImagePointsArray,
	cv::Mat& ObjectPointsMat,
	cv::Mat& ImagePointsMat)
{
	cv::Point3d* ObjectPointsMatData = ObjectPointsMat.ptr<cv::Point3d>();
	cv::Point2d* ImagePointsMatData = ImagePointsMat.ptr<cv::Point2d>();

	int TotalPointIndex = 0;

	const int NumImages = InObjectPointsArray.Num();

	for (int ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		const FObjectPoints& CurrentObjectPoints = InObjectPointsArray[ImageIndex];
		const FImagePoints& CurrentImagePoints = InImagePointsArray[ImageIndex];

		const int NumPointsInImage = CurrentObjectPoints.Points.Num();

		for (int PointIndex = 0; PointIndex < NumPointsInImage; ++PointIndex)
		{
			const FVector ObjectPointCV = FOpenCVHelper::ConvertUnrealToOpenCV(CurrentObjectPoints.Points[PointIndex]);
			ObjectPointsMatData[TotalPointIndex + PointIndex] = cv::Point3d(ObjectPointCV.X, ObjectPointCV.Y, ObjectPointCV.Z);

			ImagePointsMatData[TotalPointIndex + PointIndex] = cv::Point2d(CurrentImagePoints.Points[PointIndex].X, CurrentImagePoints.Points[PointIndex].Y);
		}

		TotalPointIndex += NumPointsInImage;
	}
}

void ULensDistortionSolverOpenCV::SubdivideJacobian(
	const cv::Mat& Jacobian,
	cv::Mat& JacRotation,
	cv::Mat& JacTranslation,
	cv::Mat& JacFocalLength,
	cv::Mat& JacImageCenter,
	cv::Mat& JacTargetOffsetRotation,
	cv::Mat& JacTargetOffsetTranslation,
	cv::Mat& JacDistortion,
	ECalibrationFlags SolverFlags)
{
	const bool bComputeJacobian = !Jacobian.empty();

	if (bComputeJacobian)
	{
		if (Jacobian.cols >= 6)
		{
			if (!(EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixExtrinsics)))
			{
				JacRotation = cv::Mat(Jacobian.colRange(0, 3));
				JacTranslation = cv::Mat(Jacobian.colRange(3, 6));
			}
		}

		if (Jacobian.cols >= 10)
		{
			if (!(EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixFocalLength)))
			{
				JacFocalLength = cv::Mat(Jacobian.colRange(6, 8));
			}

			if (!(EnumHasAnyFlags(SolverFlags, ECalibrationFlags::FixPrincipalPoint)))
			{
				JacImageCenter = cv::Mat(Jacobian.colRange(8, 10));
			}
		}

		int DistortionParameterOffset = 10;
		if (EnumHasAnyFlags(SolverFlags, ECalibrationFlags::SolveTargetOffset))
		{
			if (Jacobian.cols >= 16)
			{
				JacTargetOffsetRotation = cv::Mat(Jacobian.colRange(10, 13));
				JacTargetOffsetTranslation = cv::Mat(Jacobian.colRange(13, 16));
			}

			DistortionParameterOffset += 6;
		}

		if (Jacobian.cols > DistortionParameterOffset)
		{
			JacDistortion = cv::Mat(Jacobian.colRange(DistortionParameterOffset, Jacobian.cols));
		}
	}
}

void ULensDistortionSolverOpenCV::FindUniqueCameraPoses(const TArray<FTransform>& InCameraPoses, TArray<FTransform>& OutUniqueCameraPoses, TArray<int32>& OutUniquePoseIndices)
{
	const int32 NumPoses = InCameraPoses.Num();

	FTransform LastCameraPose = InCameraPoses[0];
	FTransform CurrentCameraPose;
	for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
	{
		CurrentCameraPose = InCameraPoses[PoseIndex];

		if (!FCameraCalibrationUtils::IsNearlyEqual(LastCameraPose, CurrentCameraPose))
		{
			OutUniqueCameraPoses.Add(LastCameraPose);
			LastCameraPose = CurrentCameraPose;
		}

		OutUniquePoseIndices.Add(OutUniqueCameraPoses.Num());
	}

	OutUniqueCameraPoses.Add(LastCameraPose);
}

FLevMarqSolver::FLevMarqSolver(int NumParamsToSolve, int NumErrors, int NumMaxIterations, ESymmetryMode SymmetryMode)
{
	Params = cv::Mat(NumParamsToSolve, 1, CV_64F);
	PreviousParams = cv::Mat(NumParamsToSolve, 1, CV_64F);
	Mask = cv::Mat(NumParamsToSolve, 1, CV_8U, cv::Scalar(1));

	JacTJac = cv::Mat::zeros(NumParamsToSolve, NumParamsToSolve, CV_64F);
	JacTErr = cv::Mat::zeros(NumParamsToSolve, 1, CV_64F);

	if (NumErrors > 0)
	{
		Jacobian = cv::Mat::zeros(NumErrors, NumParamsToSolve, CV_64F);
		Error = cv::Mat::zeros(NumErrors, 1, CV_64F);
	}

	MaxIterations = FMath::Clamp(NumMaxIterations, 1, 1000);

	SymmMode = SymmetryMode;
}

bool FLevMarqSolver::Update()
{
	if (State == ESolverState::Done)
	{
		return false;
	}

	if (State == ESolverState::Started)
	{
		State = ESolverState::ComputeJacobian;
		return true;
	}

	if (State == ESolverState::ComputeJacobian)
	{
		JacTJac = Jacobian.t() * Jacobian;
		JacTErr = Jacobian.t() * Error;

		Params.copyTo(PreviousParams);

		Step();

		if (Iterations == 0)
		{
			PreviousErrorNorm = cv::norm(Error, cv::NORM_L2);
		}

		Error.setTo(cv::Scalar::all(0));

		State = ESolverState::CheckError;
		return true;
	}

	ErrorNorm = cv::norm(Error, cv::NORM_L2);
	if (ErrorNorm > PreviousErrorNorm)
	{
		if (++LambdaLog10 <= 16)
		{
			Step();
			Error.setTo(cv::Scalar::all(0));
			State = ESolverState::CheckError;
			return true;
		}
	}

	LambdaLog10 = MAX(LambdaLog10 - 1, -16);
	if (++Iterations >= MaxIterations || cv::norm(Params, PreviousParams, cv::NORM_L2 | cv::NORM_RELATIVE) < DBL_EPSILON)
	{
		State = ESolverState::Done;
		return false;
	}

	PreviousErrorNorm = ErrorNorm;
	Jacobian.setTo(cv::Scalar::all(0));
	State = ESolverState::ComputeJacobian;
	return true;
}

bool FLevMarqSolver::UpdateAlt()
{
	if (State == ESolverState::Done)
	{
		return false;
	}

	if (State == ESolverState::Started)
	{
		State = ESolverState::ComputeJacobian;
		return true;
	}

	if (State == ESolverState::ComputeJacobian)
	{
		Params.copyTo(PreviousParams);
		Step();
		PreviousErrorNorm = ErrorNorm;
		ErrorNorm = 0;
		State = ESolverState::CheckError;
		return true;
	}

	if (ErrorNorm > PreviousErrorNorm)
	{
		if (++LambdaLog10 <= 16)
		{
			Step();
			ErrorNorm = 0;
			State = ESolverState::CheckError;
			return true;
		}
	}

	LambdaLog10 = MAX(LambdaLog10 - 1, -16);
	if (++Iterations >= MaxIterations || cv::norm(Params, PreviousParams, cv::NORM_L2 | cv::NORM_RELATIVE) < DBL_EPSILON)
	{
		State = ESolverState::Done;
		return false;
	}

	PreviousErrorNorm = ErrorNorm;
	JacTJac.setTo(cv::Scalar::all(0));
	JacTErr.setTo(cv::Scalar::all(0));
	State = ESolverState::ComputeJacobian;
	return true;
}

void FLevMarqSolver::SubMatrix(const cv::Mat& Src, cv::Mat& Dst, const cv::Mat& Cols, const cv::Mat& Rows)
{
	int NumNonZerosCols = cv::countNonZero(Cols);
	cv::Mat tmp(Src.rows, NumNonZerosCols, CV_64FC1);

	for (int i = 0, Index = 0; i < Cols.total(); i++)
	{
		if (Cols.at<uchar>(i))
		{
			Src.col(i).copyTo(tmp.col(Index++));
		}
	}

	int NumNonZeroRows = cv::countNonZero(Rows);
	Dst.create(NumNonZeroRows, NumNonZerosCols, CV_64FC1);
	for (int i = 0, Index = 0; i < Rows.total(); i++)
	{
		if (Rows.at<uchar>(i))
		{
			tmp.row(i).copyTo(Dst.row(Index++));
		}
	}
}

void FLevMarqSolver::Step()
{
	if (cv::countNonZero(Mask) == 0)
	{
		return;
	}

	cv::Mat SubJtJ;
	SubMatrix(JacTJac, SubJtJ, Mask, Mask);

	cv::Mat SubJtErr;
	SubMatrix(JacTErr, SubJtErr, cv::Mat::ones(1, 1, CV_8U), Mask);

	bool bLowerToUpper = !(SymmMode == ESymmetryMode::LowerToUpper);
	cv::completeSymm(SubJtJ, bLowerToUpper);

	const double Lambda = FMath::Exp(LambdaLog10 * FMath::Loge(10.0));
	SubJtJ.diag() *= 1.0 + Lambda;

	cv::Mat NonZeroParams;
	cv::solve(SubJtJ, SubJtErr, NonZeroParams, cv::DECOMP_SVD);

	int NonZeroParamIndex = 0;
	int NumParamsToSolve = Params.rows;

	for (int ParamIndex = 0; ParamIndex < NumParamsToSolve; ++ParamIndex)
	{
		bool bNonZero = (Mask.at<uchar>(ParamIndex) == 1);
		Params.at<double>(ParamIndex) = PreviousParams.at<double>(ParamIndex) - (bNonZero ? NonZeroParams.at<double>(NonZeroParamIndex++) : 0);
	}
}

int FOptimizeNodalOffsetSolver::getDims() const
{
	constexpr int32 NumRotationParameters = 4; // FQuat
	constexpr int32 NumLocationParameters = 3; // FVector
	return NumRotationParameters + NumLocationParameters;
}

double FOptimizeNodalOffsetSolver::calc(const double* x) const
{
	// Convert the input data (7 doubles) to an FQuat and FVector
	const FQuat Rotation = FQuat(x[0], x[1], x[2], x[3]).GetNormalized();
	const FVector Location = FVector(x[4], x[5], x[6]);

	UE_LOGF(LogCameraCalibrationSolver, VeryVerbose, "Nodal Offset Candidate:  Rotation: (%lf, %lf, %lf, %lf)  Location: (%lf, %lf, %lf)",
		Rotation.X, Rotation.Y, Rotation.Z, Rotation.W, Location.X, Location.Y, Location.Z);

	FTransform NodalOffsetCandidate;

	// As a result of the way that the downhill solver nudges the input data on each iteration, it is important to normalize the rotation
	NodalOffsetCandidate.SetRotation(Rotation);
	NodalOffsetCandidate.SetLocation(Location);

	double ReprojectionErrorTotal = 0.0;
	int32 NumTotalPoints = 0;

	const int32 NumCameraViews = CameraPoses.Num();
	for (int32 ViewIndex = 0; ViewIndex < NumCameraViews; ++ViewIndex)
	{
		const FTransform& CameraPose = CameraPoses[ViewIndex];
		const TArray<FVector>& ObjectPoints = Points3d[ViewIndex];
		const TArray<FVector2f>& ImagePoints = Points2d[ViewIndex];

		// Compute the optimal camera pose using the nodal offset candidate for this iteration and the tracked camera pose 
		const FTransform OptimalCameraPose = NodalOffsetCandidate * CameraPose;

		// Compute the reprojection error for this view and add it to the running total
		double ViewError = FOpenCVHelper::ComputeReprojectionError(ObjectPoints, ImagePoints, FocalLength, ImageCenter, OptimalCameraPose);
		ReprojectionErrorTotal += ViewError;

		NumTotalPoints += ImagePoints.Num();
	}

	const double RootMeanSquareError = FMath::Sqrt(ReprojectionErrorTotal / NumTotalPoints);

	UE_LOGF(LogCameraCalibrationSolver, VeryVerbose, "Reprojection Error: %lf", RootMeanSquareError);

	return RootMeanSquareError;
}

bool ULensDistortionSolverOpenCV::CanUseHomographyInitialization(const cv::Mat& ObjectPoints, double StdDevTolerance)
{
	if (ObjectPoints.channels() == 2)
	{
		return true;
	}

	if (ObjectPoints.channels() != 3)
	{
		return false;
	}

	// Check if any coordinate has low variance, indicating canonical plane (X=0, Y=0, or Z=0)
	cv::Scalar Mean, StdDev;
	cv::meanStdDev(ObjectPoints, Mean, StdDev);
	
	return (StdDev[0] < StdDevTolerance) || (StdDev[1] < StdDevTolerance) || (StdDev[2] < StdDevTolerance);
}

void ULensDistortionSolverOpenCV::InitIntrinsicsFrom3DPoints(
	const cv::Mat& ObjectPoints,
	const cv::Mat& ImagePoints,
	const cv::Mat& NumPoints,
	const cv::Size ImageSize,
	cv::Mat& CameraMatrix)
{
	// For 3D world coordinates, we'll use cv::calibrateCamera to get initial intrinsic estimates
	// This is more robust than homography-based methods for non XY plane points.
	
	const int32 NumImages = NumPoints.cols;
	
	// Convert the input matrices to the format expected by cv::calibrateCamera
	std::vector<std::vector<cv::Point3f>> ObjectPointsVec;
	std::vector<std::vector<cv::Point2f>> ImagePointsVec;
	
	ObjectPointsVec.reserve(NumImages);
	ImagePointsVec.reserve(NumImages);
	
	int32 ObjectPointsIndex = 0;
	for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		const int32 NumImagePoints = NumPoints.ptr<int32>()[ImageIndex];
		
		std::vector<cv::Point3f> ObjectPointsInImage;
		std::vector<cv::Point2f> ImagePointsInImage;
		
		ObjectPointsInImage.reserve(NumImagePoints);
		ImagePointsInImage.reserve(NumImagePoints);
		
		for (int32 PointIndex = 0; PointIndex < NumImagePoints; ++PointIndex)
		{
			const cv::Point3d& ObjectPoint = ObjectPoints.at<cv::Point3d>(0, ObjectPointsIndex + PointIndex);
			const cv::Point2d& ImagePoint = ImagePoints.at<cv::Point2d>(0, ObjectPointsIndex + PointIndex);
			
			ObjectPointsInImage.push_back(cv::Point3f(ObjectPoint.x, ObjectPoint.y, ObjectPoint.z));
			ImagePointsInImage.push_back(cv::Point2f(ImagePoint.x, ImagePoint.y));
		}
		
		ObjectPointsVec.push_back(ObjectPointsInImage);
		ImagePointsVec.push_back(ImagePointsInImage);
		
		ObjectPointsIndex += NumImagePoints;
	}
	
	// Use cv::calibrateCamera to get initial estimates
	cv::Mat DistCoeffs = cv::Mat::zeros(5, 1, CV_64F);
	std::vector<cv::Mat> Rvecs, Tvecs;
	
	try
	{
		cv::calibrateCamera(
			ObjectPointsVec,
			ImagePointsVec,
			ImageSize,
			CameraMatrix,
			DistCoeffs,
			Rvecs,
			Tvecs,
			cv::CALIB_USE_INTRINSIC_GUESS
		);
		
		UE_LOGF(LogCameraCalibrationSolver, Log, "Initialized intrinsics from 3D points: Fx=%.2f, Fy=%.2f, Cx=%.2f, Cy=%.2f", 
			CameraMatrix.at<double>(0, 0), CameraMatrix.at<double>(1, 1), 
			CameraMatrix.at<double>(0, 2), CameraMatrix.at<double>(1, 2));
	}
	catch (const cv::Exception& e)
	{
		UE_LOGF(LogCameraCalibrationSolver, Warning, "cv::calibrateCamera failed for 3D point initialization: %ls", ANSI_TO_TCHAR(e.what()));
		
		// Fallback to basic initialization
		const double Fx = ImageSize.width;
		const double Fy = ImageSize.width; 
		const double Cx = (ImageSize.width - 1) * 0.5;
		const double Cy = (ImageSize.height - 1) * 0.5;
		
		CameraMatrix.at<double>(0, 0) = Fx;
		CameraMatrix.at<double>(1, 1) = Fy;
		CameraMatrix.at<double>(0, 2) = Cx;
		CameraMatrix.at<double>(1, 2) = Cy;
	}
}

#endif // WITH_OPENCV

#undef LOCTEXT_NAMESPACE
