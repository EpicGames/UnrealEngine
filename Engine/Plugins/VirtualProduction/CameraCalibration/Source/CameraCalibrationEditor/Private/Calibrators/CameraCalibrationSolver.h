// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibrationTypes.h"
#include "Models/LensModel.h"
#include "Templates/SubclassOf.h"

#if WITH_OPENCV

#include "PreOpenCVHeaders.h"
#include <opencv2/core.hpp>
#include "PostOpenCVHeaders.h"

#endif	// WITH_OPENCV

#include "CameraCalibrationSolver.generated.h"

/** 
 * An array of 3D object points associated with a single calibration image.
 * Structure is needed because Blueprints and Python do not support arrays of arrays.
 */
USTRUCT(BlueprintType)
struct FObjectPoints
{
	GENERATED_BODY()

	/** 3D object points in an image */
	UPROPERTY(BlueprintReadWrite, Category = "Calibration")
	TArray<FVector> Points;
};

/** 
 * An array of 2D image points associated with a single calibration image 
 * Structure is needed because Blueprints and Python do not support arrays of arrays.
 */
USTRUCT(BlueprintType)
struct FImagePoints
{
	GENERATED_BODY()

	/** 2D image points in an image */
	UPROPERTY(BlueprintReadWrite, Category = "Calibration")
	TArray<FVector2D> Points;
};

/**
  * Flags used to modify the execution of the calibration solver
  */
UENUM()
enum class ECalibrationFlags : uint32
{
	None = 0,
	UseIntrinsicGuess = 1 << 0,  /** The solver will skip intrinsics initialization an use the input values of Focal Length and Image Center as the starting values for the optimization */
	UseExtrinsicGuess = 1 << 1,  /** The solver will skip extrinsics initialization an use the input Camera Poses as the starting value for the optimization */
	FixFocalLength = 1 << 2,     /** The solver will not optimize the focal length */
	FixPrincipalPoint = 1 << 3,  /** The solver will not optimize the principal point */
	FixExtrinsics = 1 << 4,      /** The solver will not optimize the camera extrinsics */
	FixDistortion = 1 << 5,      /** The solver will fix all distortion values at 0 */
	FixAspectRatio = 1 << 6,     /** The solver will respect the input aspect ratio when solving for Fx and Fy */
	SolveTargetOffset = 1 << 7,  /** The solver will solve for an offset for each image's input 3D object points */
	GroupCameraPoses = 1 << 8    /** The solver will group sets of images together that share the same camera pose */
};

/** 
 * Base lens distortion solver class that can be inherited from in Blueprints or Python 
 */
UCLASS(Abstract, Blueprintable)
class ULensDistortionSolver : public UObject
{
	GENERATED_BODY()

public:
	/** Calibrate camera intrinsics and distortion from a set of input 3D-2D point correspondences and initial camera intrinsics guess. */
	UFUNCTION(BlueprintNativeEvent, Category = "Calibration")
	FDistortionCalibrationResult Solve(
		const TArray<FObjectPoints>& ObjectPointArray,
		const TArray<FImagePoints>& ImagePointArray,
		const FIntPoint ImageSize,
		const FVector2D& FocalLength,
		const FVector2D& ImageCenter,
		const TArray<float>& DistortionParameters,
		const TArray<FTransform>& CameraPoses,
		const TArray<FTransform>& TargetPoses,
		TSubclassOf<ULensModel> LensModel,
		double PixelAspect,
		ECalibrationFlags SolverFlags);

	/** Get the name of this solver class for displaying in the editor UI */
	UFUNCTION(BlueprintNativeEvent, Category = "Calibration")
	FText GetDisplayName() const;

	/** Get the name of this solver class for displaying in the editor UI */
	UFUNCTION(BlueprintNativeEvent, Category = "Calibration")
	bool IsEnabled() const;

public:
	/** Get the latest status of the solve and marks the status as old. Returns true if this status was new, or false if this status was old. */
	bool GetStatusText(FText& OutStatusText);

	/** Cancels the solve by setting bIsRunning to false, inidicating that the solver should stop executing the next time it checks IsRunning() */
	void Cancel() { bIsRunning = false; }

	virtual FDistortionCalibrationResult Solve_Implementation(
		const TArray<FObjectPoints>& ObjectPointArray,
		const TArray<FImagePoints>& ImagePointArray,
		const FIntPoint ImageSize,
		const FVector2D& FocalLength,
		const FVector2D& ImageCenter,
		const TArray<float>& DistortionParameters,
		const TArray<FTransform>& CameraPoses,
		const TArray<FTransform>& TargetPoses,
		TSubclassOf<ULensModel> LensModel,
		double PixelAspect,
		ECalibrationFlags SolverFlags) PURE_VIRTUAL(ULensDistortionSolver::Solve_Implementation, return FDistortionCalibrationResult(););

	virtual FText GetDisplayName_Implementation() const PURE_VIRTUAL(ULensDistortionSolver::GetDisplayName_Implementation, return FText::GetEmpty(););

	virtual bool IsEnabled_Implementation() const { return true; }

protected:
	/** Returns true if the solver is currently running, and false if it has been cancelled. The solver should call this in critical loops in order to respond to cancellation requests. */
	UFUNCTION(BlueprintCallable, Category = "Calibration")
	bool IsRunning() const { return bIsRunning; }

	/** Sets the latest status text and marks bIsStatusNew to true */
	void SetStatusText(FText InStatusText);

protected:
	/** True is the solver should continue executing. Set to false by Cancel(), indicating that the solver should early-out the next time it checks IsRunning() */
	std::atomic<bool> bIsRunning = true;

	/** Status text describing the current state of the solve */
	FText StatusText;

	/** Set to true by SetStatusText() when a new status is available. Set to false by GetStatusText(), indicating that the latest status has already been queried. */
	bool bHasStatusChanged = false;
};

/** 
  * Lens Distortion Solver class, supporting anamorphic and spherical models 
  * The implementation is largely based on the implementation of calibrateCamera from OpenCV: https://github.com/opencv/opencv
  */

UCLASS(meta = (DisplayName="OpenCV Solver"))
class ULensDistortionSolverOpenCV : public ULensDistortionSolver
{
	GENERATED_BODY()

public:
	/** 
	  * Calibrate camera intrinsics and distortion parameters using a set of input 3D and 2D point correspondences
	  * Returns the root mean reprojection error between the input image points and the projection of the input object points
	  */
	virtual FDistortionCalibrationResult Solve_Implementation(
		const TArray<FObjectPoints>& ObjectPointArray,
		const TArray<FImagePoints>& ImagePointArray,
		const FIntPoint ImageSize,
		const FVector2D& FocalLength,
		const FVector2D& ImageCenter,
		const TArray<float>& DistortionParameters,
		const TArray<FTransform>& CameraPoses,
		const TArray<FTransform>& TargetPoses,
		TSubclassOf<ULensModel> LensModel,
		double PixelAspect,
		ECalibrationFlags SolverFlags) override;

	virtual FText GetDisplayName_Implementation() const override;

#if WITH_OPENCV
private:
	/** Initialize the camera matrix of intrinsic parameters using linear algebra techniques */
	void InitCameraIntrinsics(
		const cv::Mat& ObjectPoints,
		const cv::Mat& ImagePoints,
		const cv::Mat& NumPoints,
		const cv::Size ImageSize,
		cv::Mat& CameraMatrix,
		bool bUseCanonicalPlaneInit);

	/** Initialize the camera extrinsics (rotation and translation vectors) for each image using linear algebra techniques */
	void InitCameraExtrinsics(
		const TSubclassOf<ULensModel> LensModel,
		const cv::Mat& ObjectPoints,
		const cv::Mat& ImagePoints,
		const cv::Mat& CameraMatrix,
		const cv::Mat& DistCoeffs,
		const cv::Size ImageSize,
		cv::Mat& Rotation,
		cv::Mat& Translation,
		const ECalibrationFlags SolverFlags,
		bool bUseCanonicalPlaneInit);

	/** Project the input object points to 2D using the input camera intrinsics, extrinsics, and distortion parameters */
	void ProjectPoints(
		const TSubclassOf<ULensModel> LensModel,
		const cv::Mat& ObjectPoints,
		const FTransform& TargetPose,
		const cv::Mat& ObjectPointOffsetRotation,
		const cv::Mat& ObjectPointOffsetTranslation,
		const cv::Mat& Rotation,
		const cv::Mat& Translation,
		const cv::Mat& CameraMatrix,
		const cv::Mat& DistCoeffs,
		const cv::Size ImageSize,
		cv::Mat& ProjectedPoints);

	/** Project the input object points to 2D using the input camera intrinsics, extrinsics, and distortion parameters */
	void ProjectPoints(
		const TSubclassOf<ULensModel> LensModel,
		const cv::Mat& ObjectPoints,
		const FTransform& TargetPose,
		const cv::Mat& ObjectPointOffsetRotation,
		const cv::Mat& ObjectPointOffsetTranslation,
		const cv::Mat& Rotation,
		const cv::Mat& Translation,
		const cv::Mat& CameraMatrix,
		const cv::Mat& DistCoeffs,
		const cv::Size ImageSize,
		cv::Mat& ProjectedPoints,
		cv::Mat& Jacobian,
		ECalibrationFlags SolverFlags);

	/** Project the input object points to 2D using the input camera intrinsics, extrinsics, and anamorphic distortion parameters */
	void ProjectPointsAnamorphic(
		const cv::Mat& ObjectPoints,
		const cv::Mat& Rotation,
		const cv::Mat& Translation,
		const cv::Mat& CameraMatrix,
		const cv::Mat& DistCoeffs,
		const cv::Size ImageSize,
		cv::Mat& ProjectedPoints,
		cv::Mat& Jacobian,
		ECalibrationFlags SolverFlags);

	/** Project the input object points to 2D using the input camera intrinsics, extrinsics, and spherical distortion parameters */
	void ProjectPointsSpherical(
		const cv::Mat& ObjectPoints,
		const FTransform& TargetPose,
		const cv::Mat& ObjectPointOffsetRotation,
		const cv::Mat& ObjectPointOffsetTranslation,
		const cv::Mat& Rotation,
		const cv::Mat& Translation,
		const cv::Mat& CameraMatrix,
		const cv::Mat& DistCoeffs,
		cv::Mat& ProjectedPoints,
		cv::Mat& Jacobian,
		ECalibrationFlags SolverFlags);

	/** Copy the input 3D and 2D points to OpenCV matrices for ease of use with the solver */
	void GatherPoints(
		const TArray<FObjectPoints>& InObjectPointsArray,
		const TArray<FImagePoints>& InImagePointsArray,
		cv::Mat& ObjectPointsMat,
		cv::Mat& ImagePointsMat);

	/** Divide large jacobian matrix into smaller views for each of the parameter groups the solver will solve */
	void SubdivideJacobian(
		const cv::Mat& Jacobian,
		cv::Mat& JacRotation,
		cv::Mat& JacTranslation,
		cv::Mat& JacFocalLength,
		cv::Mat& JacImageCenter,
		cv::Mat& JacObjectPointOffsetRotation,
		cv::Mat& JacObjectPointOffsetTranslation,
		cv::Mat& JacDistortion,
		ECalibrationFlags SolverFlags);

	/** Find the unique set of camera poses and the index into that set for each image */
	void FindUniqueCameraPoses(const TArray<FTransform>& InCameraPoses, TArray<FTransform>& OutUniqueCameraPoses, TArray<int32>& OutUniquePoseIndices);

	/** Check if the input points lie in a canonical plane suitable for homography */
	bool CanUseHomographyInitialization(const cv::Mat& ObjectPoints, double Tolerance = 1e-6);

	/** Initialize camera intrinsics from 2D plane points using homography */
	void InitIntrinsicsFromCanonicalPlane(
		const cv::Mat& ObjectPoints,
		const cv::Mat& ImagePoints,
		const cv::Mat& NumPoints,
		const cv::Size ImageSize,
		cv::Mat& CameraMatrix);

	/** Initialize camera intrinsics from 3D world coordinates (not requiring canonical plane like homography) */
	void InitIntrinsicsFrom3DPoints(
		const cv::Mat& ObjectPoints,
		const cv::Mat& ImagePoints,
		const cv::Mat& NumPoints,
		const cv::Size ImageSize,
		cv::Mat& CameraMatrix);

#endif	// WITH_OPENCV
};

class FCameraCalibrationSolver
{
public:

	/** Optimize the input nodal offset transform by running a downhill solver that attempts to minimize the reprojection error of the input points */
	static double OptimizeNodalOffset(
		const TArray<TArray<FVector>>& InObjectPoints,
		const TArray<TArray<FVector2f>>& InImagePoints,
		const FVector2D& InFocalLength,
		const FVector2D& InImageCenter,
		const TArray<FTransform>& InCameraPoses,
		FTransform& InOutNodalOffset);
};

#if WITH_OPENCV

/** Levenberg–Marquardt solver for solving least-squares problems */
class FLevMarqSolver
{
public:
	/** Mode for copying the contents of symmetric matrix */
	enum class ESymmetryMode : uint8
	{
		LowerToUpper,
		UpperToLower
	};

	/** State used by the solver in its state machine */
	enum class ESolverState : uint8
	{
		Done = 0,
		Started = 1,
		ComputeJacobian = 2,
		CheckError = 3
	};

	FLevMarqSolver(int NumParamsToSolve, int NumErrors, int NumMaxIterations, ESymmetryMode SymmetryMode = ESymmetryMode::LowerToUpper);

	/** Update the solver with the input jacobian and error matrices */
	bool Update();

	/** Update the solver with the input pre-multiplied jacobian and error matrices */
	bool UpdateAlt();

private:
	/** Step the solver, modifying the values of each of the parameters */
	void Step();

	/** Create a submatrix from the source matrix using only the columns and rows specified */
	void SubMatrix(const cv::Mat& Src, cv::Mat& Dst, const cv::Mat& Cols, const cv::Mat& Rows);

public:
	/** Parameters solved by this solver */
	cv::Mat Params;

	/** Mask specifiying which parameters in the parameter list should not be modified */
	cv::Mat Mask;

	/** Current solver state */
	ESolverState State = ESolverState::Started;

	/** Average error for the current parameter set */
	double ErrorNorm = 0;

	/** Jacobian Matrix */
	cv::Mat Jacobian;

	/** Error Matrix */
	cv::Mat Error;

	/** Jacobian Transposed multiplied by the Jacobian */
	cv::Mat JacTJac;

	/** Jacobian Transposed multiplied by the Error */
	cv::Mat JacTErr;

private:
	/** Stores the previous values of each parameter */
	cv::Mat PreviousParams;

	/** Previous average error */
	double PreviousErrorNorm = DBL_MAX;

	/** Maximum number of iterations the solver will do */
	int MaxIterations = 0;

	/** Current number of iterations */
	int Iterations = 0;

	/** Mode to use when copying half of the symmetric matrix */
	ESymmetryMode SymmMode;

	int LambdaLog10 = -3;
};

class FOptimizeNodalOffsetSolver : public cv::MinProblemSolver::Function
{
public:

	//~ Begin cv::MinProblemSolver::Function interface
	int getDims() const;
	double calc(const double* x) const;
	//~ End cv::MinProblemSolver::Function interface

public:
	FVector2D FocalLength;
	FVector2D ImageCenter;

	TArray<FTransform> CameraPoses;
	TArray<TArray<FVector>> Points3d;
	TArray<TArray<FVector2f>> Points2d;
};

#endif // WITH_OPENCV
