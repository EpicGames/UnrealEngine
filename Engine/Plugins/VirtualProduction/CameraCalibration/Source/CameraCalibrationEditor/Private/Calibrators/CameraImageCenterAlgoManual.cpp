// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraImageCenterAlgoManual.h"

#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/ImageCenterTool.h"
#include "CameraCalibrationSettings.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Input/SComboBox.h"


#define LOCTEXT_NAMESPACE "CameraImageCenterAlgoManual"

void UCameraImageCenterAlgoManual::Initialize(UImageCenterTool* InImageCenterTool)
{
	Tool = InImageCenterTool;

	// Cache the LensFile
	if (FCameraCalibrationStepsController* StepsController = Tool->GetCameraCalibrationStepsController())
	{
		LensFile = StepsController->GetLensFile();
	}

	// Initialize sensitivity options for the combobox
	SensitivtyOptions.Add(MakeShared<float>(0.25f));
	SensitivtyOptions.Add(MakeShared<float>(0.5f));
	SensitivtyOptions.Add(MakeShared<float>(1.0f));
	SensitivtyOptions.Add(MakeShared<float>(2.0f));
	SensitivtyOptions.Add(MakeShared<float>(5.0f));
	SensitivtyOptions.Add(MakeShared<float>(10.0f));
}

void UCameraImageCenterAlgoManual::Shutdown()
{
	// Prompt the user to save any unsaved changes before shutting down the algo
	ApplyOrRevertAdjustedImageCenter();

	Tool.Reset();
}

void UCameraImageCenterAlgoManual::Activate()
{
	FCameraCalibrationStepsController* StepsController = Tool->GetCameraCalibrationStepsController();
	if (!StepsController)
	{
		return;
	}

	if (!LensFile.IsValid())
	{
		return;
	}

	RenderTargetSize = StepsController->GetMediaOverlayResolution();

	const FLensFileEvaluationInputs EvalInputs = StepsController->GetLensFileEvaluationInputs();
	if (!EvalInputs.bIsValid)
	{
		return;
	}

	// Cache the original image center point for the current focus and zoom
	CurrentEvalFocus = EvalInputs.Focus;
	CurrentEvalZoom = EvalInputs.Zoom;
	LensFile->EvaluateImageCenterParameters(CurrentEvalFocus, CurrentEvalZoom, OriginalImageCenter);
	AdjustedImageCenter = OriginalImageCenter;

	bIsActive = true;
}

void UCameraImageCenterAlgoManual::Deactivate()
{
	// Prompt the user to save any unsaved changes before deactivating the algo
	ApplyOrRevertAdjustedImageCenter();

	bIsActive = false;
}

void UCameraImageCenterAlgoManual::Tick(float DeltaTime)
{
	FCameraCalibrationStepsController* StepsController = Tool->GetCameraCalibrationStepsController();
	if (!StepsController)
	{
		return;
	}

	if (LensFile.IsValid())
	{
		const FLensFileEvaluationInputs EvalInputs = StepsController->GetLensFileEvaluationInputs();
		if (EvalInputs.bIsValid)
		{
			// Check if the focus or zoom has changed since the last tick
			const float InputTolerance = GetDefault<UCameraCalibrationSettings>()->GetCalibrationInputTolerance();
			if (!FMath::IsNearlyEqual(CurrentEvalFocus, EvalInputs.Focus, InputTolerance) || !FMath::IsNearlyEqual(CurrentEvalZoom, EvalInputs.Zoom, InputTolerance))
			{
				// Prompt the user to save any unsaved changes before caching a new original image center based on the new focus/zoom
				ApplyOrRevertAdjustedImageCenter();

				CurrentEvalFocus = EvalInputs.Focus;
				CurrentEvalZoom = EvalInputs.Zoom;
				LensFile->EvaluateImageCenterParameters(CurrentEvalFocus, CurrentEvalZoom, OriginalImageCenter);
				AdjustedImageCenter = OriginalImageCenter;
			}
		}
	}
}

bool UCameraImageCenterAlgoManual::OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent)
{
	if (!LensFile.IsValid())
	{
		return false;
	}

	// Guard against divide-by-zero
	if ((RenderTargetSize.X == 0) || (RenderTargetSize.Y == 0))
	{
		return false;
	}

	bool bHandled = false;

	// Arrow Key handling
	if (InKey == EKeys::Left)
	{
		AdjustedImageCenter.PrincipalPoint.X -= (AdjustmentIncrement / RenderTargetSize.X);
		bHandled = true;
	}
	else if (InKey == EKeys::Right)
	{
		AdjustedImageCenter.PrincipalPoint.X += (AdjustmentIncrement / RenderTargetSize.X);
		bHandled = true;
	}
	else if (InKey == EKeys::Up)
	{
		AdjustedImageCenter.PrincipalPoint.Y -= (AdjustmentIncrement / RenderTargetSize.Y);
		bHandled = true;
	}
	else if (InKey == EKeys::Down)
	{
		AdjustedImageCenter.PrincipalPoint.Y += (AdjustmentIncrement / RenderTargetSize.Y);
		bHandled = true;
	}

	if (bHandled)
	{
		// First, attempt to modify an existing image point in the image center table
		bool bPointExistsInTable = LensFile->ImageCenterTable.SetPoint(CurrentEvalFocus, CurrentEvalZoom, AdjustedImageCenter);

		// If no point exists at the specified focus and zoom, add a new image center point
		if (!bPointExistsInTable)
		{
			LensFile->AddImageCenterPoint(CurrentEvalFocus, CurrentEvalZoom, AdjustedImageCenter);
		}
	}

	return bHandled;
}

TSharedRef<SWidget> UCameraImageCenterAlgoManual::BuildUI()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Sensitivity
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("SensitivityWidget", "Sensitivity"), BuildSensitivityWidget())]
	;
}

bool UCameraImageCenterAlgoManual::HasImageCenterChanged()
{
	return (OriginalImageCenter.PrincipalPoint != AdjustedImageCenter.PrincipalPoint);
}

void UCameraImageCenterAlgoManual::OnSavedImageCenter()
{
	// Before beginning a transaction, restore the lens file image center point to its original state
	const FImageCenterInfo CachedAdjustedImageCenter = AdjustedImageCenter;
	AdjustedImageCenter = OriginalImageCenter;
	LensFile->ImageCenterTable.SetPoint(CurrentEvalFocus, CurrentEvalZoom, OriginalImageCenter);

	{
		FScopedTransaction Transaction(LOCTEXT("SaveAdjustedImageCenter", "Save Adjusted Image Center"));
		this->Modify();
		LensFile->Modify();

		// After beginning the transaction, reapply the final modified image center
		AdjustedImageCenter = CachedAdjustedImageCenter;
		LensFile->ImageCenterTable.SetPoint(CurrentEvalFocus, CurrentEvalZoom, AdjustedImageCenter);

		// Reset the original image center to the adjusted point to indicate that the change has been written
		OriginalImageCenter = AdjustedImageCenter;
	}
}

TSharedRef<SWidget> UCameraImageCenterAlgoManual::BuildSensitivityWidget()
{
	return SNew(SComboBox<TSharedPtr<float>>)
		.OptionsSource(&SensitivtyOptions)
		.OnSelectionChanged_Lambda([&](TSharedPtr<float> NewValue, ESelectInfo::Type Type) -> void
		{
			AdjustmentIncrement = *NewValue;
		})
		.OnGenerateWidget_Lambda([&](TSharedPtr<float> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock).Text(FText::AsNumber(*InOption));
		})
		.InitiallySelectedItem(SensitivtyOptions[2])
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				return FText::AsNumber(AdjustmentIncrement);
			})
		];
}

void UCameraImageCenterAlgoManual::ApplyOrRevertAdjustedImageCenter()
{
	// Warn the user that they have modified the image center, and ask if they want to save or revert their changes
	if (OriginalImageCenter.PrincipalPoint != AdjustedImageCenter.PrincipalPoint)
	{
		FText Message = LOCTEXT("UnsavedImageCenterChanges",
			"Would you like to apply the changes of the modified image center to the lens file?");

		EAppReturnType::Type ReturnValue = FMessageDialog::Open(EAppMsgType::YesNo, Message);

		if (ReturnValue == EAppReturnType::No)
		{
			// Reset the image center back to the original value
			if (LensFile.IsValid())
			{
				LensFile->ImageCenterTable.SetPoint(CurrentEvalFocus, CurrentEvalZoom, OriginalImageCenter);
				AdjustedImageCenter = OriginalImageCenter;
			}
		}
		else if (ReturnValue == EAppReturnType::Yes)
		{
			OnSavedImageCenter();
		}
	}
}

#undef LOCTEXT_NAMESPACE
