// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraImageCenterAlgo.h"

#include "LensData.h"
#include "LensFile.h"

#include "CameraImageCenterAlgoManual.generated.h"

class SWidget;

/**
 * Implements an image center adjustment algorithm.
 * The Manual algorithm simply takes manual input from the user and adjusts the image center by small increments
 */
UCLASS()
class UCameraImageCenterAlgoManual : public UCameraImageCenterAlgo
{
	GENERATED_BODY()

public:

	//~ Begin UCameraImageCenterAlgo
	virtual void Initialize(UImageCenterTool* InImageCenterTool) override;
	virtual void Shutdown() override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void Tick(float DeltaTime) override;
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const override { return TEXT("Manual Image Center Adjustment"); };
	virtual bool HasImageCenterChanged() override;
	virtual void OnSavedImageCenter() override;
	//~ End UCameraImageCenterAlgo

private:
	/** Builds the UI of the sensitivity widget */
	TSharedRef<SWidget> BuildSensitivityWidget();

	/** Open a dialog to let the user decide whether to save their adjustments or revert to the original state */
	void ApplyOrRevertAdjustedImageCenter();

private:
	/** Cached original image center that was evaluated at the current focus and zoom */
	UPROPERTY(Transient)
	FImageCenterInfo OriginalImageCenter;

	/** Latest image center that may have been adjusted by manual user input */
	UPROPERTY(Transient)
	FImageCenterInfo AdjustedImageCenter;

private:
	/** The image center tool controller */
	TWeakObjectPtr<UImageCenterTool> Tool;

	/** The LensFile being edited */
	TWeakObjectPtr<ULensFile> LensFile;

	/** Cached focus and zoom that were used to evaluate the original image center */
	float CurrentEvalFocus;
	float CurrentEvalZoom;

	/** Size of the render target displayed in the viewport */
	FIntPoint RenderTargetSize;

	/** Increment (in pixels) that the image center will be adjusted by each individual key press */
	float AdjustmentIncrement = 0.5f;

	/** Array of sensitivity options to display to the user to set the adjustement increment */
	TArray<TSharedPtr<float>> SensitivtyOptions;
};
