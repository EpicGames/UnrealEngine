// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraNodalOffsetAlgoAruco.h"

#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/NodalOffsetTool.h"
#include "AssetEditor/SSimulcamViewport.h"
#include "CalibrationPointComponent.h"
#include "CameraCalibrationUtils.h"
#include "CameraCalibrationUtilsPrivate.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/MessageDialog.h"
#include "OpenCVHelper.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "CameraNodalOffsetAlgoAruco"


bool UCameraNodalOffsetAlgoAruco::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We only respond to left clicks
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return false;
	}

	const FText TitleError = LOCTEXT("CalibrationError", "CalibrationError");
	FText ErrorMessage;

	if (!PopulatePoints(ErrorMessage))
	{
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);
	}

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoAruco::BuildUI()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Calibrator picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Calibrator", "Calibrator"), BuildCalibrationDevicePickerWidget())]

		+ SVerticalBox::Slot() // Calibrator component picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorComponents", "Calibrator Component(s)"), BuildCalibrationComponentPickerWidget())]

		+ SVerticalBox::Slot() // Calibrator point names
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorPoint", "Calibrator Point"), BuildCalibrationPointsComboBox())]

		+ SVerticalBox::Slot() // Show Detection
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("ShowDetection", "Show Detection"), BuildShowDetectionWidget())]

		+ SVerticalBox::Slot() // Calibration Rows
		.AutoHeight()
		.MaxHeight(12 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[BuildCalibrationPointsTable()]

		+ SVerticalBox::Slot() // Action buttons (e.g. Remove, Clear)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.AutoHeight()
		.Padding(0, 20)
		[BuildCalibrationActionButtons()]
		;
}

bool UCameraNodalOffsetAlgoAruco::PopulatePoints(FText& OutErrorMessage)
{
	const FCameraCalibrationStepsController* StepsController;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, nullptr)))
	{
		OutErrorMessage = LOCTEXT("ToolNotFound", "Tool not found");
		return false;
	}

	if (!Calibrator.IsValid())
	{
		OutErrorMessage = LOCTEXT("InvalidCalibrator", "Please select a calibrator with Aruco markers in the given combo box.");
		return false;
	}

	if (!LastCameraData.bIsValid)
	{
		OutErrorMessage = LOCTEXT("InvalidLastCameraData", "Could not find a cached set of camera data (e.g. FIZ). Check the Lens Component to make sure it has valid evaluation inputs.");
		return false;
	}


	// Detect Aruco dictionary used by looking at the calibration point names
	EArucoDictionary Dictionary = UE::CameraCalibration::Private::GetArucoDictionaryForCalibrator(Calibrator.Get());

	if (Dictionary == EArucoDictionary::None)
	{
		OutErrorMessage = LOCTEXT("InvalidArucoDictionary",
			"We could not find an appropriate Aruco dictionary for the selected Calibrator.\n"
			"Make sure you have selected a Calibrator containing Calibration Point components\n"
			"with names per the naming convention described in the info section of this algo"
		);

		return false;
	}

	// Read media pixels

	TArray<FColor> Pixels;
	FIntPoint Size;

	if (!StepsController->ReadMediaPixels(Pixels, Size, OutErrorMessage, ESimulcamViewportPortion::CameraFeed))
	{
		return false;
	}

	TArray<FArucoMarker> IdentifiedMarkers;
	bool bResult = FOpenCVHelper::IdentifyArucoMarkers(Pixels, Size, Dictionary, IdentifiedMarkers);

	if (!bResult || IdentifiedMarkers.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("NoMarkerCornersFound", "No Aruco markers were detected in the media image, "
			"or they did not match the aruco dictionary of any of the selected calibrator components."
		);
		return false;
	}

	// Show the detection to the user
	if (bShouldShowDetectionWindow)
	{
		UTexture2D* DebugTexture = UTexture2D::CreateTransient(Size.X, Size.Y, EPixelFormat::PF_B8G8R8A8);
		UE::CameraCalibration::Private::SetTextureData(DebugTexture, Pixels);
		FOpenCVHelper::DrawArucoMarkers(IdentifiedMarkers, DebugTexture);

		FCameraCalibrationWidgetHelpers::DisplayTextureInWindowAlmostFullScreen(DebugTexture, LOCTEXT("ArucoMarkerDetection", "ArucoMarkerDetection"));
	}

	// Add the detected markers that match calibration points to the table
	for(int32 MarkerIdx=0; MarkerIdx < IdentifiedMarkers.Num(); ++MarkerIdx)
	{
		// Naming convention for calibration points:
		//     DICTIONARY-ID-CORNER
		// Where CORNER can have the following 4 values:
		//     TL, TR, BR, BL, corresponding to "Top-Left" and so on.
		// Examples:
		//     DICT_6X6_250-23-TL
		//     DICT_6X6_250-23-TR

		FArucoMarker Marker = IdentifiedMarkers[MarkerIdx];

		static const TArray<FString> CornerNames = { TEXT("TL"), TEXT("TR"), TEXT("BR"), TEXT("BL") };

		// Export the latest session data
		ExportSessionData();

		 // Iterate over the 4 corners of this marker and try to find the corresponding calibration point
		for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
		{
			// Build calibrator point name based on the detected marker
			const FString DictionaryName = UE::CameraCalibration::Private::GetArucoDictionaryName(Dictionary);
			const FString MarkerName = FString::Printf(TEXT("%s-%d-%s"), *DictionaryName, Marker.MarkerID, *CornerNames[CornerIdx]);

			FNodalOffsetPointsCalibratorPointData PointCache;

			if (!CalibratorPointCacheFromName(MarkerName, PointCache))
			{
				continue;
			}

			// Create the row that we're going to add
			TSharedPtr<FNodalOffsetPointsRowData> Row = MakeShared<FNodalOffsetPointsRowData>();

			// Get the next row index for the current calibration session to assign to this new row
			const uint32 RowIndex = NodalOffsetTool->AdvanceSessionRowIndex();

			Row->Index = RowIndex;
			Row->Point2D = Marker.Corners[CornerIdx] / Size;

			Row->CalibratorPointData.Location = PointCache.Location;
			Row->CalibratorPointData.Name = MarkerName;
			Row->CalibratorPointData.bIsValid = true;

			Row->CameraData = LastCameraData;

			if (!ValidateNewRow(Row, OutErrorMessage))
			{
				// Notify the ListView of the new data
				if (CalibrationListView.IsValid())
				{
					CalibrationListView->RequestListRefresh();
				}

				return false;
			}

			CalibrationRows.Add(Row);

			// Export the data for this row to a .json file on disk
			ExportRow(Row);
		}
	}

	// Notify the ListView of the new data
	if (CalibrationListView.IsValid())
	{
		CalibrationListView->RequestListRefresh();
	}

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoAruco::BuildShowDetectionWidget()
{
	return SNew(SCheckBox)
		.IsChecked_Lambda([&]() -> ECheckBoxState
		{
			return bShouldShowDetectionWindow ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([&](ECheckBoxState NewState) -> void
		{
			bShouldShowDetectionWindow = (NewState == ECheckBoxState::Checked);
		});
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoAruco::BuildHelpWidget()
{
	return SNew(STextBlock)
		.Text(LOCTEXT("CameraNodalOffsetAlgoArucoHelp",
			"This nodal offset algorithm is equivalent to the 'Nodal Offset Points Method', except that\n"
			"instead of explicitly clicking on the 2d calibrator point locations, it will automatically\n"
			"detect the Aruco markers when the viewport is clicked and add them to the calibration\n"
			"points table.\n\n"

			"The calibrator model should have calibration point components that match one of four\n"
			"corners of any given Aruco marker. The algorithm will match the detected Aruco fiducials\n"
			"with the calibration points based on the following naming convention:\n\n"

			"     DICTIONARY-ID-CORNER\n"
			" Where CORNER can have the following 4 values:\n"
			"     TL, TR, BR, BL, corresponding to 'Top-Left' and so on.\n"
			" Examples:\n"
			"     DICT_6X6_250-23-TL\n"
			"     DICT_6X6_250-23-TR\n\n"
		));
}

#undef LOCTEXT_NAMESPACE
