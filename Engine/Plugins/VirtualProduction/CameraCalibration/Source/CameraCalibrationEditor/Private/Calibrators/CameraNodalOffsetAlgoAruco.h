// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraNodalOffsetAlgoPoints.h"

#include "CameraNodalOffsetAlgoAruco.generated.h"

class SWidget;

/** 
 * Implements a nodal offset calibration algorithm. It uses 3d points (UCalibrationPointComponent) 
 * specified in a selectable calibrator that should be named after the correponding Aruco markers.
 */
UCLASS()
class UCameraNodalOffsetAlgoAruco : public UCameraNodalOffsetAlgoPoints
{
	GENERATED_BODY()
	
public:

	//~ Begin CalibPointsNodalOffsetAlgo
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const override { return TEXT("Nodal Offset Aruco Markers"); };
	virtual FName ShortName() const override { return TEXT("Aruco"); };
	virtual TSharedRef<SWidget> BuildHelpWidget() override;
	//~ End CalibPointsNodalOffsetAlgo

protected:

	/** True if a detection window should be shown after every capture */
	bool bShouldShowDetectionWindow = false;

protected:

	/** Builds the UI for the user to select if they want a detection window to be shown after every capture */
	TSharedRef<SWidget> BuildShowDetectionWidget();

protected:

	/** Populates the calibration rows. True if successful. */
	virtual bool PopulatePoints(FText& OutErrorMessage);
};
