// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraNodalOffsetAlgoCharuco.h"
#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/NodalOffsetTool.h"
#include "CameraCalibrationCharucoBoard.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationUtilsPrivate.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "EngineUtils.h"
#include "Misc/MessageDialog.h"
#include "OpenCVHelper.h"
#include "TextureResource.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "UI/SFilterableActorPicker.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "CameraNodalOffsetAlgoCharuco"

bool UCameraNodalOffsetAlgoCharuco::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We only respond to left clicks
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return false;
	}

	FText ErrorMessage;
	if (!PopulatePoints(ErrorMessage))
	{
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, LOCTEXT("CalibrationError", "Calibration Error"));
	}

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCharuco::BuildUI()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Charuco board picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CharucoBoard", "Charuco Board"), BuildCharucoBoardPickerWidget())]

		+ SVerticalBox::Slot() // Calibrator component picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorComponents", "Calibrator Component(s)"), BuildCalibrationComponentPickerWidget())]

		+ SVerticalBox::Slot() // Show Detection
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("ShowDetection", "Show Detection"), BuildShowDetectionWidget())]

		+ SVerticalBox::Slot() // Calibration Rows
		.AutoHeight()
		.MaxHeight(12 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[BuildCalibrationPointsTable()]

		+ SVerticalBox::Slot() // Action buttons (e.g. Remove, Clear)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.AutoHeight()
		.Padding(0, 20)
		[BuildCalibrationActionButtons()]
		;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCharuco::BuildCharucoBoardPickerWidget()
{
	return SNew(SFilterableActorPicker)
		.OnSetObject_Lambda([&](const FAssetData& AssetData) -> void
		{
			if (AssetData.IsValid())
			{
				SetCalibrator(Cast<ACameraCalibrationCharucoBoard>(AssetData.GetAsset()));
			}
		})
		.OnShouldFilterAsset_Lambda([&](const FAssetData& AssetData) -> bool
		{
			return !!Cast<ACameraCalibrationCharucoBoard>(AssetData.GetAsset());
		})
		.ActorAssetData_Lambda([&]() -> FAssetData
		{
			return FAssetData(GetCalibrator(), true);
		});
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCharuco::BuildShowDetectionWidget()
{
	return SNew(SCheckBox)
		.IsChecked_Lambda([&]() -> ECheckBoxState
		{
			return bShouldShowDetectionWindow ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([&](ECheckBoxState NewState) -> void
		{
			bShouldShowDetectionWindow = (NewState == ECheckBoxState::Checked);
		});
}

bool UCameraNodalOffsetAlgoCharuco::OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition)
{
	// Calculate ROI rectangle from marquee selection
	FIntRect MarqueeROI;
	MarqueeROI.Min = FIntPoint(FMath::Floor(FMath::Min(StartPosition.X, EndPosition.X)), 
							   FMath::Floor(FMath::Min(StartPosition.Y, EndPosition.Y)));
	MarqueeROI.Max = FIntPoint(FMath::Floor(FMath::Max(StartPosition.X, EndPosition.X)), 
							   FMath::Floor(FMath::Max(StartPosition.Y, EndPosition.Y)));


	// Call PopulatePoints directly with the ROI - clean and efficient!
	FText ErrorMessage;
	bool bResult = PopulatePoints(ErrorMessage, MarqueeROI);
	
	if (!bResult)
	{
		UE_LOGF(LogCameraCalibrationEditor, Warning, "CameraNodalOffsetAlgoCharuco: Marquee selection failed: %ls", *ErrorMessage.ToString());
	}
	
	return bResult;
}

AActor* UCameraNodalOffsetAlgoCharuco::FindFirstCalibrator() const
{
	UWorld* World = GetWorld();
	if (!World)
	{
		return nullptr;
	}

	if (TActorIterator<ACameraCalibrationCharucoBoard> ActorIterator(World); ActorIterator)
	{
		return *ActorIterator;
	}

	return nullptr;
}

bool UCameraNodalOffsetAlgoCharuco::PopulatePoints(FText& OutErrorMessage)
{
	// Regular clicks use full image (empty ROI)
	return PopulatePoints(OutErrorMessage, FIntRect());
}

bool UCameraNodalOffsetAlgoCharuco::PopulatePoints(FText& OutErrorMessage, const FIntRect& ROI)
{
	const FCameraCalibrationStepsController* StepsController;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, nullptr)))
	{
		OutErrorMessage = LOCTEXT("ToolNotFound", "Tool not found");
		return false;
	}

	if (!Calibrator.IsValid())
	{
		OutErrorMessage = LOCTEXT("InvalidCalibrator", "Please select a Charuco board calibrator in the given combo box.");
		return false;
	}

	if (!LastCameraData.bIsValid)
	{
		OutErrorMessage = LOCTEXT("InvalidLastCameraData", "Could not find a cached set of camera data (e.g. FIZ). Check the Lens Component to make sure it has valid evaluation inputs.");
		return false;
	}

	// Get the Charuco board
	ACameraCalibrationCharucoBoard* CharucoBoard = Cast<ACameraCalibrationCharucoBoard>(Calibrator.Get());
	if (!CharucoBoard)
	{
		OutErrorMessage = LOCTEXT("InvalidCharucoBoard", "The selected calibrator is not a Charuco board.");
		return false;
	}

	// Read media pixels
	TArray<FColor> Pixels;
	FIntPoint Size;

	if (!StepsController->ReadMediaPixels(Pixels, Size, OutErrorMessage, ESimulcamViewportPortion::CameraFeed))
	{
		return false;
	}

	// Setup board configuration from the actor (NumCorners + 1 = squares in each direction)
	CachedBoardConfig.SquaresX = CharucoBoard->NumCornerCols + 1;
	CachedBoardConfig.SquaresY = CharucoBoard->NumCornerRows + 1;
	CachedBoardConfig.SquareSize = CharucoBoard->SquareSideLength;
	CachedBoardConfig.MarkerSize = CharucoBoard->SquareSideLength * CharucoBoard->MarkerSizeRatio;
	CachedBoardConfig.Dictionary = UE::CameraCalibration::Private::ShadowToOpenCV(CharucoBoard->ArucoDictionary);

	// Detect Charuco corners
	FCharucoCorners DetectedCorners;
	bool bDetectionSuccess = FOpenCVHelper::IdentifyCharucoCorners(Pixels, Size, CachedBoardConfig, DetectedCorners, ROI);

	if (!bDetectionSuccess || DetectedCorners.Corners.IsEmpty())
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT(
			"Could not identify Charuco corners. "
			"The expected board has %dx%d squares with ArUco dictionary %s. "
			"Make sure the board is clearly visible and well-lit."),
			CachedBoardConfig.SquaresX, CachedBoardConfig.SquaresY,
			*UEnum::GetValueAsString(CachedBoardConfig.Dictionary)
		));
		return false;
	}

	// Show detection visualization if requested
	if (bShouldShowDetectionWindow)
	{
		ShowDetectionVisualization(DetectedCorners, Pixels, Size, ROI);
	}

	// Export the latest session data
	ExportSessionData();

	if (!ensure(DetectedCorners.Corners.Num() == DetectedCorners.CornerIds.Num()))
	{
		OutErrorMessage = FText::Format(LOCTEXT("CharucoArrayMismatch", "Charuco detection failed: corner count ({0}) does not match corner ID count ({1})"), 
			DetectedCorners.Corners.Num(), DetectedCorners.CornerIds.Num());
		return false;
	}

	// Create calibration rows for each detected corner
	for (int32 CornerIdx = 0; CornerIdx < DetectedCorners.Corners.Num(); ++CornerIdx)
	{
		int32 CornerId = DetectedCorners.CornerIds[CornerIdx];
		FVector2f ImageCorner = DetectedCorners.Corners[CornerIdx];
		
		// Calculate 3D position using same approach as checkerboard algorithm
		int32 CornerRow = CornerId / CharucoBoard->NumCornerCols;
		int32 CornerCol = CornerId % CharucoBoard->NumCornerCols;
		
		FTransform LocalPoint3d;

		// Calculate local position based on the corner coordinates and the square side length
		const FVector LocalPosition = CharucoBoard->SquareSideLength * FVector(0, CornerCol + 1, CornerRow + 1);
		LocalPoint3d.SetLocation(LocalPosition);
		
		const FVector WorldCorner = (LocalPoint3d * Calibrator->GetTransform()).GetLocation();

		TSharedPtr<FNodalOffsetPointsRowData> Row = MakeShared<FNodalOffsetPointsRowData>();

		// Convert to the expected format, normalize by image size.
		Row->Point2D = FVector2f(ImageCorner.X / Size.X, ImageCorner.Y / Size.Y);
		Row->CameraData = LastCameraData;

		// Set the calibrator point data
		Row->CalibratorPointData.bIsValid = true;
		Row->CalibratorPointData.Name = FString::Printf(TEXT("Corner_%d"), CornerId);
		Row->CalibratorPointData.Location = WorldCorner;

		// Add to the calibration points
		CalibrationRows.Add(Row);

		// Export the data for this row to a .json file on disk
		ExportRow(Row);
	}

	// Notify the ListView of the new data
	if (CalibrationListView.IsValid())
	{
		CalibrationListView->RequestListRefresh();
	}

	return true;
}

void UCameraNodalOffsetAlgoCharuco::ShowDetectionVisualization(const FCharucoCorners& DetectedCorners, const TArray<FColor>& Image, FIntPoint ImageSize, const FIntRect& ROI)
{
	// Create debug texture
	UTexture2D* DebugTexture = UTexture2D::CreateTransient(ImageSize.X, ImageSize.Y, EPixelFormat::PF_B8G8R8A8);

	if (!DebugTexture)
	{
		return;
	}

	// Copy image data to texture in nested scope to ensure unlock before UpdateResource
	{
		FTexture2DMipMap& Mip = DebugTexture->GetPlatformData()->Mips[0];
		uint8* TextureData = static_cast<uint8*>(Mip.BulkData.Lock(LOCK_READ_WRITE));
		
		ON_SCOPE_EXIT
		{
			if (Mip.BulkData.IsLocked())
			{
				Mip.BulkData.Unlock();
			}
		};
		
		if (!TextureData)
		{
			DebugTexture->MarkAsGarbage();
			return;
		}
		
		FMemory::Memcpy(TextureData, Image.GetData(), Image.Num() * sizeof(FColor));
	}

	DebugTexture->UpdateResource();

	// Draw the detection results on the texture
	FOpenCVHelper::DrawCharucoCorners(DetectedCorners, DebugTexture);

	// Draw ROI rectangle if specified
	if (ROI.Width() > 0 && ROI.Height() > 0)
	{
		DrawROIRectangle(DebugTexture, ROI, ImageSize);
	}

	// Display the texture in a window
	FText WindowTitle = ROI.Width() > 0 && ROI.Height() > 0 ? 
		FText::Format(LOCTEXT("CharucoDetectionROI", "Charuco Corner Detection ROI ({0},{1}) to ({2},{3})"), 
			ROI.Min.X, ROI.Min.Y, ROI.Max.X, ROI.Max.Y) :
		LOCTEXT("CharucoDetection", "Charuco Corner Detection");
		
	FCameraCalibrationWidgetHelpers::DisplayTextureInWindowAlmostFullScreen(DebugTexture, MoveTemp(WindowTitle));
}

void UCameraNodalOffsetAlgoCharuco::DrawROIRectangle(UTexture2D* DebugTexture, const FIntRect& ROI, FIntPoint ImageSize)
{
	if (!DebugTexture)
	{
		return;
	}

	// Draw ROI rectangle
	{
		FTexture2DMipMap& Mip = DebugTexture->GetPlatformData()->Mips[0];
		uint8* TextureData = static_cast<uint8*>(Mip.BulkData.Lock(LOCK_READ_WRITE));
		
		ON_SCOPE_EXIT
		{
			if (Mip.BulkData.IsLocked())
			{
				Mip.BulkData.Unlock();
			}
		};
		
		if (!TextureData)
		{
			return;
		}

		// ROI rectangle color (bright cyan for good visibility)
		const FColor ROIColor = FColor::Cyan;
		const int32 LineThickness = 3; // Make it thick enough to see

		// Clamp ROI to image bounds
		const int32 MinX = FMath::Clamp(ROI.Min.X, 0, ImageSize.X - 1);
		const int32 MinY = FMath::Clamp(ROI.Min.Y, 0, ImageSize.Y - 1);
		const int32 MaxX = FMath::Clamp(ROI.Max.X, MinX + 1, ImageSize.X);
		const int32 MaxY = FMath::Clamp(ROI.Max.Y, MinY + 1, ImageSize.Y);

		// Draw horizontal lines (top and bottom)
		for (int32 Thickness = 0; Thickness < LineThickness; ++Thickness)
		{
			// Top line
			int32 TopY = FMath::Clamp(MinY + Thickness, 0, ImageSize.Y - 1);
			for (int32 X = MinX; X < MaxX; ++X)
			{
				int32 PixelIndex = TopY * ImageSize.X + X;
				reinterpret_cast<FColor*>(TextureData)[PixelIndex] = ROIColor;
			}

			// Bottom line  
			int32 BottomY = FMath::Clamp(MaxY - 1 - Thickness, 0, ImageSize.Y - 1);
			for (int32 X = MinX; X < MaxX; ++X)
			{
				int32 PixelIndex = BottomY * ImageSize.X + X;
				reinterpret_cast<FColor*>(TextureData)[PixelIndex] = ROIColor;
			}
		}

		// Draw vertical lines (left and right)
		for (int32 Thickness = 0; Thickness < LineThickness; ++Thickness)
		{
			// Left line
			int32 LeftX = FMath::Clamp(MinX + Thickness, 0, ImageSize.X - 1);
			for (int32 Y = MinY; Y < MaxY; ++Y)
			{
				int32 PixelIndex = Y * ImageSize.X + LeftX;
				reinterpret_cast<FColor*>(TextureData)[PixelIndex] = ROIColor;
			}

			// Right line
			int32 RightX = FMath::Clamp(MaxX - 1 - Thickness, 0, ImageSize.X - 1);
			for (int32 Y = MinY; Y < MaxY; ++Y)
			{
				int32 PixelIndex = Y * ImageSize.X + RightX;
				reinterpret_cast<FColor*>(TextureData)[PixelIndex] = ROIColor;
			}
		}
	}

	DebugTexture->UpdateResource();
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCharuco::BuildHelpWidget()
{
	return SNew(STextBlock)
		.Text(LOCTEXT("CharucoHelp", 
			"This algorithm uses a Charuco board (chessboard with embedded ArUco markers) for calibration.\n\n"
			"Advantages:\n"
			"• Higher precision than pure ArUco markers\n"
			"• More robust than pure checkerboards\n"
			"• Partial occlusion handling\n"
			"Instructions:\n"
			"1. Create a Charuco board actor in your scene\n"
			"2. Select the board in the 'Charuco Board' dropdown\n"
			"3. Position the board so it's clearly visible in the media feed\n"
			"4. Click on the media viewport to capture calibration points\n"
			"5. Repeat from different positions and orientations"
		))
		.AutoWrapText(true);
}

#undef LOCTEXT_NAMESPACE