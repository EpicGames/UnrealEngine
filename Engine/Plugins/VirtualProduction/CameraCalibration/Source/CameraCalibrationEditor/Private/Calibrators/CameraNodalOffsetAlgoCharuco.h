// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibrationCharucoBoard.h"
#include "CameraNodalOffsetAlgoPoints.h"
#include "OpenCVHelper.h"

#include "CameraNodalOffsetAlgoCharuco.generated.h"

class ACameraCalibrationCharucoBoard;
class SWidget;

/** 
 * Implements a nodal offset calibration algorithm based on a Charuco board.
 * Charuco combines the benefits of checkerboard corners (high precision) 
 * with ArUco markers (robust detection and orientation).
 */
UCLASS()
class UCameraNodalOffsetAlgoCharuco : public UCameraNodalOffsetAlgoPoints
{
	GENERATED_BODY()
	
public:

	//~ Begin UCameraNodalOffsetAlgoPoints Interface
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const override { return TEXT("Nodal Offset Charuco"); }
	virtual FName ShortName() const override { return TEXT("Charuco"); }
	virtual TSharedRef<SWidget> BuildHelpWidget() override;
	//~ End UCameraNodalOffsetAlgoPoints Interface


protected:

	//~ Begin UCameraNodalOffsetAlgoPoints Interface
	virtual AActor* FindFirstCalibrator() const override;
	//~ End UCameraNodalOffsetAlgoPoints Interface

protected:

	/** Builds the UI for the Charuco board picker */
	TSharedRef<SWidget> BuildCharucoBoardPickerWidget();

	/** Builds the UI for showing/hiding detection visualization */
	TSharedRef<SWidget> BuildShowDetectionWidget();

	/** Populates the calibration rows with detected Charuco corners. True if successful. */
	virtual bool PopulatePoints(FText& OutErrorMessage);
	
	/** Populates the calibration rows with detected Charuco corners using ROI. True if successful. */
	virtual bool PopulatePoints(FText& OutErrorMessage, const FIntRect& ROI);

	/** Shows a debug window with the detected corners and markers, optionally showing ROI */
	void ShowDetectionVisualization(const FCharucoCorners& DetectedCorners, const TArray<FColor>& Image, FIntPoint ImageSize, const FIntRect& ROI = FIntRect());

	/** Draws an ROI rectangle on the debug texture */
	void DrawROIRectangle(UTexture2D* DebugTexture, const FIntRect& ROI, FIntPoint ImageSize);

protected:

	/** True if a detection window should be shown after every capture */
	UPROPERTY()
	bool bShouldShowDetectionWindow = false;

	/** Cached Charuco board configuration from the selected board actor */
	FCharucoBoardConfig CachedBoardConfig;
};