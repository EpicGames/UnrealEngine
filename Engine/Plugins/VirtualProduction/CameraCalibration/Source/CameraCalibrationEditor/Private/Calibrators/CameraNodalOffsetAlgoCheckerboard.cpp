// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraNodalOffsetAlgoCheckerboard.h"
#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/NodalOffsetTool.h"
#include "AssetRegistry/AssetData.h"
#include "CameraCalibrationCheckerboard.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineUtils.h"
#include "Misc/MessageDialog.h"
#include "OpenCVHelper.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "UI/SFilterableActorPicker.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "CameraNodalOffsetAlgoCheckerboard"


bool UCameraNodalOffsetAlgoCheckerboard::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We only respond to left clicks
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return false;
	}

	{
		FText ErrorMessage;

		if (!PopulatePoints(ErrorMessage))
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, LOCTEXT("CalibrationError", "CalibrationError"));
		}
	}

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCheckerboard::BuildUI()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Checkerboard picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Checkerboard", "Checkerboard"), BuildCheckerboardPickerWidget())]

		+ SVerticalBox::Slot() // Calibrator component picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorComponents", "Calibrator Component(s)"), BuildCalibrationComponentPickerWidget())]

		+ SVerticalBox::Slot() // Calibration Rows
		.AutoHeight()
		.MaxHeight(12 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[BuildCalibrationPointsTable()]

		+ SVerticalBox::Slot() // Action buttons (e.g. Remove, Clear)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.AutoHeight()
		.Padding(0, 20)
		[BuildCalibrationActionButtons()]
		;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCheckerboard::BuildCheckerboardPickerWidget()
{
	return SNew(SFilterableActorPicker)
		.OnSetObject_Lambda([&](const FAssetData& AssetData) -> void
		{
			if (AssetData.IsValid())
			{
				SetCalibrator(Cast<ACameraCalibrationCheckerboard>(AssetData.GetAsset()));
			}
		})
		.OnShouldFilterAsset_Lambda([&](const FAssetData& AssetData) -> bool
		{
			return !!Cast<ACameraCalibrationCheckerboard>(AssetData.GetAsset());
		})
		.ActorAssetData_Lambda([&]() -> FAssetData
		{
			return FAssetData(GetCalibrator(), true);
		});
}


bool UCameraNodalOffsetAlgoCheckerboard::PopulatePoints(FText& OutErrorMessage)
{
	const FCameraCalibrationStepsController* StepsController;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, nullptr)))
	{
		OutErrorMessage = LOCTEXT("ToolNotFound", "Tool not found");
		return false;
	}

	if (!Calibrator.IsValid())
	{
		OutErrorMessage = LOCTEXT("InvalidCalibrator", "Please select a calibrator with Aruco markers in the given combo box.");
		return false;
	}

	if (!LastCameraData.bIsValid)
	{
		OutErrorMessage = LOCTEXT("InvalidLastCameraData", "Could not find a cached set of camera data (e.g. FIZ). Check the Lens Component to make sure it has valid evaluation inputs.");
		return false;
	}

	TArray<FColor> Pixels;
	FIntPoint Size;

	if (!StepsController->ReadMediaPixels(Pixels, Size, OutErrorMessage, ESimulcamViewportPortion::CameraFeed))
	{
		return false;
	}

	const ACameraCalibrationCheckerboard* Checkerboard = Cast<ACameraCalibrationCheckerboard>(Calibrator.Get());
	check(Checkerboard); // This should be ensured by the picker.

	const FIntPoint CheckerboardDimensions = FIntPoint(Checkerboard->NumCornerCols, Checkerboard->NumCornerRows);

	TArray<FVector2f> Corners;
	const bool bCornersFound = FOpenCVHelper::IdentifyCheckerboard(Pixels, Size, CheckerboardDimensions, Corners);

	if (!bCornersFound || Corners.IsEmpty())
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT(
			"Could not identify the expected checkerboard points of interest. "
			"The expected checkerboard has %dx%d inner corners."),
			Checkerboard->NumCornerCols, Checkerboard->NumCornerRows)
		);

		return false;
	}

	// Export the latest session data
	ExportSessionData();

	// Create and populate the new calibration rows that we're going to add
	for (int32 RowIdx = 0; RowIdx < Checkerboard->NumCornerRows; ++RowIdx)
	{
		for (int32 ColIdx = 0; ColIdx < Checkerboard->NumCornerCols; ++ColIdx)
		{
			const int32 PointIdx = RowIdx * Checkerboard->NumCornerCols + ColIdx;

			TSharedPtr<FNodalOffsetPointsRowData> Row = MakeShared<FNodalOffsetPointsRowData>();

			// Get the next row index for the current calibration session to assign to this new row
			const uint32 RowIndex = NodalOffsetTool->AdvanceSessionRowIndex();

			Row->Index = RowIndex;
			Row->Point2D = Corners[PointIdx] / Size;

			FTransform LocalPoint3d;
			LocalPoint3d.SetLocation(Checkerboard->SquareSideLength * FVector(0, ColIdx, Checkerboard->NumCornerRows - RowIdx - 1));

			Row->CalibratorPointData.Location = (LocalPoint3d * Calibrator->GetTransform()).GetLocation();
			Row->CalibratorPointData.Name = FString::Printf(TEXT("Corner[%d][%d]"), RowIdx, ColIdx);
			Row->CalibratorPointData.bIsValid = true;

			Row->CameraData = LastCameraData;

			if (!ValidateNewRow(Row, OutErrorMessage))
			{
				// Notify the ListView of the new data
				if (CalibrationListView.IsValid())
				{
					CalibrationListView->RequestListRefresh();
				}

				return false;
			}

			CalibrationRows.Add(Row);

			// Export the data for this row to a .json file on disk
			ExportRow(Row);
		}
	}

	// Notify the ListView of the new data
	if (CalibrationListView.IsValid())
	{
		CalibrationListView->RequestListRefresh();
	}

	return true;
}

AActor* UCameraNodalOffsetAlgoCheckerboard::FindFirstCalibrator() const
{
	const FCameraCalibrationStepsController* StepsController;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, nullptr)))
	{
		return nullptr;
	}

	UWorld* World = StepsController->GetWorld();

	const EActorIteratorFlags Flags = EActorIteratorFlags::SkipPendingKill;

	if (TActorIterator<AActor> It(World, ACameraCalibrationCheckerboard::StaticClass(), Flags); It)
	{
		return CastChecked<ACameraCalibrationCheckerboard>(*It);
	}

	return nullptr;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoCheckerboard::BuildHelpWidget()
{
	return SNew(STextBlock)
		.Text(LOCTEXT("CameraNodalOffsetAlgoCheckerboardHelp",
			"This nodal offset algorithm is equivalent to the 'Nodal Offset Points Method', except that\n"
			"instead of explicitly clicking on the 2d calibrator point locations, it will automatically\n"
			"detect the chessboard corners when the viewport is clicked and add them to the calibration\n"
			"points table.\n\n"

			"Notes:\n\n"

			"- Due the symmetrical nature of chessboards, it is required that the chessboard appears\n"
			"  sufficiently upright in the media plate, in order to avoid orientation ambiguities.\n\n"
			
			"- In addition, due to the planar nature of the chessboard, the yielded nodal offset may not\n"
			"  be as accurate as when using other calibrator devices.\n"
		));
}

#undef LOCTEXT_NAMESPACE
