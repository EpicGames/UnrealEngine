// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraNodalOffsetAlgoPoints.h"

#include "CameraNodalOffsetAlgoCheckerboard.generated.h"

class ACameraCalibrationCheckerboard;
class SWidget;

/** 
 * Implements a nodal offset calibration algorithm based on a checkerboard.
 * It requires the checkerboard to be upright in the image due to symmetry ambiguity.
 */
UCLASS()
class UCameraNodalOffsetAlgoCheckerboard : public UCameraNodalOffsetAlgoPoints
{
	GENERATED_BODY()
	
public:

	//~ Begin CalibPointsNodalOffsetAlgo
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual FName FriendlyName() const override { return TEXT("Nodal Offset Checkerboard"); };
	virtual FName ShortName() const override { return TEXT("Checkerboard"); };
	virtual TSharedRef<SWidget> BuildHelpWidget() override;
	//~ End CalibPointsNodalOffsetAlgo

protected:

	//~ Begin UCameraNodalOffsetAlgoPoints
	virtual AActor* FindFirstCalibrator() const override;
	//~ End UCameraNodalOffsetAlgoPoints

protected:

	/** Builds the UI of the checkerboard device picker */
	TSharedRef<SWidget> BuildCheckerboardPickerWidget();

protected:

	/** Populates the calibration rows. True if successful. */
	virtual bool PopulatePoints(FText& OutErrorMessage);
};
