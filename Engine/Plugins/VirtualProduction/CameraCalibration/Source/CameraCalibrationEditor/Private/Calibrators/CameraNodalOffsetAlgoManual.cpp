// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraNodalOffsetAlgoManual.h"

#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/NodalOffsetTool.h"
#include "Camera/CameraActor.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SVectorInputBox.h"

#define LOCTEXT_NAMESPACE "CameraNodalOffsetAlgoManual"

void UCameraNodalOffsetAlgoManual::Initialize(UNodalOffsetTool* InNodalOffsetTool)
{
	NodalOffsetTool = InNodalOffsetTool;
}

void UCameraNodalOffsetAlgoManual::Tick(float DeltaTime)
{
	const FCameraCalibrationStepsController* const StepsController = GetStepsController();

	if (!StepsController)
	{
		return;
	}

	const FLensFileEvaluationInputs CurrentEvalInputs = StepsController->GetLensFileEvaluationInputs();

	if (CurrentEvalInputs.bIsValid)
	{
		if (const ULensFile* const LensFile = StepsController->GetLensFile())
		{
			// Query the LensFile for a Nodal Offset Point at the current Focus and Zoom
			bNodalPointExists = LensFile->GetNodalOffsetPoint(CurrentEvalInputs.Focus, CurrentEvalInputs.Zoom, CurrentNodalOffset);

			// If there is no nodal point at the current focus and zoom, evaluate the Lens File for the interpolated value of the nodal offset 
			if (!bNodalPointExists)
			{
				LensFile->EvaluateNodalPointOffset(CurrentEvalInputs.Focus, CurrentEvalInputs.Zoom, CurrentNodalOffset);
			}

			// If the evaluation inputs have changed (or become valid for the first time), update the starting nodal offset, which is used as the base for undoing all manual adjustments
			if ((!CachedEvalInputs.bIsValid) || (CachedEvalInputs.Focus != CurrentEvalInputs.Focus) || (CachedEvalInputs.Zoom != CurrentEvalInputs.Zoom))
			{
				StartingNodalOffset = CurrentNodalOffset;
			}
		}
	}
 	else
 	{
		bNodalPointExists = false;
		CurrentNodalOffset = FNodalPointOffset();
 	}

	CachedEvalInputs = CurrentEvalInputs;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoManual::BuildUI()
{
 	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Sensitivity Control
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("SensitivityWidget", "Sensitivity"), BuildSensitivityWidget())
		]

		+ SVerticalBox::Slot() // Location Widget
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("LocationWidget", "Location"), BuildLocationWidget())
		]			

		+ SVerticalBox::Slot() // Rotation Widget
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("RotationWidget", "Rotation"), BuildRotationWidget())
		]

		+ SVerticalBox::Slot() // Text Warning that there is no nodal offset point at the current Focus and Zoom
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.Padding(0, 10)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoNodalPointWarning", "There is not a nodal offset point at the current Focus and Zoom. To make manual adjustments, you must first add a new nodal offset point."))
			.ColorAndOpacity(FSlateColor(FLinearColor::Yellow))
			.Justification(ETextJustify::Center)
			.AutoWrapText(true)
			.Visibility_UObject(this, &UCameraNodalOffsetAlgoManual::GetNodalPointVisibility, false)
		]

		+ SVerticalBox::Slot() // Button to add a nodal offset point at the current Focus and Zoom
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0, 10)
			[
				SNew(SButton)
				.Text(LOCTEXT("AddNodalPointButton", "Add Nodal Point at Current Focus/Zoom"))
				.OnClicked_UObject(this, &UCameraNodalOffsetAlgoManual::AddNodalPointAtEvalInputs)
				.Visibility_UObject(this, &UCameraNodalOffsetAlgoManual::GetNodalPointVisibility, false)
			]
		]

		+ SVerticalBox::Slot() // Undo Manual Changes Button
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0, 10)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearAll", "Undo Manual Changes"))
				.OnClicked_UObject(this, &UCameraNodalOffsetAlgoManual::OnUndoManualChanges)
				.Visibility_UObject(this, &UCameraNodalOffsetAlgoManual::GetNodalPointVisibility, true)
			]
		]
	;
}

FReply UCameraNodalOffsetAlgoManual::OnUndoManualChanges()
{
	SaveNodalOffset(StartingNodalOffset);

	return FReply::Handled();
}

FReply UCameraNodalOffsetAlgoManual::AddNodalPointAtEvalInputs()
{
	if (CachedEvalInputs.bIsValid)
	{
		if (const FCameraCalibrationStepsController* const StepsController = GetStepsController())
		{
			if (ULensFile* const LensFile = StepsController->GetLensFile())
			{
				// Query the Lens File for the value of the interpolated nodal point offset at the current Focus and Zoom
				FNodalPointOffset NewNodalOffset;
				LensFile->EvaluateNodalPointOffset(CachedEvalInputs.Focus, CachedEvalInputs.Zoom, NewNodalOffset);

				LensFile->AddNodalOffsetPoint(CachedEvalInputs.Focus, CachedEvalInputs.Zoom, NewNodalOffset);
			}
		}
	}
	return FReply::Handled();
}

bool UCameraNodalOffsetAlgoManual::DoesNodalPointExist() const
{
	return bNodalPointExists;
}

EVisibility UCameraNodalOffsetAlgoManual::GetNodalPointVisibility(bool bShowIfNodalPointExists) const
{
	if (bShowIfNodalPointExists == bNodalPointExists)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

double UCameraNodalOffsetAlgoManual::GetSensitivity() const
{
	switch (CurrentSensitivity)
	{
	case EAdjustmentSensitivity::Fine:
		return 0.01;
	case EAdjustmentSensitivity::Medium:
		return 0.1;
	case EAdjustmentSensitivity::Coarse:
		return 1.0;
	default:
		return 0.0;
	}
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoManual::BuildSensitivityWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.IsChecked_UObject(this, &UCameraNodalOffsetAlgoManual::IsSensitivityOptionSelected, EAdjustmentSensitivity::Fine)
			.OnCheckStateChanged_UObject(this, &UCameraNodalOffsetAlgoManual::OnSensitivityChanged, EAdjustmentSensitivity::Fine)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(LOCTEXT("FineButtonText", "Fine"))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.IsChecked_UObject(this, &UCameraNodalOffsetAlgoManual::IsSensitivityOptionSelected, EAdjustmentSensitivity::Medium)
			.OnCheckStateChanged_UObject(this, &UCameraNodalOffsetAlgoManual::OnSensitivityChanged, EAdjustmentSensitivity::Medium)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(LOCTEXT("MediumButtonText", "Medium"))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.IsChecked_UObject(this, &UCameraNodalOffsetAlgoManual::IsSensitivityOptionSelected, EAdjustmentSensitivity::Coarse)
			.OnCheckStateChanged_UObject(this, &UCameraNodalOffsetAlgoManual::OnSensitivityChanged, EAdjustmentSensitivity::Coarse)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(LOCTEXT("CoarseButtonText", "Coarse"))
			]
		]
	;
}

ECheckBoxState UCameraNodalOffsetAlgoManual::IsSensitivityOptionSelected(EAdjustmentSensitivity InSensitivity) const
{
	return CurrentSensitivity == InSensitivity ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void UCameraNodalOffsetAlgoManual::OnSensitivityChanged(ECheckBoxState CheckBoxState, EAdjustmentSensitivity InSensitivity)
{
	if (CheckBoxState == ECheckBoxState::Checked)
	{
		CurrentSensitivity = InSensitivity;
	}
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoManual::BuildLocationWidget()
{
	return SNew(SNumericVectorInputBox<double>)
		.bColorAxisLabels(true)
		.AllowSpin(true)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled_UObject(this, &UCameraNodalOffsetAlgoManual::DoesNodalPointExist)
		.X_UObject(this, &UCameraNodalOffsetAlgoManual::GetLocation, EAxis::X)
		.Y_UObject(this, &UCameraNodalOffsetAlgoManual::GetLocation, EAxis::Y)
		.Z_UObject(this, &UCameraNodalOffsetAlgoManual::GetLocation, EAxis::Z)
		.OnXChanged_UObject(this, &UCameraNodalOffsetAlgoManual::SetLocation, EAxis::X)
		.OnYChanged_UObject(this, &UCameraNodalOffsetAlgoManual::SetLocation, EAxis::Y)
		.OnZChanged_UObject(this, &UCameraNodalOffsetAlgoManual::SetLocation, EAxis::Z)
		.SpinDelta_UObject(this, &UCameraNodalOffsetAlgoManual::GetSensitivity)
		.OnBeginSliderMovement_UObject(this, &UCameraNodalOffsetAlgoManual::OnBeginSlider)
		.OnEndSliderMovement_UObject(this, &UCameraNodalOffsetAlgoManual::OnEndSlider)
	;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoManual::BuildRotationWidget()
{
	return SNew(SNumericVectorInputBox<double>)
		.bColorAxisLabels(true)
		.AllowSpin(true)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled_UObject(this, &UCameraNodalOffsetAlgoManual::DoesNodalPointExist)
		.X_UObject(this, &UCameraNodalOffsetAlgoManual::GetRotation, EAxis::X)
		.Y_UObject(this, &UCameraNodalOffsetAlgoManual::GetRotation, EAxis::Y)
		.Z_UObject(this, &UCameraNodalOffsetAlgoManual::GetRotation, EAxis::Z)
		.OnXChanged_UObject(this, &UCameraNodalOffsetAlgoManual::SetRotation, EAxis::X)
		.OnYChanged_UObject(this, &UCameraNodalOffsetAlgoManual::SetRotation, EAxis::Y)
		.OnZChanged_UObject(this, &UCameraNodalOffsetAlgoManual::SetRotation, EAxis::Z)
		.SpinDelta_UObject(this, &UCameraNodalOffsetAlgoManual::GetSensitivity)
		.OnBeginSliderMovement_UObject(this, &UCameraNodalOffsetAlgoManual::OnBeginSlider)
		.OnEndSliderMovement_UObject(this, &UCameraNodalOffsetAlgoManual::OnEndSlider)
		;
}

TOptional<double> UCameraNodalOffsetAlgoManual::GetLocation(EAxis::Type Axis) const
{
	switch (Axis)
	{
	case EAxis::X:
		return CurrentNodalOffset.LocationOffset.X;
	case EAxis::Y:
		return CurrentNodalOffset.LocationOffset.Y;
	case EAxis::Z:
		return CurrentNodalOffset.LocationOffset.Z;
	default:
		return TOptional<double>();
	}
}

TOptional<double> UCameraNodalOffsetAlgoManual::GetRotation(EAxis::Type Axis) const
{
	const FRotator Rotator = CurrentNodalOffset.RotationOffset.Rotator();

	switch (Axis)
	{
	case EAxis::X:
		return Rotator.Roll;
	case EAxis::Y:
		return Rotator.Pitch;
	case EAxis::Z:
		return Rotator.Yaw;
	default:
		return TOptional<double>();
	}
}

void UCameraNodalOffsetAlgoManual::SetLocation(double NewValue, EAxis::Type Axis)
{
	FVector Diff = FVector::ZeroVector;

	if (Axis == EAxis::X)
	{
		if (FMath::IsNearlyEqual(CurrentNodalOffset.LocationOffset.X, NewValue))
		{
			return;
		}
		Diff.X = NewValue - CurrentNodalOffset.LocationOffset.X;
	}
	else if (Axis == EAxis::Y)
	{
		if (FMath::IsNearlyEqual(CurrentNodalOffset.LocationOffset.Y, NewValue))
		{
			return;
		}
		Diff.Y = NewValue - CurrentNodalOffset.LocationOffset.Y;
	}
	else if (Axis == EAxis::Z)
	{
		if (FMath::IsNearlyEqual(CurrentNodalOffset.LocationOffset.Z, NewValue))
		{
			return;
		}
		Diff.Z = NewValue - CurrentNodalOffset.LocationOffset.Z;
	}

	UpdateNodalOffset(Diff, FRotator::ZeroRotator);
}

void UCameraNodalOffsetAlgoManual::SetRotation(double NewValue, EAxis::Type Axis)
{
	const FRotator CurrentRotation = CurrentNodalOffset.RotationOffset.Rotator();

	FRotator Diff = FRotator::ZeroRotator;

	if (Axis == EAxis::X)
	{
		if (FMath::IsNearlyEqual(CurrentRotation.Roll, NewValue))
		{
			return;
		}
		Diff.Roll = NewValue - CurrentRotation.Roll;
	}
	else if (Axis == EAxis::Y)
	{
		if (FMath::IsNearlyEqual(CurrentRotation.Pitch, NewValue))
		{
			return;
		}
		Diff.Pitch = NewValue - CurrentRotation.Pitch;
	}
	else if (Axis == EAxis::Z)
	{
		if (FMath::IsNearlyEqual(CurrentRotation.Yaw, NewValue))
		{
			return;
		}
		Diff.Yaw = NewValue - CurrentRotation.Yaw;
	}

	UpdateNodalOffset(FVector::ZeroVector, Diff);
}

ACameraActor* UCameraNodalOffsetAlgoManual::GetCamera()
{
	if (const FCameraCalibrationStepsController* const StepsController = GetStepsController())
	{
		return StepsController->GetCamera();
	}
	return nullptr;
}

FCameraCalibrationStepsController* UCameraNodalOffsetAlgoManual::GetStepsController()
{
	if (UNodalOffsetTool* NodalOffsetToolPtr = NodalOffsetTool.Get())
	{
		return NodalOffsetToolPtr->GetCameraCalibrationStepsController();
	}
	return nullptr;
}

void UCameraNodalOffsetAlgoManual::UpdateNodalOffset(FVector LocationOffset, FRotator RotationOffset)
{
	if (ACameraActor* const Camera = GetCamera())
	{
		// Interactive changes that result from moving the slider will cache these values in OnBeginSlider()
		if (!bIsSliderMoving)
		{
			BaseCameraTransform = Camera->GetActorTransform();
			BaseNodalOffset = CurrentNodalOffset;
		}

		// Add an additional offset to the camera. Interacting with the slider will block the game thread, so just modifying the nodal offset in the Lens File
		// will not be visible until the user releases the slider and the game thread can update on the next Tick. Manually offsetting the camera this way lets 
		// the camera's viewport re-render and ensures that the tool is interactive while the user makes fine adjustments.
		Camera->AddActorLocalOffset(LocationOffset);
		Camera->AddActorLocalRotation(RotationOffset);

		FTransform BaseNodalTransform;
		BaseNodalTransform.SetLocation(BaseNodalOffset.LocationOffset);
		BaseNodalTransform.SetRotation(BaseNodalOffset.RotationOffset);

		// Compute the new nodal offset as the offset from the camera's current pose from its most recent pose, factoring in the base nodal offset that was applied
		const FTransform NewNodalTransform = Camera->GetActorTransform() * BaseCameraTransform.Inverse() * BaseNodalTransform;
		CurrentNodalOffset.LocationOffset = NewNodalTransform.GetLocation();
		CurrentNodalOffset.RotationOffset = NewNodalTransform.GetRotation();

		// Update the value of the nodal offset point in the Lens File
		SaveNodalOffset(CurrentNodalOffset);
	}
}

void UCameraNodalOffsetAlgoManual::SaveNodalOffset(FNodalPointOffset NewNodalOffset)
{
	if (CachedEvalInputs.bIsValid)
	{
		if (const FCameraCalibrationStepsController* const StepsController = GetStepsController())
		{
			if (ULensFile* const LensFile = StepsController->GetLensFile())
			{
				// The transaction for interactive changes that result from moving the slider are handled in OnBeginSlider() / OnEndSlider()
				const bool bShouldTransact = !bIsSliderMoving;

				if (bShouldTransact)
				{
					GEditor->BeginTransaction(LOCTEXT("AdjustNodalOffset", "Manually Adjust Nodal Offset"));
				}

				LensFile->Modify();
				LensFile->AddNodalOffsetPoint(CachedEvalInputs.Focus, CachedEvalInputs.Zoom, NewNodalOffset);

				if (bShouldTransact)
				{
					GEditor->EndTransaction();
				}
			}
		}
	}
}

void UCameraNodalOffsetAlgoManual::OnBeginSlider()
{
	bIsSliderMoving = true;
	GEditor->BeginTransaction(LOCTEXT("AdjustNodalOffset", "Manually Adjust Nodal Offset"));

	if (const ACameraActor* const Camera = GetCamera())
	{
		BaseCameraTransform = Camera->GetActorTransform();
	}

	BaseNodalOffset = CurrentNodalOffset;
}

void UCameraNodalOffsetAlgoManual::OnEndSlider(double NewValue)
{
	bIsSliderMoving = false;
	GEditor->EndTransaction();
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoManual::BuildHelpWidget()
{
	return SNew(STextBlock)
		.Text(LOCTEXT("NodalOffsetAlgoPointsHelp",
			"Adjust the nodal offset of the camera by using the location and rotation widgets to manually\n"
			"align one or more features from the media plate and CG scene.\n\n"

			"You can adjust the sensitivity for finer or coarser adjustments.\n\n"

			"If the LensFile does not contain a nodal offset point at the current Focus and Zoom settings,\n"
			"you will see a prompt to add a new point before making adjustments to it. This can be useful\n"
			"for fine-tuning the nodal offset between two calibrated points if the interpolate offset is\n"
			"not accurate enough.\n\n"

			"Note: Adjustments are made relative to the current camera view. Therefore, as you change values\n"
			"in the location and rotation widgets, more than one value will likely change at the same time\n"
			"as the absolute nodal offset is recomputed based on the adjustments."
		));
}

#undef LOCTEXT_NAMESPACE
