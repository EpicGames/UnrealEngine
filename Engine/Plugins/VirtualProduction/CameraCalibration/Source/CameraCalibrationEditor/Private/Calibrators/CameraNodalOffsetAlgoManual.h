// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraNodalOffsetAlgo.h"

#include "LensFile.h"

#include "CameraNodalOffsetAlgoManual.generated.h"

class ACameraActor;
class FCameraCalibrationStepsController;

/**
 * Nodal Offset "algo" for making manual adjustments to the nodal offset using location and rotation widgets
 */
UCLASS()
class UCameraNodalOffsetAlgoManual : public UCameraNodalOffsetAlgo
{
	GENERATED_BODY()

public:
	/** Sensitivity options for interacting with the location and rotation widget sliders */
	enum class EAdjustmentSensitivity : uint8
	{
		Fine,
		Medium,
		Coarse
	};

	//~ Begin UCameraNodalOffsetAlgo
	virtual void Initialize(UNodalOffsetTool* InNodalOffsetTool) override;
	virtual void Tick(float DeltaTime) override;
	virtual TSharedRef<SWidget> BuildUI() override;

	virtual FName FriendlyName() const override 
	{ 
		return TEXT("Nodal Offset Manual"); 
	};

	virtual FName ShortName() const override 
	{ 
		return TEXT("Manual"); 
	};

	virtual TSharedRef<SWidget> BuildHelpWidget() override;
	//~ End UCameraNodalOffsetAlgo

protected:
	/** Returns the value of the nodal offset's location offset for the given axis */
	TOptional<double> GetLocation(EAxis::Type Axis) const;

	/** Returns the value of the nodal offset's rotation offset for the given axis */
	TOptional<double> GetRotation(EAxis::Type Axis) const;

	/** Updates the value of the nodal offset's location offset for the given axis to the input value */
	void SetLocation(double NewValue, EAxis::Type Axis);

	/** Updates the value of the nodal offset's rotation offset for the given axis to the input value */
	void SetRotation(double NewValue, EAxis::Type Axis);

	/** Updates the camera's pose by the new location and rotation offset and computes the new nodal offset that would get to the new camera pose */
	void UpdateNodalOffset(FVector LocationOffset, FRotator RotationOffset);

	/** Saves the input nodal offset to the Lens File at the current Focus and Zoom */
	void SaveNodalOffset(FNodalPointOffset NewNodalOffset);

	/** Triggered when the user begins dragging any of the location/rotation sliders, caching the current camera pose and nodal offset, and beginning a new transaction */
	void OnBeginSlider();

	/** Triggered when the uesr releases any of the location/rotation sliders, ending the current transaction */
	void OnEndSlider(double NewValue);

	/** Reset the nodal offset at the current Focus and Zoom to their values before the user began making any manual adjustments */
	FReply OnUndoManualChanges();

	/** Add a new nodal offset point to the Lens File at the current Focus and Zoom */
	FReply AddNodalPointAtEvalInputs();

	/** Create the sensitivity checkbox widgets */
	TSharedRef<SWidget> BuildSensitivityWidget();

	/** Create the location widget */
	TSharedRef<SWidget> BuildLocationWidget();

	/** Create the roation widget */
	TSharedRef<SWidget> BuildRotationWidget();

	/** Determines widget visibility based on whether there is a nodal offset point at the current Focus and Zoom */
	EVisibility GetNodalPointVisibility(bool bShowIfNodalPointExists) const;

	/** Returns true if there is a nodal offset point at the current Focus and Zoom */
	bool DoesNodalPointExist() const;

	/** Returns a value corresponding to the currently selected sensitivity to drive the delta of the location and rotation sliders */
	double GetSensitivity() const;

	/** Determines whether the input sensitivity option is currently selected */
	ECheckBoxState IsSensitivityOptionSelected(EAdjustmentSensitivity InSensitivity) const;

	/** Changes the current adjustment sensitivity */
	void OnSensitivityChanged(ECheckBoxState CheckBoxState, EAdjustmentSensitivity InSensitivity);

	/** Returns the camera actor that the nodal offset is being applied to */
	ACameraActor* GetCamera();

	/** Returns the steps controller */
	FCameraCalibrationStepsController* GetStepsController();

protected:
	/** Weak pointer to the tool that created this algo */
	TWeakObjectPtr<UNodalOffsetTool> NodalOffsetTool;

	/** The most recent Focus and Zoom values used to evaluate the Lens File */
	FLensFileEvaluationInputs CachedEvalInputs;

	/** The value of the nodal offset before the user begins to make any manual adjustments. This is reset any time the Focus and Zoom are changed. */
	FNodalPointOffset StartingNodalOffset;

	/** The current value of the nodal offset at the current Focus and Zoom */
	FNodalPointOffset CurrentNodalOffset;

	/** Cached value of the nodal offset immediately before an adjustment is made (or when the user begins dragging to slider) to compute the new nodal offset from the updated camera pose */
	FNodalPointOffset BaseNodalOffset;

	/** Cached value of the camera pose immediately before an adjustment is made (or when the user begins dragging to slider) to compute the new nodal offset from the updated camera pose */
	FTransform BaseCameraTransform;

	/** Whether there is a nodal offset point in the Lens File at the current Focus and Zoom */
	bool bNodalPointExists = true;

	/** Whether the user is currently moving any of the location/rotation sliders */
	bool bIsSliderMoving = false;

	/** The current sensitivitiy, used to set the delta of the location/rotation sliders */
	EAdjustmentSensitivity CurrentSensitivity;
};
