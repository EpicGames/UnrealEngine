// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraNodalOffsetAlgoOpticalAxis.h"

#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/NodalOffsetTool.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationUtils.h"
#include "Kismet/KismetMathLibrary.h"
#include "LensDistortionModelHandlerBase.h"
#include "Misc/MessageDialog.h"
#include "OpenCVHelper.h"
#include "OverlayRendering.h"
#include "ScopedTransaction.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "UObject/Package.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"


#define LOCTEXT_NAMESPACE "CameraNodalOffsetAlgoOpticalAxis"


void UCameraNodalOffsetAlgoOpticalAxis::Initialize(UNodalOffsetTool* InNodalOffsetTool)
{
	Super::Initialize(InNodalOffsetTool);

	OpticalAxis.Reset();

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return;
	}

	// Create the render target for the crosshair overlay
	FIntPoint OverlaySize = StepsController->GetMediaOverlayResolution();
	if (OverlaySize.SizeSquared() == 0)
	{
		// If the media overlay does not have a valid resolution yet, use a default size
		OverlaySize = FIntPoint(1920, 1080);
	}

	OverlayTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass()));

	OverlayTexture->RenderTargetFormat = RTF_RGBA16f;
	OverlayTexture->ClearColor = FLinearColor::Transparent;
	OverlayTexture->bAutoGenerateMips = false;
	OverlayTexture->InitAutoFormat(OverlaySize.X, OverlaySize.Y);
	OverlayTexture->UpdateResourceImmediate(true);

	if (UMaterialInstanceDynamic* OverlayMID = NodalOffsetTool->GetOverlayMID())
	{
		OverlayMID->SetTextureParameterValue(FName(TEXT("OutputTexture")), OverlayTexture);
	}

	// Initialize sensitivity options for the combobox
	SensitivityOptions.Add(MakeShared<float>(0.001f));
	SensitivityOptions.Add(MakeShared<float>(0.005f));
	SensitivityOptions.Add(MakeShared<float>(0.01f));
	SensitivityOptions.Add(MakeShared<float>(0.05f));
	SensitivityOptions.Add(MakeShared<float>(0.1f));
	SensitivityOptions.Add(MakeShared<float>(0.5f));
}

void UCameraNodalOffsetAlgoOpticalAxis::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	if (!NodalOffsetTool.IsValid())
	{
		return;
	}

	FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return;
	}

	const ULensDistortionModelHandlerBase* DistortionHandler = StepsController->GetDistortionHandler();

	if (!DistortionHandler)
	{
		return;
	}

	const FLensDistortionState DistortionState = DistortionHandler->GetCurrentDistortionState();

	// Inform the user if the focus or zoom has changed since the last tick
	if (OpticalAxis.IsSet() && !bFocusZoomWarningIssued)
	{
		const FLensFileEvaluationInputs EvalInputs = StepsController->GetLensFileEvaluationInputs();
		if (EvalInputs.bIsValid)
		{
			const float InputTolerance = GetDefault<UCameraCalibrationSettings>()->GetCalibrationInputTolerance();
			if (!FMath::IsNearlyEqual(CachedFocus, EvalInputs.Focus, InputTolerance) || !FMath::IsNearlyEqual(CachedZoom, EvalInputs.Zoom, InputTolerance))
			{
				FFormatOrderedArguments Arguments;
				Arguments.Add(FText::FromString(FString::Printf(TEXT("%.2f"), CachedFocus)));
				Arguments.Add(FText::FromString(FString::Printf(TEXT("%.2f"), CachedZoom)));

				FText Message = FText::Format(LOCTEXT("Focus or Zoom has changed",
					"The focus or zoom input has changed. You may continue to modify the entrance pupil position, "
					"but note that those modifications will affect the nodal offset at the previous values of "
					"Focus = {0}, Zoom = {1}. You may calibrate at the new focus and zoom values by capturing new "
					"points and clicking 'Add To Nodal Offset Calibration'."
				), Arguments);

				FMessageDialog::Open(EAppMsgType::Ok, Message);

				bFocusZoomWarningIssued = true;
			}
		}
	}

	// Update the crosshair overlay if the principal point has changed
	if (LastPrincipalPoint != DistortionState.ImageCenter.PrincipalPoint)
	{
		FCrosshairOverlayParams OverlayParams;
		OverlayParams.PrincipalPoint = FVector2f(DistortionState.ImageCenter.PrincipalPoint);
		OverlayRendering::DrawCrosshairOverlay(OverlayTexture, OverlayParams);

		LastPrincipalPoint = DistortionState.ImageCenter.PrincipalPoint;
	}
}

bool UCameraNodalOffsetAlgoOpticalAxis::OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent)
{
	if (!OpticalAxis.IsSet())
	{
		return false;
	}

	bool bHandled = false;

	// Arrow Key handling
	if (InEvent == EInputEvent::IE_Pressed || InEvent == EInputEvent::IE_Repeat)
	{
		if (InKey == EKeys::Up)
		{
			EntrancePupilPosition -= (AdjustmentIncrement);
			bHandled = true;
		}
		else if (InKey == EKeys::Down)
		{
			EntrancePupilPosition += (AdjustmentIncrement);
			bHandled = true;
		}

		if (bHandled)
		{
			UpdateNodalOffset();
		}
	}
	else if (InEvent == EInputEvent::IE_Released)
	{
		// Only trigger a transaction when the key is released
		if (InKey == EKeys::Up || InKey == EKeys::Down)
		{
			SaveUpdatedNodalOffset();
			bHandled = true;
		}
	}

	return bHandled;
}

bool UCameraNodalOffsetAlgoOpticalAxis::GetNodalOffset(FNodalPointOffset& OutNodalOffset, float& OutFocus, float& OutZoom, float& OutError, FText& OutErrorMessage)
{
	if (!NodalOffsetTool.IsValid())
	{
		return false;
	}

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return false;
	}

	const ULensFile* LensFile = StepsController->GetLensFile();

	if (!LensFile)
	{
		return false;
	}

	const ULensDistortionModelHandlerBase* DistortionHandler = StepsController->GetDistortionHandler();

	if (!DistortionHandler)
	{
		return false;
	}

	// Validate that there are enough points to compute an optical axis
 	if (CalibrationRows.Num() < 2)
 	{
 		OutErrorMessage = LOCTEXT("AtLeastTwoPoints", "At least two points are required");
 		return false;
 	}

	// Validate that all rows have approximately the same camera pose
	const TSharedPtr<FNodalOffsetPointsRowData>& FirstRow = CalibrationRows[0];
	const FTransform CameraPose = FirstRow->CameraData.Pose;
	CachedInverseCameraPose = CameraPose.Inverse();

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : CalibrationRows)
	{
		if (!FCameraCalibrationUtils::IsNearlyEqual(CameraPose, Row->CameraData.Pose))
		{
			OutErrorMessage = LOCTEXT("CameraMoved", "Camera moved too much between samples.");
			return false;
		}
	}

	// Cache the evaluated focus and zoom so that modifications to the nodal offset table will update the same point
	CachedFocus = FirstRow->CameraData.InputFocus;
	CachedZoom = FirstRow->CameraData.InputZoom;

	OutFocus = CachedFocus;
	OutZoom = CachedZoom;

	// Gather the 3D points from each of the calibration rows
	const int32 NumPoints = CalibrationRows.Num();

	TArray<FVector> Points3d;
	Points3d.Reserve(NumPoints);

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : CalibrationRows)
	{
		Points3d.Add(Row->CalibratorPointData.Location);
	}

	FVector FitLine;
	FOpenCVHelper::FitLine3D(Points3d, FitLine, PointAlongAxis);
	OpticalAxis = FitLine;

	// Find a look at rotation for the camera
	const FVector Point0 = PointAlongAxis + (0.0f * OpticalAxis.GetValue());
	const FVector Point1 = PointAlongAxis - (1.0f * OpticalAxis.GetValue());

	LookAtRotation = UKismetMathLibrary::FindLookAtRotation(Point1, Point0);

	// Cache the existing nodal offset, if one exists
	CachedNodalOffset = FTransform::Identity;

	if (FirstRow->CameraData.bWasNodalOffsetApplied)
	{
		FNodalPointOffset NodalPointOffset;

		if (LensFile->EvaluateNodalPointOffset(OutFocus, OutZoom, NodalPointOffset))
		{
			CachedNodalOffset.SetTranslation(NodalPointOffset.LocationOffset);
			CachedNodalOffset.SetRotation(NodalPointOffset.RotationOffset);
		}
	}

	// Compute an initial nodal offset transform with the Entrance Pupil Position equal to zero
	EntrancePupilPosition = 0.0f;
	OutNodalOffset = ComputeNodalOffset();

	LastSavedNodalOffset = OutNodalOffset;
	LastSavedEntrancePupilPosition = EntrancePupilPosition;

	// Validate that the captured points all project to approximately the principal point,
	// which would indicate that they were in roughly a straight line. 
	const FLensDistortionState DistortionState = DistortionHandler->GetCurrentDistortionState();

	const FVector2D FocalLength = DistortionState.FocalLengthInfo.FxFy;
	const FVector2D ImageCenter = DistortionState.ImageCenter.PrincipalPoint;

	// Compute the optimal camera pose using the new nodal offset and convert it to opencv's coordinate system
	FTransform NewNodalOffsetTransform;
	NewNodalOffsetTransform.SetLocation(OutNodalOffset.LocationOffset);
	NewNodalOffsetTransform.SetRotation(OutNodalOffset.RotationOffset);

	FTransform OptimalCameraPose = NewNodalOffsetTransform * CachedNodalOffset.Inverse() * CameraPose;

	// Initialize all of the corresponding 2D points to be exactly at the principal point of the lens
	TArray<FVector2f> ImagePoints;
	ImagePoints.Init(FVector2f(ImageCenter.X, ImageCenter.Y), NumPoints);

	OutError = FOpenCVHelper::ComputeReprojectionError(Points3d, ImagePoints, FocalLength, ImageCenter, OptimalCameraPose);

	bFocusZoomWarningIssued = false;

	return true;
}

UMaterialInterface* UCameraNodalOffsetAlgoOpticalAxis::GetOverlayMaterial() const
{
	return TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibration/Materials/M_OutputTexture.M_OutputTexture"))).LoadSynchronous();
}

bool UCameraNodalOffsetAlgoOpticalAxis::IsOverlayEnabled() const
{
	// Always display the crosshair overlay while this algo is active
	return true;
}

FNodalPointOffset UCameraNodalOffsetAlgoOpticalAxis::ComputeNodalOffset() const
{
	const FVector CameraLocation = PointAlongAxis + (EntrancePupilPosition * OpticalAxis.GetValue());

	FTransform DesiredCameraTransform;
	DesiredCameraTransform.SetLocation(CameraLocation);
	DesiredCameraTransform.SetRotation(LookAtRotation.Quaternion());

	const FTransform DesiredOffset = DesiredCameraTransform * CachedInverseCameraPose * CachedNodalOffset;

	FNodalPointOffset NodalOffset;
	NodalOffset.LocationOffset = DesiredOffset.GetLocation();
	NodalOffset.RotationOffset = DesiredOffset.GetRotation();

	return NodalOffset;
}

bool UCameraNodalOffsetAlgoOpticalAxis::UpdateNodalOffset()
{
	// Do not update if the optical axis has not been found yet
	if (!OpticalAxis.IsSet())
	{
		return false;
	}

	if (!NodalOffsetTool.IsValid())
	{
		return false;
	}

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return false;
	}

	ULensFile* LensFile = StepsController->GetLensFile();

	if (!LensFile)
	{
		return false;
	}

	AdjustedNodalOffset = ComputeNodalOffset();

	// Attempt to modify an existing point in the nodal offset table
	bool bPointExistsInTable = LensFile->NodalOffsetTable.SetPoint(CachedFocus, CachedZoom, AdjustedNodalOffset);

	return bPointExistsInTable;
}

void UCameraNodalOffsetAlgoOpticalAxis::SaveUpdatedNodalOffset()
{
	if (!NodalOffsetTool.IsValid())
	{
		return;
	}

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return;
	}

	ULensFile* LensFile = StepsController->GetLensFile();

	if (!LensFile)
	{
		return;
	}

	// Before beginning a transaction, restore the nodal offset and entrance pupil position to their most recently saved states
	const FNodalPointOffset CurrentNodalOffset = AdjustedNodalOffset;
	const float CurrentEntrancePupilPosition = EntrancePupilPosition;

	AdjustedNodalOffset = LastSavedNodalOffset;
	EntrancePupilPosition = LastSavedEntrancePupilPosition;

	LensFile->NodalOffsetTable.SetPoint(CachedFocus, CachedZoom, LastSavedNodalOffset);
	{
		FScopedTransaction Transaction(LOCTEXT("SaveAdjustedNodalOffset", "Save Adjusted Nodal Offset"));
		this->Modify();
		LensFile->Modify();

		// After beginning the transaction, reapply the final modified nodal offset and entrance pupil position
		AdjustedNodalOffset = CurrentNodalOffset;
		EntrancePupilPosition = CurrentEntrancePupilPosition;
		LensFile->NodalOffsetTable.SetPoint(CachedFocus, CachedZoom, AdjustedNodalOffset);

		// Reset the last saved nodal offset and entrance pupil position to the adjusted values to indicate that the change has been written
		LastSavedNodalOffset = AdjustedNodalOffset;
		LastSavedEntrancePupilPosition = EntrancePupilPosition;
	}
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoOpticalAxis::BuildUI()
{
	return SNew(SVerticalBox)

	+ SVerticalBox::Slot() // Calibrator picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Calibrator", "Calibrator"), BuildCalibrationDevicePickerWidget())]

	+ SVerticalBox::Slot() // Calibrator component picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorComponents", "Calibrator Component(s)"), BuildCalibrationComponentPickerWidget())]

	+ SVerticalBox::Slot() // Calibrator point names
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorPoint", "Calibrator Point"), BuildCalibrationPointsComboBox())]

	+ SVerticalBox::Slot() // Entrance Pupil Position
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("EntrancePupilPosition", "Entrance Pupil Position"), BuildEntrancePupilPositionWidget())]

	+ SVerticalBox::Slot() // Entrance Pupil Position
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Sensitivity", "Sensitivity"), BuildSensitivityWidget())]

	+ SVerticalBox::Slot() // Calibration Rows
		.AutoHeight()
		.MaxHeight(12 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[BuildCalibrationPointsTable()]

	+ SVerticalBox::Slot() // Row manipulation
		.AutoHeight()
		[
			SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot() // Button to clear all rows
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearAll", "Clear All"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([&]() -> FReply
				{
					ClearCalibrationRows();
					return FReply::Handled();
				})
			]
		]
	;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoOpticalAxis::BuildEntrancePupilPositionWidget()
{
	return SNew(SNumericEntryBox<float>)
		.Value_Lambda([&]() { return EntrancePupilPosition; })
		.ToolTipText(LOCTEXT("EntrancePupil", "Entrance Pupil"))
		.IsEnabled_Lambda([&] { return OpticalAxis.IsSet(); })
		.OnValueChanged_Lambda([&](double InValue)
		{
			EntrancePupilPosition = InValue;
			UpdateNodalOffset();
			SaveUpdatedNodalOffset();
		});
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoOpticalAxis::BuildSensitivityWidget()
{
	return SNew(SComboBox<TSharedPtr<float>>)
		.OptionsSource(&SensitivityOptions)
		.IsEnabled_Lambda([&] { return OpticalAxis.IsSet(); })
		.OnSelectionChanged_Lambda([&](TSharedPtr<float> NewValue, ESelectInfo::Type Type) -> void
		{
			AdjustmentIncrement = *NewValue;
		})
		.OnGenerateWidget_Lambda([&](TSharedPtr<float> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock).Text(FText::AsNumber(*InOption));
		})
		.InitiallySelectedItem(SensitivityOptions[2])
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				return FText::AsNumber(AdjustmentIncrement);
			})
		];
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoOpticalAxis::BuildHelpWidget()
{
	return SNew(STextBlock)
		.Text(LOCTEXT("NodalOffsetAlgoOpticalAxisHelp",
			"This nodal offset algorithm will find the nodal point of the lens and compute an offset based\n"
			"on the current camera pose. First, it will use a set of 3D points that project to the center\n"
			"of the image to find the optical axis of the lens. Then it will find the nodal point by\n"
			"finding the entrance pupil point which lies along the optical axis.\n\n"

			"The 3D points are taken from the calibrator object, which you need to select using the\n"
			"provided picker. All that is required is that the object contains one or more 'Calibration\n'"
			"Point Components'. These 3d calibration points will appear in the provided drop-down.\n"
			"The calibrator you use should have some identifiable marker that will be easy to locate\n"
			"in the viewport (such as a bright LED).\n\n"

			"To build the set of 3d points that lie along the optical axis, align the marker on your\n"
			"calibrator to the center of the crosshair that appears over the viewport. Note that the\n"
			"crosshair may not be centered at the precise center of the image, but rather at the principal\n"
			"point (which should already have been calibrated in the lens distortion calibration). You will\n"
			"need to capture at least two points to find the optical axis, but more points may give more\n"
			"accurate results.\n\n"

			"Once those points are captured, the algorithm will find a best-fit line that passes through\n"
			"the set of points. This is the optical axis of the lens, and the nodal point lies somewhere\n"
			"along that axis.\n\n"

			"The final step is to adjust the entrance pupil position until a tracked object in the scene\n"
			"is visibly aligned with its CG counterpart in the simulcam viewport.\n\n"

			"Notes:\n\n"
			" - This calibration step relies on the camera having a lens distortion calibration.\n"
			" - It requires the camera to not move much from the moment you capture the first\n"
			"   point until you capture the last one.\n"
		));
}

#undef LOCTEXT_NAMESPACE
