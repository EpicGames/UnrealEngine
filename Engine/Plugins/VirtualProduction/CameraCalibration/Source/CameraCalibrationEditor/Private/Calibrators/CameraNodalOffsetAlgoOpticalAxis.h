// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraNodalOffsetAlgoPoints.h"

#include "CameraNodalOffsetAlgoOpticalAxis.generated.h"

/**
 * Implements a nodal offset calibration algorithm that relies on the idea that the nodal point of the lens
 * lies along the optical axis of the lens, and that the optical axis can be found by finding multiple points
 * in 3D that map to the principal point of the lens in 2D.
 *
 * The algorithm uses 3d points (UCalibrationPointComponent) specified in a selectable calibrator with features 
 * that the user can identify. By aligning that feature such that it projects to the principal point of the lens
 * (easily identifiable using the automatically enabled crosshair overlay), the user captures two or more 3D
 * points. The algorithm computes a line that connects those points, pointing away from the camera, which will
 * be the optical axis. The nodal point is then found by moving the camera along that optical axis until the 
 * calibrator appears perfectly aligned in the composite image (CG + Media).
 */
UCLASS()
class UCameraNodalOffsetAlgoOpticalAxis : public UCameraNodalOffsetAlgoPoints
{
	GENERATED_BODY()

public:
	//~ Begin CalibPointsNodalOffsetAlgo
	virtual void Initialize(UNodalOffsetTool* InNodalOffsetTool) override;
	virtual void Tick(float DeltaTime) override;
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual bool GetNodalOffset(FNodalPointOffset& OutNodalOffset, float& OutFocus, float& OutZoom, float& OutError, FText& OutErrorMessage) override;
	virtual FName FriendlyName() const override { return TEXT("Nodal Offset Optical Axis"); };
	virtual FName ShortName() const override { return TEXT("OpticalAxis"); };
	virtual UMaterialInterface* GetOverlayMaterial() const override;
	virtual bool IsOverlayEnabled() const override;
	virtual TSharedRef<SWidget> BuildHelpWidget() override;
	//~ End CalibPointsNodalOffsetAlgo

protected:
	/** The axis passing through the center of the lens, on which lies the nodal point */
	TOptional<FVector> OpticalAxis;

	/** A point that lies along the optical axis (not necessarily the nodal point) */
	FVector PointAlongAxis = FVector::ZeroVector;

	/** The rotation in which the camera is looking (along the optical axis) */
	FRotator LookAtRotation = FRotator::ZeroRotator;

	/** The current value of the nodal offset that has been written to the lens file, but not necessarily transacted */
	UPROPERTY(Transient)
	FNodalPointOffset AdjustedNodalOffset;

	/** The last nodal offset to have been saved and transacted */
	UPROPERTY(Transient)
	FNodalPointOffset LastSavedNodalOffset;

	/** The parametric position along the optical axis where the nodal points sits */
	UPROPERTY(Transient)
	float EntrancePupilPosition = 0.0f;

	/** The last entrance pupil position to have been saved and transacted */
	UPROPERTY(Transient)
	float LastSavedEntrancePupilPosition = 0.0f;

	/** Cached inverse of the camera pose when the points were captured */
	FTransform CachedInverseCameraPose = FTransform::Identity;

	/** Cached nodal offset of the camera when the points were captured */
	FTransform CachedNodalOffset = FTransform::Identity;

	/** Cached evaluated focus and zoom values when the points were captured */
	float CachedFocus = 0.0f;
	float CachedZoom = 0.0f;

	/** Texture used to draw the crosshair overlay */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> OverlayTexture;

	/** Increment (in pixels) that the entrance pupil position will be adjusted by each individual key press */
	float AdjustmentIncrement = 0.05f;

	/** Array of sensitivity options to display to the user to set the adjustment increment */
	TArray<TSharedPtr<float>> SensitivityOptions;

	/** Used to avoid spamming the user with warnings about changing focus/zoom values */
	bool bFocusZoomWarningIssued = false;

	/** Used to track changes to the principal point so that the crosshair overlay can be updated */
	FVector2D LastPrincipalPoint = FVector2D::ZeroVector;

protected:
	/** Build a widget that allows the user to manually enter a value for the entrance pupil position */
	TSharedRef<SWidget> BuildEntrancePupilPositionWidget();

	/** Builds the UI of the sensitivity widget */
	TSharedRef<SWidget> BuildSensitivityWidget();

	/** Compute the nodal offset based on the cached data and the current entrance pupil position */
	FNodalPointOffset ComputeNodalOffset() const;

	/** Recomputes the nodal offset and updates the lens file table */
	bool UpdateNodalOffset();

	/** Modifies the Lens File so that there is a transaction associated with the most recent nodal offset update */
	void SaveUpdatedNodalOffset();
};
