// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraNodalOffsetAlgoPoints.h"
#include "AssetEditor/CameraCalibrationStepsController.h"
#include "AssetEditor/NodalOffsetTool.h"
#include "CalibrationPointComponent.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationSolver.h"
#include "CameraCalibrationUtils.h"
#include "CineCameraComponent.h"
#include "DistortionRenderingUtils.h"
#include "Dom/JsonObject.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Input/Events.h"
#include "Internationalization/Text.h"
#include "JsonObjectConverter.h"
#include "Layout/Geometry.h"
#include "LensComponent.h"
#include "LensDistortionModelHandlerBase.h"
#include "LensFile.h"
#include "Math/Vector.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "OpenCVHelper.h"
#include "PropertyCustomizationHelpers.h"
#include "UI/SFilterableActorPicker.h"
#include "ScopedTransaction.h"
#include "UI/CameraCalibrationWidgetHelpers.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "CameraNodalOffsetAlgoPoints"

const int UCameraNodalOffsetAlgoPoints::DATASET_VERSION = 1;

// String constants for import/export
namespace UE::CameraCalibration::Private::NodalOffsetPointsExportFields
{
	static const FString Version(TEXT("Version"));
}

namespace UE::NodalOffsetAlgoPointsAnalytics
{
	void RecordEvent(FString FriendlyName, int32 DatasetSize)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;

			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Pattern"), *FriendlyName));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DatasetSize"), FString::Printf(TEXT("%d"), DatasetSize)));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("CameraCalibration.NodalOffsetCalibrationStarted"), EventAttributes);
		}
	}
}

namespace CameraNodalOffsetAlgoPoints
{
	class SCalibrationRowGenerator
		: public SMultiColumnTableRow<TSharedPtr<FNodalOffsetPointsRowData>>
	{

	public:
		SLATE_BEGIN_ARGS(SCalibrationRowGenerator) {}

		/** The list item for this row */
		SLATE_ARGUMENT(TSharedPtr<FNodalOffsetPointsRowData>, CalibrationRowData)

		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
		{
			CalibrationRowData = Args._CalibrationRowData;

			SMultiColumnTableRow<TSharedPtr<FNodalOffsetPointsRowData>>::Construct(
				FSuperRowType::FArguments()
				.Padding(1.0f),
				OwnerTableView
			);
		}

		//~Begin SMultiColumnTableRow
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == TEXT("Name"))
			{
				const FString Text = CalibrationRowData->CalibratorPointData.Name;
				return SNew(STextBlock).Text(FText::FromString(Text));
			}

			if (ColumnName == TEXT("Point2D"))
			{
				const FString Text = FString::Printf(TEXT("(%.2f, %.2f)"),
					CalibrationRowData->Point2D.X,
					CalibrationRowData->Point2D.Y);

				return SNew(STextBlock).Text(FText::FromString(Text));
			}

			if (ColumnName == TEXT("Point3D"))
			{
				const FString Text = FString::Printf(TEXT("(%.0f, %.0f, %.0f)"),
					CalibrationRowData->CalibratorPointData.Location.X,
					CalibrationRowData->CalibratorPointData.Location.Y,
					CalibrationRowData->CalibratorPointData.Location.Z);

				return SNew(STextBlock).Text(FText::FromString(Text));
			}

			return SNullWidget::NullWidget;
		}
		//~End SMultiColumnTableRow


	private:
		TSharedPtr<FNodalOffsetPointsRowData> CalibrationRowData;
	};

	/** Contains basic result of a nodal offset calibration based on a single camera pose for all samples */
	struct FSinglePoseResult
	{
		/** Tranform that can be a world coordinate or an offset */
		FTransform Transform;

		/** Number of calibration samples/rows to generate this result */
		int32 NumSamples;
	};

	/** 
	 * Weight-averages the transform of all single camera pose results. 
	 * Weights are given by relative number of samples used for each calibration result.
	 * 
	 * @param SinglePoseResults Array with independent calibration results for each camera pose
	 * @param OutAvgTransform Weighted average of 
	 * 
	 * @result True if successful
	 */
	bool AverageSinglePoseResults(const TArray<FSinglePoseResult>& SinglePoseResults, FTransform& OutAvgTransform)
	{
		// Calculate the total number of samples in order to later calculate the weights of each single pose result.

		int32 TotalNumSamples = 0;

		for (const FSinglePoseResult& SinglePoseResult : SinglePoseResults)
		{
			TotalNumSamples  += SinglePoseResult.NumSamples;
		}

		if (TotalNumSamples < 1)
		{
			return false;
		}

		// Average the location

		FVector AverageLocation(0);

		for (const FSinglePoseResult& SinglePoseResult : SinglePoseResults)
		{
			const float Weight = float(SinglePoseResult.NumSamples) / TotalNumSamples;
			AverageLocation += Weight * SinglePoseResult.Transform.GetLocation();
		}

		// Average the rotation

		FQuat::FReal AverageQuatVec[4] = { 0 }; // Simple averaging should work for similar quaterions, which these are.

		for (const FSinglePoseResult& SinglePoseResult : SinglePoseResults)
		{
			const FQuat Rotation = SinglePoseResult.Transform.GetRotation();

			const FQuat::FReal ThisQuat[4] = {
				Rotation.X,
				Rotation.Y,
				Rotation.Z,
				Rotation.W,
			};

			float Weight = float(SinglePoseResult.NumSamples) / TotalNumSamples;

			if ((Rotation | SinglePoseResults[0].Transform.GetRotation()) < 0)
			{
				Weight = -Weight;
			}

			for (int32 QuatIdx = 0; QuatIdx < 4; ++QuatIdx)
			{
				AverageQuatVec[QuatIdx] += Weight * ThisQuat[QuatIdx];
			}
		}

		const FQuat AverageQuat(AverageQuatVec[0], AverageQuatVec[1], AverageQuatVec[2], AverageQuatVec[3]);

		// Populate output

		OutAvgTransform.SetTranslation(AverageLocation);
		OutAvgTransform.SetRotation(AverageQuat.GetNormalized());
		OutAvgTransform.SetScale3D(FVector(1));

		return true;
	}
};

double UCameraNodalOffsetAlgoPoints::MinimizeReprojectionError(FTransform& InOutNodalOffset, const TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>>& SamePoseRowGroups) const
{
	const FCameraCalibrationStepsController* StepsController;
	const ULensFile* LensFile;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, &LensFile)))
	{
		return -1.0;
	}

	const ULensDistortionModelHandlerBase* DistortionHandler = StepsController->GetDistortionHandler();
	if (!DistortionHandler)
	{
		return -1.0;
	}

	const FLensDistortionState DistortionState = DistortionHandler->GetCurrentDistortionState();

	const FVector2D FocalLength = DistortionState.FocalLengthInfo.FxFy;
	const FVector2D ImageCenter = FVector2D(0.5, 0.5);

	if (!ensureMsgf((CalibrationRows.Num() > 0), TEXT("Not enough calibration rows")))
	{
		return -1.0;
	}

	const TSharedPtr<FNodalOffsetPointsRowData>& FirstRow = CalibrationRows[0];

	FTransform ExistingOffset = FTransform::Identity;

	if (FirstRow->CameraData.bWasNodalOffsetApplied)
	{
		FNodalPointOffset NodalPointOffset;
		const float Focus = FirstRow->CameraData.InputFocus;
		const float Zoom = FirstRow->CameraData.InputZoom;

		if (LensFile->EvaluateNodalPointOffset(Focus, Zoom, NodalPointOffset))
		{
			ExistingOffset.SetTranslation(NodalPointOffset.LocationOffset);
			ExistingOffset.SetRotation(NodalPointOffset.RotationOffset);
		}
	}

	if (!ensureMsgf(FirstRow->bUndistortedIsValid, TEXT("This method operates on undistorted 2D points. Call UndistortCalibrationRowPoints() prior to calling this method")))
	{
		return -1.0;
	}

	const int32 NumGroups = SamePoseRowGroups.Num();

	TArray<TArray<FVector>> ObjectPoints;
	ObjectPoints.Reserve(NumGroups);

	TArray<TArray<FVector2f>> ImagePoints;
	ImagePoints.Reserve(NumGroups);

	TArray<FTransform> CameraPoses;
	CameraPoses.Reserve(NumGroups);

	for (int32 PoseGroupIndex = 0; PoseGroupIndex < NumGroups; ++PoseGroupIndex)
	{
		const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& PoseGroup = *(SamePoseRowGroups[PoseGroupIndex]);

		if (!ensureMsgf((PoseGroup.Num() > 0), TEXT("Not enough calibration rows")))
		{
			return -1.0;
		}

		const TSharedPtr<FNodalOffsetPointsRowData>& FirstRowInPoseGroup = PoseGroup[0];
		const FTransform CameraPose = FirstRowInPoseGroup->CameraData.Pose;

		// Undo the existing nodal offset transformation
		CameraPoses.Add(ExistingOffset.Inverse() * CameraPose);

		TArray<FVector> Points3d;
		TArray<FVector2f> Points2d;
		Points3d.Reserve(PoseGroup.Num());
		Points2d.Reserve(PoseGroup.Num());

		for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : PoseGroup)
		{
			Points3d.Add(Row->CalibratorPointData.Location);
			Points2d.Add(Row->UndistortedPoint2D);
		}

		ObjectPoints.Add(Points3d);
		ImagePoints.Add(Points2d);
	}

	return FCameraCalibrationSolver::OptimizeNodalOffset(ObjectPoints, ImagePoints, FocalLength, ImageCenter, CameraPoses, InOutNodalOffset);
}

void UCameraNodalOffsetAlgoPoints::Initialize(UNodalOffsetTool* InNodalOffsetTool)
{
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UCameraNodalOffsetAlgoPoints::OnObjectsReplaced);

	NodalOffsetTool = InNodalOffsetTool;

	// Guess which calibrator to use by searching for actors with CalibrationPointComponents.
	SetCalibrator(FindFirstCalibrator());
}

void UCameraNodalOffsetAlgoPoints::Shutdown()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	NodalOffsetTool.Reset();
}

void UCameraNodalOffsetAlgoPoints::Tick(float DeltaTime)
{
	if (!NodalOffsetTool.IsValid())
	{
		return;
	}

	FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return;
	}

	// If not paused, cache calibrator 3d point position
	if (!StepsController->IsPaused())
	{
		// Get calibration point data
		do 
		{	
			LastCalibratorPoints.Empty();

			for (const TSharedPtr<FCalibratorPointData>& CalibratorPoint : CurrentCalibratorPoints)
			{
				if (!CalibratorPoint.IsValid())
				{
					continue;
				}

				FNodalOffsetPointsCalibratorPointData PointCache;

				if (!CalibratorPointCacheFromName(CalibratorPoint->Name, PointCache))
				{
					continue;
				}

				LastCalibratorPoints.Emplace(MoveTemp(PointCache));
			}
		} while (0);

		// Get camera data
		do
		{
			LastCameraData.bIsValid = false;

			const FLensFileEvaluationInputs EvalInputs = StepsController->GetLensFileEvaluationInputs();

			// We require lens evaluation data
			if (!EvalInputs.bIsValid)
			{
				break;
			}

			const ACameraActor* Camera = StepsController->GetCamera();

			if (!Camera)
			{
				break;
			}

			const UCameraComponent* CameraComponent = Camera->GetCameraComponent();

			if (!CameraComponent)
			{
				break;
			}

			LastCameraData.Pose = CameraComponent->GetComponentToWorld();
			LastCameraData.UniqueId = Camera->GetUniqueID();
			LastCameraData.InputFocus = EvalInputs.Focus;
			LastCameraData.InputZoom = EvalInputs.Zoom;

			const ULensComponent* LensComponent = StepsController->FindLensComponent();
			if (LensComponent)
			{
				LastCameraData.bWasNodalOffsetApplied = LensComponent->WasNodalOffsetAppliedThisTick();
				LastCameraData.bWasDistortionEvaluated = LensComponent->WasDistortionEvaluated();
			}

			const AActor* CameraParentActor = Camera->GetAttachParentActor();

			if (CameraParentActor)
			{
				LastCameraData.ParentPose = CameraParentActor->GetTransform();
				LastCameraData.ParentUniqueId = CameraParentActor->GetUniqueID();
			}
			else
			{
				LastCameraData.ParentUniqueId = INDEX_NONE;
			}

			if (Calibrator.IsValid())
			{
				LastCameraData.CalibratorPose = Calibrator->GetTransform();
				LastCameraData.CalibratorUniqueId = Calibrator->GetUniqueID();

				const AActor* CalibratorParentActor = Calibrator->GetAttachParentActor();

				if (CalibratorParentActor)
				{
					LastCameraData.CalibratorParentPose = CalibratorParentActor->GetTransform();
					LastCameraData.CalibratorParentUniqueId = CalibratorParentActor->GetUniqueID();
				}
				else
				{
					LastCameraData.CalibratorParentUniqueId = INDEX_NONE;
				}

				LastCameraData.CalibratorComponentPoses.Empty();

				for (const TWeakObjectPtr<const UCalibrationPointComponent>& CalibrationComponentPtr : ActiveCalibratorComponents)
				{
					if (const UCalibrationPointComponent* const CalibrationComponent = CalibrationComponentPtr.Get())
					{
						if (USceneComponent* AttachComponent = CalibrationComponent->GetAttachParent())
						{
							LastCameraData.CalibratorComponentPoses.Add(AttachComponent->GetUniqueID(), AttachComponent->GetComponentTransform());
						}
					}
				}
			}
			else
			{
				LastCameraData.CalibratorUniqueId = INDEX_NONE;
				LastCameraData.CalibratorParentUniqueId = INDEX_NONE;
			}

			LastCameraData.bIsValid = true;

		} while (0);
	}
}

bool UCameraNodalOffsetAlgoPoints::OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We only respond to left clicks
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return false;
	}

	if (!ensure(NodalOffsetTool.IsValid()))
	{
		return true;
	}

	FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!ensure(StepsController))
	{
		return true;
	}

	if (!LastCameraData.bIsValid)
	{
		FText ErrorMessage = LOCTEXT("InvalidLastCameraData", "Could not find a cached set of camera data (e.g. FIZ). Check the Lens Component to make sure it has valid evaluation inputs.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		return true;
	}

	// Get currently selected calibrator point
	FNodalOffsetPointsCalibratorPointData LastCalibratorPoint;
	LastCalibratorPoint.bIsValid = false;
	{
		TSharedPtr<FCalibratorPointData> CalibratorPoint = CalibratorPointsComboBox->GetSelectedItem();

		if (!CalibratorPoint.IsValid())
		{
			return true;
		}

		// Find its values in the cache
		for (const FNodalOffsetPointsCalibratorPointData& PointCache : LastCalibratorPoints)
		{
			if (PointCache.bIsValid && (PointCache.Name == CalibratorPoint->Name))
			{
				LastCalibratorPoint = PointCache;
				break;
			}
		}
	}

	// Check that we have a valid calibrator 3dpoint or camera data
	if (!LastCalibratorPoint.bIsValid)
	{
		return true;
	}

	// Export the latest session data
	ExportSessionData();

	// Create the row that we're going to add
	TSharedPtr<FNodalOffsetPointsRowData> Row = MakeShared<FNodalOffsetPointsRowData>();

	// Get the next row index for the current calibration session to assign to this new row
	const uint32 RowIndex = NodalOffsetTool->AdvanceSessionRowIndex();

	Row->Index = RowIndex;
	Row->CameraData = LastCameraData;
	Row->CalibratorPointData = LastCalibratorPoint;

	// Get the mouse click 2d position
	if (!StepsController->CalculateNormalizedMouseClickPosition(MyGeometry, MouseEvent, Row->Point2D, ESimulcamViewportPortion::CameraFeed))
	{
		return true;
	}

	// Validate the new row, show a message if validation fails.
	{
		FText ErrorMessage;

		if (!ValidateNewRow(Row, ErrorMessage))
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, LOCTEXT("NewRowError", "New Row Error"));
			return true;
		}
	}

	// Add this data point
	CalibrationRows.Add(Row);

	// Export the data for this row to a .json file on disk
	ExportRow(Row);

	// Notify the ListView of the new data
	if (CalibrationListView.IsValid())
	{
		CalibrationListView->RequestListRefresh();
		CalibrationListView->RequestNavigateToItem(Row);
	}

	// Auto-advance to the next calibration point (if it exists)
	if (AdvanceCalibratorPoint())
	{
		// Play media if this was the last point in the object
		StepsController->Play();
	}

	return true;
}

bool UCameraNodalOffsetAlgoPoints::AdvanceCalibratorPoint()
{
	TSharedPtr<FCalibratorPointData> CurrentItem = CalibratorPointsComboBox->GetSelectedItem();

	if (!CurrentItem.IsValid())
	{
		return false;
	}

	for (int32 PointIdx = 0; PointIdx < CurrentCalibratorPoints.Num(); PointIdx++)
	{
		if (CurrentCalibratorPoints[PointIdx]->Name == CurrentItem->Name)
		{
			const int32 NextIdx = (PointIdx + 1) % CurrentCalibratorPoints.Num();
			CalibratorPointsComboBox->SetSelectedItem(CurrentCalibratorPoints[NextIdx]);

			// return true if we wrapped around (NextIdx is zero)
			return !NextIdx;
		}
	}

	return false;
}

bool UCameraNodalOffsetAlgoPoints::GetCurrentCalibratorPointLocation(FVector& OutLocation)
{
	TSharedPtr<FCalibratorPointData> CalibratorPointData = CalibratorPointsComboBox->GetSelectedItem();

	if (!CalibratorPointData.IsValid())
	{
		return false;
	}

	FNodalOffsetPointsCalibratorPointData PointCache;

	if (!CalibratorPointCacheFromName(CalibratorPointData->Name, PointCache))
	{
		return false;
	}

	OutLocation = PointCache.Location;

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildUI()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Calibrator picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[ FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("Calibrator", "Calibrator"), BuildCalibrationDevicePickerWidget()) ]

		+ SVerticalBox::Slot() // Calibrator component picker
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorComponents", "Calibrator Component(s)"), BuildCalibrationComponentPickerWidget())]

		+SVerticalBox::Slot() // Calibrator point names
		.AutoHeight()
		.MaxHeight(FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[ FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(LOCTEXT("CalibratorPoint", "Calibrator Point"), BuildCalibrationPointsComboBox()) ]
		
		+ SVerticalBox::Slot() // Calibration Rows
		.AutoHeight()
		.MaxHeight(12 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		[ BuildCalibrationPointsTable() ]
		
		+ SVerticalBox::Slot() // Action buttons (e.g. Remove, Clear)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.AutoHeight()
		.Padding(0,20)
		[ BuildCalibrationActionButtons() ]
		;
}

bool UCameraNodalOffsetAlgoPoints::ValidateNewRow(TSharedPtr<FNodalOffsetPointsRowData>& Row, FText& OutErrorMessage) const
{
	const FCameraCalibrationStepsController* StepsController;
	const ULensFile* LensFile;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, &LensFile)))
	{
		OutErrorMessage = LOCTEXT("ToolNotFound", "Tool not found");
		return false;
	}

	if (!Row.IsValid())
	{
		OutErrorMessage = LOCTEXT("InvalidRowPointer", "Invalid row pointer");
		return false;
	}

	// Distortion was evaluated

	if (!Row->CameraData.bWasDistortionEvaluated)
	{
		OutErrorMessage = LOCTEXT("DistortionNotEvaluated", "Distortion was not evaluated");
		return false;
	}

	// Same camera in view

	const ACameraActor* Camera = StepsController->GetCamera();

	if (!Camera || !Camera->GetCameraComponent())
	{
		OutErrorMessage = LOCTEXT("MissingCamera", "Missing camera");
		return false;
	}

	if (Camera->GetUniqueID() != Row->CameraData.UniqueId)
	{
		OutErrorMessage = LOCTEXT("DifferentCameraAsSelected", "Different camera as selected");
		return false;
	}

	if (!CalibrationRows.Num())
	{
		return true;
	}

	// Same camera as before
	const TSharedPtr<FNodalOffsetPointsRowData>& FirstRow = CalibrationRows[0];

	if (FirstRow->CameraData.UniqueId != Row->CameraData.UniqueId)
	{
		OutErrorMessage = LOCTEXT("CameraChangedDuringTheTest", "Camera changed during the test");
		return false;
	}

	// Same parent as before

	if (FirstRow->CameraData.ParentUniqueId != Row->CameraData.ParentUniqueId)
	{
		OutErrorMessage = LOCTEXT("CameraParentChangedDuringTheTest", "Camera parent changed during the test");
		return false;
	}

	// bApplyNodalOffset did not change.
	//
	// It can't change because we need to know if the camera pose is being affected or not by the current nodal offset evaluation.
	// And we need to know that because the offset we calculate will need to either subtract or not the current evaluation when adding it to the LUT.
	if (FirstRow->CameraData.bWasNodalOffsetApplied != Row->CameraData.bWasNodalOffsetApplied)
	{
		OutErrorMessage = LOCTEXT("ApplyNodalOffsetChanged", "Apply nodal offset changed");
		return false;
	}

	//@todo Focus and zoom did not change much (i.e. inputs to distortion and nodal offset). 
	//      Threshold for physical units should differ from normalized encoders.

	return true;
}

bool UCameraNodalOffsetAlgoPoints::BasicCalibrationChecksPass(const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, FText& OutErrorMessage) const
{
	const FCameraCalibrationStepsController* StepsController;
	const ULensFile* LensFile;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, &LensFile)))
	{
		OutErrorMessage = LOCTEXT("ToolNotFound", "Tool not found");
		return false;
	}

	// Sanity checks
	//

	// Enough points
	if (Rows.Num() < 4)
	{
		OutErrorMessage = LOCTEXT("LessThanFourSamples", "At least 4 correspondence points are required");
		return false;
	}

	// All points are valid
	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : Rows)
	{
		if (!ensure(Row.IsValid()))
		{
			OutErrorMessage = LOCTEXT("InvalidRow", "Invalid Row");
			return false;
		}
	}

	// Get camera.

	const ACameraActor* Camera = StepsController->GetCamera();

	if (!Camera || !Camera->GetCameraComponent())
	{
		OutErrorMessage = LOCTEXT("MissingCamera", "Missing camera");
		return false;
	}

	UCameraComponent* CameraComponent = Camera->GetCameraComponent();
	check(CameraComponent);

	UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CameraComponent);

	if (!CineCameraComponent)
	{
		OutErrorMessage = LOCTEXT("OnlyCineCamerasSupported", "Only cine cameras are supported");
		return false;
	}

	const TSharedPtr<FNodalOffsetPointsRowData>& FirstRow = Rows[0];

	// Still same camera (since we need it to get the distortion handler, which much be the same)

	if (Camera->GetUniqueID() != FirstRow->CameraData.UniqueId)
	{
		OutErrorMessage = LOCTEXT("DifferentCameraAsSelected", "Different camera as selected");
		return false;
	}

	// Camera did not move much.
	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : Rows)
	{
		if (!FCameraCalibrationUtils::IsNearlyEqual(FirstRow->CameraData.Pose, Row->CameraData.Pose))
		{
			OutErrorMessage = LOCTEXT("CameraMoved", "Camera moved too much between samples.");
			return false;
		}
	}

	return true;
}

bool UCameraNodalOffsetAlgoPoints::CalculatedOptimalCameraComponentPose(
	FTransform& OutDesiredCameraTransform, 
	const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, 
	FText& OutErrorMessage) const
{
	if (!BasicCalibrationChecksPass(Rows, OutErrorMessage))
	{
		return false;
	}

	if (!NodalOffsetTool.IsValid())
	{
		return false;
	}

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return false;
	}

	const ULensDistortionModelHandlerBase* DistortionHandler = StepsController->GetDistortionHandler();
	if (!DistortionHandler)
	{
		OutErrorMessage = LOCTEXT("DistortionHandlerNotFound", "No distortion source found");
		return false;
	}

	TArray<FVector> ObjectPoints;
	ObjectPoints.Reserve(Rows.Num());

	TArray<FVector2f> ImagePoints;
	ImagePoints.Reserve(Rows.Num());

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : Rows)
	{
		ObjectPoints.Add(Row->CalibratorPointData.Location);
		ImagePoints.Add(Row->UndistortedPoint2D);
	}

	const FLensDistortionState DistortionState = DistortionHandler->GetCurrentDistortionState();
	const FVector2D FocalLength = DistortionState.FocalLengthInfo.FxFy;
	const FVector2D ImageCenter = FVector2D(0.5, 0.5);

	TArray<float> DistortionParameters = { 0.0f, 0.0f, 0.0f, 0.0f, 0.0f };

	// We send no distortion parameters, because Points2d was manually undistorted already
	bool bResult = FOpenCVHelper::SolvePnP(ObjectPoints, ImagePoints, FocalLength, ImageCenter, DistortionParameters, OutDesiredCameraTransform);
	if (!bResult)
	{
		OutErrorMessage = LOCTEXT("SolvePnpFailed", "Failed to resolve a camera position given the data in the calibration rows. Please retry the calibration.");
		return false;
	}

	return true;
}

bool UCameraNodalOffsetAlgoPoints::CalibratorMovedInAnyRow(
	const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows) const
{
	if (!Rows.Num())
	{
		return false;
	}

	TSharedPtr<FNodalOffsetPointsRowData> FirstRow;

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : Rows)
	{
		if (!FirstRow.IsValid())
		{
			if (ensure(Row.IsValid()))
			{
				FirstRow = Row;
			}

			continue;
		}

		if (!ensure(Row.IsValid()))
		{
			continue;
		}

		if (!FCameraCalibrationUtils::IsNearlyEqual(FirstRow->CameraData.CalibratorPose, Row->CameraData.CalibratorPose))
		{
			return true;
		}
	}

	return false;
}


bool UCameraNodalOffsetAlgoPoints::CalibratorMovedAcrossGroups(
	const TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>>& SamePoseRowGroups) const
{
	TArray<TSharedPtr<FNodalOffsetPointsRowData>> Rows;

	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		if (!ensure(SamePoseRowGroup.IsValid()))
		{
			continue;
		}

		Rows.Append(*SamePoseRowGroup);
	}

	return CalibratorMovedInAnyRow(Rows);
}

bool UCameraNodalOffsetAlgoPoints::GetNodalOffsetSinglePose(
	FNodalPointOffset& OutNodalOffset, 
	float& OutFocus, 
	float& OutZoom, 
	float& OutError, 
	const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, 
	FText& OutErrorMessage) const
{
	const FCameraCalibrationStepsController* StepsController;
	const ULensFile* LensFile;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, &LensFile)))
	{
		OutErrorMessage = LOCTEXT("LensNotFound", "Lens not found");
		return false;
	}

	FTransform DesiredCameraTransform;

	if (!CalculatedOptimalCameraComponentPose(DesiredCameraTransform, Rows, OutErrorMessage))
	{
		return false;
	}

	// This is how we update the offset even when the camera is evaluating the current
	// nodal offset curve in the Lens File:
	// 
	// CameraTransform = ExistingOffset * CameraTransformWithoutOffset
	// => CameraTransformWithoutOffset = ExistingOffset' * CameraTransform
	//
	// DesiredTransform = Offset * CameraTransformWithoutOffset
	// => Offset = DesiredTransform * CameraTransformWithoutOffset'
	// => Offset = DesiredTransform * (ExistingOffset' * CameraTransform)'
	// => Offset = DesiredTransform * (CameraTransform' * ExistingOffset)

	// Evaluate nodal offset

	// Determine the input values to the LUT (focus and zoom)

	check(Rows.Num()); // There must have been rows for CalculatedOptimalCameraComponentPose to have succeeded.
	check(Rows[0].IsValid()); // All rows should be valid.

	const TSharedPtr<FNodalOffsetPointsRowData>& FirstRow = Rows[0];

	OutFocus = FirstRow->CameraData.InputFocus;
	OutZoom  = FirstRow->CameraData.InputZoom;

	// See if the camera already had an offset applied, in which case we need to account for it.

	FTransform ExistingOffset = FTransform::Identity;

	if (FirstRow->CameraData.bWasNodalOffsetApplied)
	{
		FNodalPointOffset NodalPointOffset;

		if (LensFile->EvaluateNodalPointOffset(OutFocus, OutZoom, NodalPointOffset))
		{
			ExistingOffset.SetTranslation(NodalPointOffset.LocationOffset);
			ExistingOffset.SetRotation(NodalPointOffset.RotationOffset);
		}
	}

	FTransform DesiredOffset = DesiredCameraTransform * FirstRow->CameraData.Pose.Inverse() * ExistingOffset;

	OutNodalOffset.LocationOffset = DesiredOffset.GetLocation();
	OutNodalOffset.RotationOffset = DesiredOffset.GetRotation();

	return true;
}

bool UCameraNodalOffsetAlgoPoints::GetNodalOffset(FNodalPointOffset& OutNodalOffset, float& OutFocus, float& OutZoom, float& OutError, FText& OutErrorMessage)
{
	using namespace CameraNodalOffsetAlgoPoints;

	const FCameraCalibrationStepsController* StepsController;
	const ULensFile* LensFile;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, &LensFile)))
	{
		OutErrorMessage = LOCTEXT("LensNotFound", "Lens not found");
		return false;
	}

	// Group Rows by camera poses.
	TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>> SamePoseRowGroups;
	GroupRowsByCameraPose(SamePoseRowGroups, CalibrationRows);

	if (!SamePoseRowGroups.Num())
	{
		OutErrorMessage = LOCTEXT("NotEnoughRows", "Not enough calibration rows. Please add more samples and try again.");
		return false;
	}

	// Do some basic checks on each group
	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		if (!BasicCalibrationChecksPass(*SamePoseRowGroup, OutErrorMessage))
		{
			return false;
		}
	}

	UE::NodalOffsetAlgoPointsAnalytics::RecordEvent(FriendlyName().ToString(), CalibrationRows.Num());

	// Undistort the 2D points in each calibration row 
	UndistortCalibrationRowPoints();

	TArray<FSinglePoseResult> SinglePoseResults;
	SinglePoseResults.Reserve(SamePoseRowGroups.Num());

	// Solve each group independently
	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		FNodalPointOffset NodalOffset;

		if (!GetNodalOffsetSinglePose(
			NodalOffset,
			OutFocus,
			OutZoom,
			OutError,
			*SamePoseRowGroup,
			OutErrorMessage))
		{
			return false;
		}

		// Add results to the array of single pose results

		FSinglePoseResult SinglePoseResult;

		SinglePoseResult.Transform.SetLocation(NodalOffset.LocationOffset);
		SinglePoseResult.Transform.SetRotation(NodalOffset.RotationOffset);
		SinglePoseResult.NumSamples = SamePoseRowGroup->Num();

		SinglePoseResults.Add(SinglePoseResult);
	}

	check(SinglePoseResults.Num()); // If any single pose result failed then we should not have reached here.

	FTransform AverageTransform;

	if (!AverageSinglePoseResults(SinglePoseResults, AverageTransform))
	{
		OutErrorMessage = LOCTEXT("CouldNotAverageSinglePoseResults",
			"There was an error when averaging the single pose results");

		return false;
	}

	// Refine the weighted average transform using a downhill solver
	OutError = MinimizeReprojectionError(AverageTransform, SamePoseRowGroups);

	// Assign output nodal offset.

	OutNodalOffset.LocationOffset = AverageTransform.GetLocation();
	OutNodalOffset.RotationOffset = AverageTransform.GetRotation();

	// OutFocus, OutZoom were already assigned.
	// Note that OutError will have the error of the last camera pose instead of a global error.

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildCalibrationDevicePickerWidget()
{
	return SNew(SFilterableActorPicker)
		.OnSetObject_Lambda([&](const FAssetData& AssetData) -> void
		{
			if (AssetData.IsValid())
			{
				SetCalibrator(Cast<AActor>(AssetData.GetAsset()));
			}
		})
		.OnShouldFilterAsset_Lambda([&](const FAssetData& AssetData) -> bool
		{
			const AActor* Actor = Cast<AActor>(AssetData.GetAsset());

			if (!Actor)
			{
				return false;
			}

			TArray<UCalibrationPointComponent*, TInlineAllocator<NumInlineAllocations>> CalibrationPoints;
			Actor->GetComponents(CalibrationPoints);

			return (CalibrationPoints.Num() > 0);
		})
		.ActorAssetData_Lambda([&]() -> FAssetData
		{
			return FAssetData(GetCalibrator(), true);
		});
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildCalibrationComponentMenu()
{
	// Generate menu
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("CalibrationComponents", LOCTEXT("CalibrationComponents", "Calibration Point Components"));

	if (AActor* CalibratorPtr = Calibrator.Get())
	{
		TArray<UCalibrationPointComponent*, TInlineAllocator<NumInlineAllocations>> CalibrationPointComponents;
		CalibratorPtr->GetComponents(CalibrationPointComponents);

		for (UCalibrationPointComponent* CalibratorComponent : CalibrationPointComponents)
		{
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("ComponentLabel", "{0}"), FText::FromString(CalibratorComponent->GetName())),
				FText::Format(LOCTEXT("ComponentTooltip", "{0}"), FText::FromString(CalibratorComponent->GetName())),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, CalibratorComponent] { OnCalibrationComponentSelected(CalibratorComponent);}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, CalibratorComponent] { return IsCalibrationComponentSelected(CalibratorComponent); })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	else
	{
		MenuBuilder.AddMenuEntry(
			FText::FromName(NAME_None),
			FText::FromName(NAME_None),
			FSlateIcon(),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked()
			),
			NAME_None,
			EUserInterfaceActionType::None
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildCalibrationComponentPickerWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SComboButton)
			.OnGetMenuContent_Lambda([this]() { return BuildCalibrationComponentMenu(); })
			.ContentPadding(FMargin(4.0, 2.0))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() {
					if (ActiveCalibratorComponents.Num() > 1)
					{
						return LOCTEXT("MultipleCalibrationComponents", "Multiple Values");
					}
					else if (ActiveCalibratorComponents.Num() == 1)
					{
						if (const UCalibrationPointComponent* CalibratorComponent = ActiveCalibratorComponents[0].Get())
						{
							return FText::FromString(CalibratorComponent->GetName());
						}
					}

					return LOCTEXT("NoCalibrationComponents", "None");
				})
			]
		]; 
}

void UCameraNodalOffsetAlgoPoints::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	for (TWeakObjectPtr<const UCalibrationPointComponent>& OldComponentPtr : ActiveCalibratorComponents)
	{
		constexpr bool bEvenIfPendingKill = true;
		if (UObject* NewObject = OldToNewInstanceMap.FindRef(OldComponentPtr.Get(bEvenIfPendingKill)))
		{
			// Reassign the object
			OldComponentPtr = Cast<UCalibrationPointComponent>(NewObject);
		}
	}
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildCalibrationActionButtons()
{
	const float ButtonPadding = 3.0f;

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Row manipulation
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot() // Button to clear all rows
			.AutoWidth()
			[ 
				SNew(SButton)
				.Text(LOCTEXT("ClearAll", "Clear All"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([&]() -> FReply
				{
					ClearCalibrationRows();
					return FReply::Handled();
				})
			]
		]
		+ SVerticalBox::Slot() // Spacer
		[
			SNew(SBox)
			.MinDesiredHeight(0.5 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
			.MaxDesiredHeight(0.5 * FCameraCalibrationWidgetHelpers::DefaultRowHeight)
		]
		+ SVerticalBox::Slot() // Apply To Calibrator
		.AutoHeight()
		.Padding(0, ButtonPadding)
		[
			SNew(SButton)
			.Text(LOCTEXT("ApplyToCalibrator", "Apply To Calibrator"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked_Lambda([&]() -> FReply
			{
				FScopedTransaction Transaction(LOCTEXT("ApplyNodalOffsetToCalibrator", "Applying Nodal Offset to Calibrator"));
				ApplyNodalOffsetToCalibrator();
				return FReply::Handled();
			})
		]
		+ SVerticalBox::Slot() // Apply To Camera Parent
		.AutoHeight()
		.Padding(0, ButtonPadding)
		[
			SNew(SButton)
			.Text(LOCTEXT("ApplyToTrackingOrigin", "Apply To Camera Parent"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked_Lambda([&]() -> FReply
			{
				FScopedTransaction Transaction(LOCTEXT("ApplyNodalOffsetToTrackingOrigin", "Applying Nodal Offset to Tracking Origin"));
				ApplyNodalOffsetToTrackingOrigin();
				return FReply::Handled();
			})
		]
		+ SVerticalBox::Slot() // Apply To Calibrator Parent
		.AutoHeight()
		.Padding(0, ButtonPadding)
		[
			SNew(SButton)
			.Text(LOCTEXT("ApplyToCalibratorParent", "Apply To Calibrator Parent"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked_Lambda([&]() -> FReply
			{
				FScopedTransaction Transaction(LOCTEXT("ApplyNodalOffsetToCalibratorParent", "Applying Nodal Offset to Calibrator Parent"));
				ApplyNodalOffsetToCalibratorParent();
				return FReply::Handled();
			})
		]
		+ SVerticalBox::Slot() // Apply To Calibrator Component
			[
				SNew(SButton)
				.Text(LOCTEXT("ApplyToCalibratorComponentParent", "Apply To Calibrator Component"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([&]() -> FReply
				{
					FScopedTransaction Transaction(LOCTEXT("ApplyNodalOffsetToCalibratorComponents", "Applying Nodal Offset to Calibrator Components"));
					ApplyNodalOffsetToCalibratorComponents();
					return FReply::Handled();
				})
			]
		;
}

void UCameraNodalOffsetAlgoPoints::OnCalibrationComponentSelected(const UCalibrationPointComponent* const SelectedComponent)
{
	if (IsCalibrationComponentSelected(SelectedComponent))
	{
		ActiveCalibratorComponents.Remove(SelectedComponent);
	}
	else
	{
		ActiveCalibratorComponents.Add(SelectedComponent);
	}

	CurrentCalibratorPoints.Empty();
	for (const TWeakObjectPtr<const UCalibrationPointComponent>& CalibrationPointPtr : ActiveCalibratorComponents)
	{
		if (const UCalibrationPointComponent* const CalibrationPoint = CalibrationPointPtr.Get())
		{
			TArray<FString> PointNames;

			CalibrationPoint->GetNamespacedPointNames(PointNames);

			for (FString& PointName : PointNames)
			{
				CurrentCalibratorPoints.Add(MakeShared<FCalibratorPointData>(PointName));
			}
		}
	}

	if (!CalibratorPointsComboBox)
	{
		return;
	}

	CalibratorPointsComboBox->RefreshOptions();

	if (CurrentCalibratorPoints.Num())
	{
		CalibratorPointsComboBox->SetSelectedItem(CurrentCalibratorPoints[0]);
	}
	else
	{
		CalibratorPointsComboBox->SetSelectedItem(nullptr);
	}
}

bool UCameraNodalOffsetAlgoPoints::IsCalibrationComponentSelected(const UCalibrationPointComponent* const SelectedComponent) const
{
	return ActiveCalibratorComponents.Contains(SelectedComponent);
}

bool UCameraNodalOffsetAlgoPoints::ApplyNodalOffsetToCalibrator()
{
	using namespace CameraNodalOffsetAlgoPoints;

	// Get the desired camera component world pose

	FText ErrorMessage;
	const FText TitleError = LOCTEXT("CalibrationError", "CalibrationError");

	// Get the calibrator

	if (!Calibrator.IsValid())
	{
		ErrorMessage = LOCTEXT("MissingCalibrator", "Missing Calibrator");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	if (!CalibrationRows.Num())
	{
		ErrorMessage = LOCTEXT("NotEnoughSampleRows", "Not enough sample rows. Please add more and try again.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// All calibration points should correspond to the same calibrator

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : CalibrationRows)
	{
		check(Row.IsValid());

		if (Row->CameraData.CalibratorUniqueId != Calibrator->GetUniqueID())
		{
			ErrorMessage = LOCTEXT("WrongCalibrator", "All rows must belong to the same calibrator");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}
	}

	const TSharedPtr<FNodalOffsetPointsRowData>& LastRow = CalibrationRows[CalibrationRows.Num() - 1];
	check(LastRow.IsValid());

	// Verify that calibrator did not move much for all the samples
	if (CalibratorMovedInAnyRow(CalibrationRows))
	{
		ErrorMessage = LOCTEXT("CalibratorMoved", "The calibrator moved during the calibration");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// Group Rows by camera poses.
	TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>> SamePoseRowGroups;
	GroupRowsByCameraPose(SamePoseRowGroups, CalibrationRows);

	if (!SamePoseRowGroups.Num())
	{
		ErrorMessage = LOCTEXT("NotEnoughRows", "Not enough calibration rows. Please add more samples and try again.");
		return false;
	}

	TArray<FSinglePoseResult> SinglePoseResults;
	SinglePoseResults.Reserve(SamePoseRowGroups.Num());

	// Solve each group independently
	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		FSinglePoseResult SinglePoseResult;

		const bool bSucceeded = CalcCalibratorPoseForSingleCamPose(*SamePoseRowGroup, SinglePoseResult.Transform, ErrorMessage);

		if (!bSucceeded)
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}

		SinglePoseResult.NumSamples = SamePoseRowGroup->Num();
		SinglePoseResults.Add(SinglePoseResult);
	}

	if (!SinglePoseResults.Num())
	{
		ErrorMessage = LOCTEXT("NoSinglePoseResults",
			"There were no valid single pose results. See Output Log for additional details.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	FTransform DesiredCalibratorPose;

	if (!AverageSinglePoseResults(SinglePoseResults, DesiredCalibratorPose))
	{
		ErrorMessage = LOCTEXT("CouldNotAverageSinglePoseResults",
			"There was an error when averaging the single pose results");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// apply the new calibrator transform
	Calibrator->Modify();
	Calibrator->SetActorTransform(DesiredCalibratorPose);

	// Since the offset was applied, there is no further use for the current samples.
	ClearCalibrationRows();

	return true;
}

bool UCameraNodalOffsetAlgoPoints::CalcTrackingOriginPoseForSingleCamPose(
	const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, 
	FTransform& OutTransform, 
	FText& OutErrorMessage)
{
	// Here we are assuming that the camera parent is the tracking origin.

	// Get the desired camera component world pose

	FTransform DesiredCameraPose;

	// Undistort the 2D points in each calibration row 
	UndistortCalibrationRowPoints();

	if (!CalculatedOptimalCameraComponentPose(DesiredCameraPose, Rows, OutErrorMessage))
	{
		return false;
	}

	check(Rows.Num()); // Must be non-zero if CalculatedOptimalCameraComponentPose succeeded.

	const TSharedPtr<FNodalOffsetPointsRowData>& LastRow = Rows[Rows.Num() - 1];
	check(LastRow.IsValid());

	// calculate the new parent transform

	// CameraPose = RelativeCameraPose * ParentPose
	// => RelativeCameraPose = CameraPose * ParentPose'
	// 
	// DesiredCameraPose = RelativeCameraPose * DesiredParentPose
	// => DesiredParentPose = RelativeCameraPose' * DesiredCameraPose
	// => DesiredParentPose = (CameraPose * ParentPose')' * DesiredCameraPose
	// => DesiredParentPose = ParentPose * CameraPose' * DesiredCameraPose

	OutTransform = LastRow->CameraData.ParentPose * LastRow->CameraData.Pose.Inverse() * DesiredCameraPose;

	return true;
}


bool UCameraNodalOffsetAlgoPoints::ApplyNodalOffsetToTrackingOrigin()
{
	using namespace CameraNodalOffsetAlgoPoints;

	// Here we are assuming that the camera parent is the tracking origin.

	// get camera

	const FCameraCalibrationStepsController* StepsController;
	const ULensFile* LensFile;

	const FText TitleError = LOCTEXT("CalibrationError", "CalibrationError");
	FText ErrorMessage;

	if (!ensure(GetStepsControllerAndLensFile(&StepsController, &LensFile)))
	{
		ErrorMessage = LOCTEXT("ToolNotFound", "Tool not found");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	const ACameraActor* Camera = StepsController->GetCamera();

	if (!Camera)
	{
		ErrorMessage = LOCTEXT("CameraNotFound", "Camera Not Found");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// Get the parent transform

	AActor* ParentActor = Camera->GetAttachParentActor();

	if (!ParentActor)
	{
		ErrorMessage = LOCTEXT("CameraParentNotFound", "Camera Parent not found");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	if (!CalibrationRows.Num())
	{
		ErrorMessage = LOCTEXT("NotEnoughSamples", "Not Enough Samples");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	const TSharedPtr<FNodalOffsetPointsRowData>& LastRow = CalibrationRows[CalibrationRows.Num() - 1];
	check(LastRow.IsValid());

	if (LastRow->CameraData.ParentUniqueId != ParentActor->GetUniqueID())
	{
		ErrorMessage = LOCTEXT("ParentChanged", "Parent changed");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// Group Rows by camera poses.
	TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>> SamePoseRowGroups;
	GroupRowsByCameraPose(SamePoseRowGroups, CalibrationRows);

	TArray<FSinglePoseResult> SinglePoseResults;
	SinglePoseResults.Reserve(SamePoseRowGroups.Num());

	// Solve each group independently
	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		FSinglePoseResult SinglePoseResult;

		const bool bSucceeded = CalcTrackingOriginPoseForSingleCamPose(*SamePoseRowGroup, SinglePoseResult.Transform, ErrorMessage);

		if (!bSucceeded)
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}

		SinglePoseResult.NumSamples = SamePoseRowGroup->Num();
		SinglePoseResults.Add(SinglePoseResult);
	}

	if (!SinglePoseResults.Num())
	{
		ErrorMessage = LOCTEXT("NoSinglePoseResults",
			"There were no valid single pose results. See Output Log for additional details.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	FTransform DesiredParentPose;

	if (!AverageSinglePoseResults(SinglePoseResults, DesiredParentPose))
	{
		ErrorMessage = LOCTEXT("CouldNotAverageSinglePoseResults",
			"There was an error when averaging the single pose results");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// apply the new parent transform
	ParentActor->Modify();
	ParentActor->SetActorTransform(DesiredParentPose);

	// Since the offset was applied, there is no further use for the current samples.
	ClearCalibrationRows();

	return true;
}

bool UCameraNodalOffsetAlgoPoints::CalcCalibratorPoseForSingleCamPose(
	const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows,
	FTransform& OutTransform,
	FText& OutErrorMessage)
{
	FTransform DesiredCameraPose;

	// Undistort the 2D points in each calibration row 
	UndistortCalibrationRowPoints();

	if (!CalculatedOptimalCameraComponentPose(DesiredCameraPose, Rows, OutErrorMessage))
	{
		return false;
	}

	check(Rows.Num());

	const TSharedPtr<FNodalOffsetPointsRowData>& LastRow = Rows[Rows.Num() - 1];
	check(LastRow.IsValid());

	// Calculate the offset
	// 
	// Calibrator = DesiredCalibratorToCamera * DesiredCamera
	// => DesiredCalibratorToCamera = Calibrator * DesiredCamera'
	// 
	// DesiredCalibrator = DesiredCalibratorToCamera * Camera
	// => DesiredCalibrator = Calibrator * DesiredCamera' * Camera

	OutTransform = LastRow->CameraData.CalibratorPose * DesiredCameraPose.Inverse() * LastRow->CameraData.Pose;

	return true;
}

bool UCameraNodalOffsetAlgoPoints::ApplyNodalOffsetToCalibratorParent()
{
	using namespace CameraNodalOffsetAlgoPoints;

	// Get the desired camera component world pose

	FText ErrorMessage;
	const FText TitleError = LOCTEXT("CalibrationError", "CalibrationError");

	// Get the calibrator

	if (!Calibrator.IsValid())
	{
		ErrorMessage = LOCTEXT("MissingCalibrator", "Missing Calibrator");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// Get the parent

	AActor* ParentActor = Calibrator->GetAttachParentActor();

	if (!ParentActor)
	{
		ErrorMessage = LOCTEXT("CalibratorParentNotFound", "Calibrator Parent not found");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// All calibration points should correspond to the same calibrator and calibrator parent

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : CalibrationRows)
	{
		check(Row.IsValid());

		if (Row->CameraData.CalibratorUniqueId != Calibrator->GetUniqueID())
		{
			ErrorMessage = LOCTEXT("WrongCalibrator", "All rows must belong to the same calibrator");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}

		if (Row->CameraData.CalibratorParentUniqueId != ParentActor->GetUniqueID())
		{
			ErrorMessage = LOCTEXT("WrongCalibratorParent", "All rows must belong to the same calibrator parent");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}
	}

	// Verify that calibrator did not move much for all the samples
	if (CalibratorMovedInAnyRow(CalibrationRows))
	{
		ErrorMessage = LOCTEXT("CalibratorMoved", "The calibrator moved during the calibration");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// Group Rows by camera poses.
	TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>> SamePoseRowGroups;
	GroupRowsByCameraPose(SamePoseRowGroups, CalibrationRows);

	if (!SamePoseRowGroups.Num())
	{
		ErrorMessage = LOCTEXT("NotEnoughRows", "Not enough calibration rows. Please add more samples and try again.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	TArray<FSinglePoseResult> SinglePoseResults;
	SinglePoseResults.Reserve(SamePoseRowGroups.Num());

	// Solve each group independently
	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		FSinglePoseResult SinglePoseResult;

		const bool bSucceeded = CalcCalibratorPoseForSingleCamPose(*SamePoseRowGroup, SinglePoseResult.Transform, ErrorMessage);

		if (!bSucceeded)
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}

		SinglePoseResult.NumSamples = SamePoseRowGroup->Num();
		SinglePoseResults.Add(SinglePoseResult);
	}

	if (!SinglePoseResults.Num())
	{
		ErrorMessage = LOCTEXT("NoSinglePoseResults",
			"There were no valid single pose results. See Output Log for additional details.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	FTransform DesiredCalibratorPose;

	if (!AverageSinglePoseResults(SinglePoseResults, DesiredCalibratorPose))
	{
		ErrorMessage = LOCTEXT("CouldNotAverageSinglePoseResults",
			"There was an error when averaging the single pose results");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	const TSharedPtr<FNodalOffsetPointsRowData>& LastRow = CalibrationRows[CalibrationRows.Num() - 1];
	check(LastRow.IsValid());

	// Apply the new calibrator parent transform
	ParentActor->Modify();
	ParentActor->SetActorTransform(LastRow->CameraData.CalibratorParentPose * LastRow->CameraData.CalibratorPose.Inverse() * DesiredCalibratorPose);

	// Since the offset was applied, there is no further use for the current samples.
	ClearCalibrationRows();

	return true;
}

bool UCameraNodalOffsetAlgoPoints::ApplyNodalOffsetToCalibratorComponents()
{
	using namespace CameraNodalOffsetAlgoPoints;

	// Get the desired camera component world pose

	FText ErrorMessage;
	const FText TitleError = LOCTEXT("CalibrationError", "CalibrationError");

	// Get the calibrator

	if (!Calibrator.IsValid())
	{
		ErrorMessage = LOCTEXT("MissingCalibrator", "Missing Calibrator");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

	
		return false;
	}

	TArray<uint32> ActiveComponentUniqueIDs;
	ActiveComponentUniqueIDs.Reserve(ActiveCalibratorComponents.Num());

	for (const TWeakObjectPtr<const UCalibrationPointComponent>& CalibratorComponentPtr : ActiveCalibratorComponents)
	{
		if (const UCalibrationPointComponent* const CalibratorComponent = CalibratorComponentPtr.Get())
		{
			if (USceneComponent* AttachComponent = CalibratorComponent->GetAttachParent())
			{
				ActiveComponentUniqueIDs.Add(AttachComponent->GetUniqueID());
			}
			else
			{
				ErrorMessage = FText::Format(LOCTEXT("CalibratorSceneComponentNotFound", "{0} is not attached to another scene component.\n\nConsider \"Apply To Calibrator\" instead."), FText::FromString(CalibratorComponent->GetName()));
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

				return false;
			}
		}
	}

	// All calibration points should correspond to the same calibrator and same set of calibrator components

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : CalibrationRows)
	{
		check(Row.IsValid());

		if (Row->CameraData.CalibratorUniqueId != Calibrator->GetUniqueID())
		{
			ErrorMessage = LOCTEXT("WrongCalibrator", "All rows must belong to the same calibrator");
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}

		TArray<uint32> CalibratorComponentUniqueIDs;
		Row->CameraData.CalibratorComponentPoses.GetKeys(CalibratorComponentUniqueIDs);
		CalibratorComponentUniqueIDs.Sort();
		ActiveComponentUniqueIDs.Sort();

 		if (CalibratorComponentUniqueIDs != ActiveComponentUniqueIDs)
 		{
 			ErrorMessage = LOCTEXT("CalibratorComponentsChanged", "The set of active calibrator components changed during the calibration");
 			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);
 
 			return false;
 		}
	}

	// Verify that calibrator did not move much for all the samples
	if (CalibratorMovedInAnyRow(CalibrationRows))
	{
		ErrorMessage = LOCTEXT("CalibratorMoved", "The calibrator moved during the calibration");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	// Group Rows by camera poses.
	TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>> SamePoseRowGroups;
	GroupRowsByCameraPose(SamePoseRowGroups, CalibrationRows);

	if (!SamePoseRowGroups.Num())
	{
		ErrorMessage = LOCTEXT("NotEnoughRows", "Not enough calibration rows. Please add more samples and try again.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	TArray<FSinglePoseResult> SinglePoseResults;
	SinglePoseResults.Reserve(SamePoseRowGroups.Num());

	// Solve each group independently
	for (const auto& SamePoseRowGroup : SamePoseRowGroups)
	{
		FSinglePoseResult SinglePoseResult;

		const bool bSucceeded = CalcCalibratorPoseForSingleCamPose(*SamePoseRowGroup, SinglePoseResult.Transform, ErrorMessage);

		if (!bSucceeded)
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

			return false;
		}

		SinglePoseResult.NumSamples = SamePoseRowGroup->Num();
		SinglePoseResults.Add(SinglePoseResult);
	}

	if (!SinglePoseResults.Num())
	{
		ErrorMessage = LOCTEXT("NoSinglePoseResults",
			"There were no valid single pose results. See Output Log for additional details.");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	FTransform DesiredCalibratorPose;

	if (!AverageSinglePoseResults(SinglePoseResults, DesiredCalibratorPose))
	{
		ErrorMessage = LOCTEXT("CouldNotAverageSinglePoseResults",
			"There was an error when averaging the single pose results");
		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, TitleError);

		return false;
	}

	const TSharedPtr<FNodalOffsetPointsRowData>& LastRow = CalibrationRows[CalibrationRows.Num() - 1];
	check(LastRow.IsValid());

	// Apply the desired transform to each of the calibrator components
	for (const TWeakObjectPtr<const UCalibrationPointComponent>& CalibratorComponentPtr : ActiveCalibratorComponents)
	{
		if (const UCalibrationPointComponent* const CalibratorComponent = CalibratorComponentPtr.Get())
		{
			if (USceneComponent* AttachComponent = CalibratorComponent->GetAttachParent())
			{
				if (LastRow->CameraData.CalibratorComponentPoses.Contains(AttachComponent->GetUniqueID()))
				{
					AttachComponent->Modify();
					AttachComponent->SetWorldTransform(LastRow->CameraData.CalibratorComponentPoses[AttachComponent->GetUniqueID()] * LastRow->CameraData.CalibratorPose.Inverse()* DesiredCalibratorPose);
				}
			}
		}
	}

	// Since the offset was applied, there is no further use for the current samples.
	ClearCalibrationRows();

	return true;
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildCalibrationPointsComboBox()
{
	CalibratorPointsComboBox = SNew(SComboBox<TSharedPtr<FCalibratorPointData>>)
		.OptionsSource(&CurrentCalibratorPoints)
		.OnGenerateWidget_Lambda([&](TSharedPtr<FCalibratorPointData> InOption) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock).Text(FText::FromString(*InOption->Name));
		})
		.InitiallySelectedItem(nullptr)
		[
			SNew(STextBlock)
			.Text_Lambda([&]() -> FText
			{
				if (CalibratorPointsComboBox.IsValid() && CalibratorPointsComboBox->GetSelectedItem().IsValid())
				{
					return FText::FromString(CalibratorPointsComboBox->GetSelectedItem()->Name);
				}

				return LOCTEXT("InvalidComboOption", "None");
			})
		];

	// Update combobox from current calibrator
	SetCalibrator(GetCalibrator());

	return CalibratorPointsComboBox.ToSharedRef();
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildCalibrationPointsTable()
{
	CalibrationListView = SNew(SListView<TSharedPtr<FNodalOffsetPointsRowData>>)
		.ListItemsSource(&CalibrationRows)
		.OnGenerateRow_Lambda([&](TSharedPtr<FNodalOffsetPointsRowData> InItem, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
		{
			return SNew(CameraNodalOffsetAlgoPoints::SCalibrationRowGenerator, OwnerTable).CalibrationRowData(InItem);
		})
		.SelectionMode(ESelectionMode::Multi)
		.OnKeyDownHandler_Lambda([&](const FGeometry& Geometry, const FKeyEvent& KeyEvent) -> FReply
		{
			if (!CalibrationListView.IsValid())
			{
				return FReply::Unhandled();
			}

			if (KeyEvent.GetKey() == EKeys::Delete)
			{
				// Delete selected items

				const TArray<TSharedPtr<FNodalOffsetPointsRowData>> SelectedItems = CalibrationListView->GetSelectedItems();

				for (const TSharedPtr<FNodalOffsetPointsRowData>& SelectedItem : SelectedItems)
				{
					CalibrationRows.Remove(SelectedItem);

					// Delete the previously exported .json file for the row that is being deleted
					if (UNodalOffsetTool* Tool = NodalOffsetTool.Get())
					{
						Tool->DeleteExportedRow(SelectedItem->Index);
					}
				}

				CalibrationListView->RequestListRefresh();
				return FReply::Handled();
			}
			else if (KeyEvent.GetModifierKeys().IsControlDown() && (KeyEvent.GetKey() == EKeys::A))
			{
				// Select all items

				CalibrationListView->SetItemSelection(CalibrationRows, true);
				return FReply::Handled();
			}
			else if (KeyEvent.GetKey() == EKeys::Escape)
			{
				// Deselect all items

				CalibrationListView->ClearSelection();
				return FReply::Handled();
			}

			return FReply::Unhandled();
		})
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column("Name")
			.DefaultLabel(LOCTEXT("Name", "Name"))

			+ SHeaderRow::Column("Point2D")
			.DefaultLabel(LOCTEXT("Point2D", "Point2D"))

			+ SHeaderRow::Column("Point3D")
			.DefaultLabel(LOCTEXT("Point3D", "Point3D"))
		);

	return CalibrationListView.ToSharedRef();
}

AActor* UCameraNodalOffsetAlgoPoints::FindFirstCalibrator() const
{
	// We find the first UCalibrationPointComponent object and return its actor owner.

	if (!NodalOffsetTool.IsValid())
	{
		return nullptr;
	}

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return nullptr;
	}

	const UWorld* World = StepsController->GetWorld();
	const EObjectFlags ExcludeFlags = RF_ClassDefaultObject; // We don't want the calibrator CDOs.

	for (TObjectIterator<UCalibrationPointComponent> It(ExcludeFlags, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		AActor* Actor = It->GetOwner();

		if (Actor && (Actor->GetWorld() == World))
		{
			return Actor;
		}
	}

	return nullptr;
}

bool UCameraNodalOffsetAlgoPoints::CalibratorPointCacheFromName(const FString& Name, FNodalOffsetPointsCalibratorPointData& CalibratorPointCache) const
{
	CalibratorPointCache.bIsValid = false;

	if (!Calibrator.IsValid())
	{
		return false;
	}

	for (const TWeakObjectPtr<const UCalibrationPointComponent>& CalibrationPointPtr : ActiveCalibratorComponents)
	{
		if (const UCalibrationPointComponent* const CalibrationPoint = CalibrationPointPtr.Get())
		{
			if (CalibrationPoint->GetWorldLocation(Name, CalibratorPointCache.Location))
			{
				CalibratorPointCache.bIsValid = true;
				CalibratorPointCache.Name = Name;
				return true;
			}
		}
	}

	return false;
}

void UCameraNodalOffsetAlgoPoints::SetCalibrator(AActor* InCalibrator)
{
	Calibrator = InCalibrator;

	// Update the list of points

	CurrentCalibratorPoints.Empty();

	if (!Calibrator.IsValid())
	{
		return;
	}

	TArray<UCalibrationPointComponent*, TInlineAllocator<NumInlineAllocations>> CalibrationPoints;
	Calibrator->GetComponents(CalibrationPoints);

	ActiveCalibratorComponents.Empty();

	for (const UCalibrationPointComponent* CalibrationPoint : CalibrationPoints)
	{
		TArray<FString> PointNames;

		CalibrationPoint->GetNamespacedPointNames(PointNames);

		for (FString& PointName : PointNames)
		{
			CurrentCalibratorPoints.Add(MakeShared<FCalibratorPointData>(PointName));
		}

		// Only add calibration point components that have an attached scene component
		if (CalibrationPoint->GetAttachParent())
		{
			ActiveCalibratorComponents.Add(CalibrationPoint);
		}
	}

	// Notify combobox

	if (!CalibratorPointsComboBox)
	{
		return;
	}

	CalibratorPointsComboBox->RefreshOptions();

	if (CurrentCalibratorPoints.Num())
	{
		CalibratorPointsComboBox->SetSelectedItem(CurrentCalibratorPoints[0]);
	}
	else
	{
		CalibratorPointsComboBox->SetSelectedItem(nullptr);
	}
}

AActor* UCameraNodalOffsetAlgoPoints::GetCalibrator() const
{
	return Calibrator.Get();
}

void UCameraNodalOffsetAlgoPoints::OnSavedNodalOffset()
{
	// Since the nodal point was saved, there is no further use for the current samples.
	ClearCalibrationRows();
}

void UCameraNodalOffsetAlgoPoints::ClearCalibrationRows()
{
	CalibrationRows.Empty();

	if (CalibrationListView.IsValid())
	{
		CalibrationListView->RequestListRefresh();
	}

	// End the current calibration session (a new one will begin the next time a new row is added)
	if (UNodalOffsetTool* Tool = NodalOffsetTool.Get())
	{
		Tool->EndCalibrationSession();
	}
}

void UCameraNodalOffsetAlgoPoints::UndistortCalibrationRowPoints()
{
	if (CalibrationRows.Num() == 0)
	{
		return;
	}

	// If the data for these rows was already undistorted, early-out
	const TSharedPtr<FNodalOffsetPointsRowData>& FirstRow = CalibrationRows[0];
	if (FirstRow->bUndistortedIsValid)
	{
		return;
	}

	if (!NodalOffsetTool.IsValid())
	{
		return;
	}

	const FCameraCalibrationStepsController* StepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

	if (!StepsController)
	{
		return;
	}

	const ULensDistortionModelHandlerBase* DistortionHandler = StepsController->GetDistortionHandler();
	if (!DistortionHandler)
	{
		return;
	}

	const int32 NumPoints = CalibrationRows.Num();

	TArray<FVector2f> ImagePoints;
	ImagePoints.Reserve(NumPoints);

	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : CalibrationRows)
	{
		ImagePoints.Add(Row->Point2D);
	}

	TArray<FVector2f> UndistortedPoints;
	UndistortedPoints.AddZeroed(ImagePoints.Num());
	DistortionRenderingUtils::UndistortImagePoints(DistortionHandler->GetDistortionDisplacementMap(), ImagePoints, UndistortedPoints);

	for (int32 Index = 0; Index < NumPoints; ++Index)
	{
		const TSharedPtr<FNodalOffsetPointsRowData>& Row = CalibrationRows[Index];
		Row->UndistortedPoint2D = UndistortedPoints[Index];
		Row->bUndistortedIsValid = true;
	}
}

bool UCameraNodalOffsetAlgoPoints::GetStepsControllerAndLensFile(
	const FCameraCalibrationStepsController** OutStepsController,
	const ULensFile** OutLensFile) const
{
	if (!NodalOffsetTool.IsValid())
	{
		return false;
	}

	if (OutStepsController)
	{
		*OutStepsController = NodalOffsetTool->GetCameraCalibrationStepsController();

		if (!*OutStepsController)
		{
			return false;
		}
	}

	if (OutLensFile)
	{
		if (!OutStepsController)
		{
			return false;
		}

		*OutLensFile = (*OutStepsController)->GetLensFile();

		if (!*OutLensFile)
		{
			return false;
		}
	}

	return true;
}

void UCameraNodalOffsetAlgoPoints::GroupRowsByCameraPose(
	TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>>& OutSamePoseRowGroups,
	const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows) const
{
	for (const TSharedPtr<FNodalOffsetPointsRowData>& Row : Rows)
	{
		check(Row.IsValid());

		// Find the group it belongs to based on transform
		TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>> ClosestGroup;

		for (const auto& SamePoseRowGroup : OutSamePoseRowGroups)
		{
			if (FCameraCalibrationUtils::IsNearlyEqual(Row->CameraData.Pose, (*SamePoseRowGroup)[0]->CameraData.Pose))
			{
				ClosestGroup = SamePoseRowGroup;
				break;
			}
		}

		if (!ClosestGroup.IsValid())
		{
			ClosestGroup = MakeShared<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>();
			OutSamePoseRowGroups.Add(ClosestGroup);
		}

		ClosestGroup->Add(Row);
	}
}

void UCameraNodalOffsetAlgoPoints::ExportSessionData()
{
	using namespace UE::CameraCalibration::Private;

	if (UNodalOffsetTool* Tool = NodalOffsetTool.Get())
	{
		// Add all data to a json object that is needed to run this algorithm that is NOT part of a specific row
		TSharedPtr<FJsonObject> JsonSessionData = MakeShared<FJsonObject>();

		JsonSessionData->SetNumberField(NodalOffsetPointsExportFields::Version, DATASET_VERSION);

		// Export the session data to a .json file
		Tool->ExportSessionData(JsonSessionData.ToSharedRef());
	}
}

void UCameraNodalOffsetAlgoPoints::ExportRow(TSharedPtr<FNodalOffsetPointsRowData> Row)
{
	if (UNodalOffsetTool* Tool = NodalOffsetTool.Get())
	{
		if (const TSharedPtr<FJsonObject>& RowObject = FJsonObjectConverter::UStructToJsonObject<FNodalOffsetPointsRowData>(Row.ToSharedRef().Get()))
		{
			// Export the row data to a .json file
			Tool->ExportCalibrationRow(Row->Index, RowObject.ToSharedRef());
		}
	}
}

bool UCameraNodalOffsetAlgoPoints::HasCalibrationData() const
{
	return (CalibrationRows.Num() > 0);
}

void UCameraNodalOffsetAlgoPoints::PreImportCalibrationData()
{
	// Clear the current set of rows before importing new ones
	CalibrationRows.Empty();
}

int32 UCameraNodalOffsetAlgoPoints::ImportCalibrationRow(const TSharedRef<FJsonObject>& CalibrationRowObject, const FImage& RowImage)
{
	UNodalOffsetTool* Tool = NodalOffsetTool.Get();
	if (Tool == nullptr)
	{
		return -1;
	}

	const FCameraCalibrationStepsController* StepsController = Tool->GetCameraCalibrationStepsController();
	if (StepsController == nullptr)
	{
		return -1;
	}

	// Create a new row to populate with data from the Json object
	TSharedPtr<FNodalOffsetPointsRowData> NewRow = MakeShared<FNodalOffsetPointsRowData>();

	// We enforce strict mode to ensure that every field in the UStruct of row data is present in the imported json.
	// If any fields are missing, it is likely the row will be invalid, which will lead to errors in the calibration.
	constexpr int64 CheckFlags = 0;
	constexpr int64 SkipFlags = 0;
	constexpr bool bStrictMode = true;
	if (FJsonObjectConverter::JsonObjectToUStruct<FNodalOffsetPointsRowData>(CalibrationRowObject, NewRow.Get(), CheckFlags, SkipFlags, bStrictMode))
	{
		CalibrationRows.Add(NewRow);

		// Set the camera guid to match the currently selected camera
		if (const ACameraActor* Camera = StepsController->GetCamera())
		{
			NewRow->CameraData.UniqueId = Camera->GetUniqueID();
		}

		// Set the camera parent guid to match the parent of the currently selected camera
		if (const ACameraActor* Camera = StepsController->GetCamera())
		{
			if (const AActor* CameraParentActor = Camera->GetAttachParentActor())
			{
				NewRow->CameraData.ParentUniqueId = CameraParentActor->GetUniqueID();
			}
		}

		// Set the calibrator guid to match the currently selected calibrator
		if (AActor* CalibratorActor = Calibrator.Get())
		{
			NewRow->CameraData.CalibratorUniqueId = Calibrator->GetUniqueID();
		}

		// Set the calibrator parent guid to match the parent of the currently selected calibrator
		if (AActor* CalibratorActor = Calibrator.Get())
		{
			if (const AActor* CalibratorParentActor = Calibrator->GetAttachParentActor())
			{
				NewRow->CameraData.CalibratorParentUniqueId = CalibratorParentActor->GetUniqueID();
			}
		}
	}
	else
	{
		UE_LOGF(LogCameraCalibrationEditor, Warning, "NodalOffsetPoints algo failed to import calibration row because at least one field could not be deserialized from the json file.");
	}

	return NewRow->Index;
}

void UCameraNodalOffsetAlgoPoints::PostImportCalibrationData()
{
	// Sort imported calibration rows by row index
	CalibrationRows.Sort([](const TSharedPtr<FNodalOffsetPointsRowData>& LHS, const TSharedPtr<FNodalOffsetPointsRowData>& RHS) { return LHS->Index < RHS->Index; });

	// Notify the ListView of the new data
	if (CalibrationListView.IsValid())
	{
		CalibrationListView->RequestListRefresh();
	}
}

TSharedRef<SWidget> UCameraNodalOffsetAlgoPoints::BuildHelpWidget()
{
	return SNew(STextBlock)
		.Text(LOCTEXT("NodalOffsetAlgoPointsHelp",
			"This nodal offset algorithm will estimate the camera pose by minimizing the reprojection\n"
			"error of a set of 3d points.\n\n"

			"The 3d points are taken from the calibrator object, which you need to select using the\n"
			"provided picker. All that is required is that the object contains one or more 'Calibration\n'"
			"Point Components'. These 3d calibration points will appear in the provided drop-down.\n\n"

			"To build the table that correlates these 3d points with where they are in the media plate,\n"
			"simply click on the viewport, as accurately as possible, where their physical counterpart\n"
			"appears. You can right-click the viewport to pause it if it helps in accuracy.\n\n"

			"Once the table is built, the algorithm will calculate where the camera must be so that\n"
			"the projection of these 3d points onto the camera plane are as close as possible to their\n"
			"actual 2d location that specified by clicking on the viewport.\n\n"

			"This camera pose information can then be used in the following ways:\n\n"

			"- To calculate the offset between where it currently is and where it should be. This offset\n"
			"  will be added to the lens file when 'Add To Nodal Offset Calibration' is clicked, and will\n"
			"  ultimately be applied to the tracking data so that the camera's position in the CG scene\n"
			"  is accurate. This requires that the position of the calibrator is accurate with respect to\n"
			"  the camera tracking system.\n\n"

			"- To place the calibrator actor, and any actors parented to it, in such a way that it coincides\n"
			"  with its physical counterpart as seen by both the live action camera and the virtual camera.\n"
			"  In this case, it is not required that the calibrator is tracked, and its pose will be\n"
			"  altered directly. In this case, the lens file is not modified, and requires that the camera\n"
			"  nodal offset (i.e. no parallax point) is already calibrated.\n\n"

			"- The same as above, but by offsetting the calibrator's parent. In this case, it is implied\n"
			"  that we are adjusting the calibrator's tracking system origin.\n\n"

			"- Similarly as above, but by offsetting the camera's parent. The camera lens file is not\n"
			"  changed, and it is implied that we are calibrating the camera tracking system origin.\n\n"

			"Notes:\n\n"
			" - This calibration step relies on the camera having a lens distortion calibration.\n"
			" - It requires the camera to not move much from the moment you capture the first\n"
			"   point until you capture the last one.\n"
		));
}

#undef LOCTEXT_NAMESPACE
