// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraNodalOffsetAlgo.h"

#include "LensFile.h"

#include "CameraNodalOffsetAlgoPoints.generated.h"

class FCameraCalibrationStepsController;

template <typename ItemType>
class SListView;

class FJsonObject;
class UCalibrationPointComponent;

template<typename OptionType>
class SComboBox;


namespace CameraNodalOffsetAlgoPoints
{
	class SCalibrationRowGenerator;
};

/** Holds information of the identified calibrator 2d point for a given sample of a 2d-3d correlation */
USTRUCT()
struct FNodalOffsetPointsCalibratorPointData
{
	GENERATED_BODY()

	// True if the rest of the contents are valid.
	UPROPERTY()
	bool bIsValid = false;

	// The name of the 3d calibrator point, as defined in the mesh
	UPROPERTY()
	FString Name;

	// The world space 3d location of the point
	UPROPERTY()
	FVector Location = FVector::ZeroVector;
};

/** Holds information of the camera pose for a given sample of a 2d-3d correlation */
USTRUCT()
struct FNodalOffsetPointsCameraData
{
	GENERATED_BODY()

	// True if the rest of the contents are valid.
	UPROPERTY()
	bool bIsValid = false;

	// The unique id of the camera object. Used to detect camera selection changes during a calibration session.
	UPROPERTY()
	uint32 UniqueId = INDEX_NONE;

	// The camera pose
	UPROPERTY()
	FTransform Pose = FTransform::Identity;

	// True if the cached camera pose included a nodal offset
	UPROPERTY()
	bool bWasNodalOffsetApplied = false;

	// True if distortion was evaluated (using this LensFile) 
	UPROPERTY()
	bool bWasDistortionEvaluated = false;

	// Input focus from lens file evaluation data
	UPROPERTY()
	float InputFocus = 0.0f;

	// Input zoom from lens file evaluation data
	UPROPERTY()
	float InputZoom = 0.0f;

	// The parent pose (expected to be the tracker origin)
	UPROPERTY()
	FTransform ParentPose = FTransform::Identity;

	// The parent unique id
	UPROPERTY()
	uint32 ParentUniqueId = INDEX_NONE;

	// Calibrator Pose
	UPROPERTY()
	FTransform CalibratorPose = FTransform::Identity;

	// Calibrator ParentPose
	UPROPERTY()
	FTransform CalibratorParentPose = FTransform::Identity;

	// Calibrator ComponentPose
	UPROPERTY()
	TMap<uint32, FTransform> CalibratorComponentPoses;

	// Calibrator unique id
	UPROPERTY()
	uint32 CalibratorUniqueId = INDEX_NONE;

	// Calibrator parent unique id
	UPROPERTY()
	uint32 CalibratorParentUniqueId = INDEX_NONE;
};

/** Holds information of the calibrator 3d point for a given sample of a 2d-3d correlation */
USTRUCT()
struct FNodalOffsetPointsRowData
{
	GENERATED_BODY()

	// Index of this calibration row
	UPROPERTY()
	int32 Index = INDEX_NONE;

	// Normalized 0~1 2d location of the identified calibrator point in the media plate.
	UPROPERTY()
	FVector2f Point2D = FVector2f::ZeroVector;

	// Location of Point2D after it has been undistorted
	UPROPERTY()
	FVector2f UndistortedPoint2D = FVector2f::ZeroVector;

	// True if this row has had its Point2D undistorted
	UPROPERTY()
	bool bUndistortedIsValid = false;

	// Holds information of the calibrator point data for this sample.
	UPROPERTY()
	FNodalOffsetPointsCalibratorPointData CalibratorPointData;

	// Holds information of the camera data for this sample
	UPROPERTY()
	FNodalOffsetPointsCameraData CameraData;
};

/** 
 * Implements a nodal offset calibration algorithm. It uses 3d points (UCalibrationPointComponent) 
 * specified in a selectable calibrator with features that the user can identify by clicking on the 
 * simulcam viewport, and after enough points have been identified, it can calculate the nodal offset
 * that needs to be applied to the associated camera in order to align its CG with the live action media plate.
 * 
 * Limitations:
 * - Only supports Brown–Conrady model lens model (FSphericalDistortionParameters)
 * - The camera or the lens should not be moved during the calibration of each nodal offset sample.
 */
UCLASS()
class UCameraNodalOffsetAlgoPoints : public UCameraNodalOffsetAlgo
{
	GENERATED_BODY()
	
public:

	//~ Begin CalibPointsNodalOffsetAlgo
	virtual void Initialize(UNodalOffsetTool* InNodalOffsetTool) override;
	virtual void Shutdown() override;
	virtual void Tick(float DeltaTime) override;
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual TSharedRef<SWidget> BuildUI() override;
	virtual bool GetNodalOffset(FNodalPointOffset& OutNodalOffset, float& OutFocus, float& OutZoom, float& OutError, FText& OutErrorMessage) override;
	virtual FName FriendlyName() const override { return TEXT("Nodal Offset Points Method"); };
	virtual FName ShortName() const override { return TEXT("Points"); };
	virtual void OnSavedNodalOffset() override;
	virtual bool HasCalibrationData() const override;
	virtual void PreImportCalibrationData() override;
	virtual int32 ImportCalibrationRow(const TSharedRef<FJsonObject>& CalibrationRowObject, const FImage& RowImage) override;
	virtual void PostImportCalibrationData() override;
	virtual TSharedRef<SWidget> BuildHelpWidget() override;
	//~ End CalibPointsNodalOffsetAlgo

protected:

	// SCalibrationRowGenerator will need access to the row structures below.
	friend class CameraNodalOffsetAlgoPoints::SCalibrationRowGenerator;

	/** Holds item information a given calibrator point in the calibrator model */
	struct FCalibratorPointData
	{
		FCalibratorPointData(FString& InName)
			: Name(InName)
		{}

		/** Name of the calibrator 3d point, as defined in the mesh */
		FString Name;
	};

protected:
	/** Version of the current calibration dataset */
	static const int DATASET_VERSION;

	/** The nodal offset tool controller */
	TWeakObjectPtr<UNodalOffsetTool> NodalOffsetTool;

	/** The currently selected calibrator object. It is expected to contain one or more UCalibrationPointComponent in it */
	TWeakObjectPtr<AActor> Calibrator;

	/** Container for the set of calibrator components selected in the component combobox */
	TArray<TWeakObjectPtr<const UCalibrationPointComponent>> ActiveCalibratorComponents;

	/** Options source for the CalibratorPointsComboBox. Lists the calibrator points found in the selected calibrator object */
	TArray<TSharedPtr<FCalibratorPointData>> CurrentCalibratorPoints;

	/** Allows the selection of calibrator point that will be visually identified in the simulcam viewport */
	TSharedPtr<SComboBox<TSharedPtr<FCalibratorPointData>>> CalibratorPointsComboBox;

	/** Rows source for the CalibrationListView */
	TArray<TSharedPtr<FNodalOffsetPointsRowData>> CalibrationRows;

	/** Displays the list of calibration points that will be used to calculate the nodal offset */
	TSharedPtr<SListView<TSharedPtr<FNodalOffsetPointsRowData>>> CalibrationListView;

	/** Caches the last calibrator point 3d location.  Will hold last value before the nodal offset tool is paused */
	TArray<FNodalOffsetPointsCalibratorPointData> LastCalibratorPoints;

	/** Caches the last camera data.  Will hold last value before the nodal offset tool is paused */
	FNodalOffsetPointsCameraData LastCameraData;

	/** Instructs the inline allocation policy how many elements to allocate in a single allocation */
	static constexpr uint32 NumInlineAllocations = 32;

protected:

	/** Builds the UI of the calibration device picker */
	TSharedRef<SWidget> BuildCalibrationDevicePickerWidget();

	/** Builds the UI of the calibration component picker */
	TSharedRef<SWidget> BuildCalibrationComponentPickerWidget();

	/** Builds the UI of the calibration points table */
	TSharedRef<SWidget> BuildCalibrationPointsTable();

	/** Buils the UI for selecting the current calibrator point */
	TSharedRef<SWidget> BuildCalibrationPointsComboBox();

	/** Builds the UI for the action buttons (RemoveLast, ClearAll) */
	TSharedRef<SWidget> BuildCalibrationActionButtons();

	/** Builds the UI of the calibration component picker */
	TSharedRef<SWidget> BuildCalibrationComponentMenu();

protected:

	/** Returns the first calibrator object in the scene that it can find */
	virtual AActor* FindFirstCalibrator() const;

	/** Sets the calibrator object to be used. Updates the selection picker. */
	void SetCalibrator(AActor* InCalibrator);

	/** Returns the currently selected calibrator object. */
	AActor* GetCalibrator() const;

	/** Clears the list of calibration sample rows */
	void ClearCalibrationRows();

	/* Undistort the 2D points in each calibration row using the current distortion displacement map */
	void UndistortCalibrationRowPoints();

	/** 
	 * Retrieves by name the calibration point data of the currently selected calibrator.
	 * 
	 * @param Name The name of the point (namespaced if it is a subpoint).
	 * @param CalibratorPointCache This data structure will be populated with the information found.
	 * 
	 * @return True if successful.
	 */
	bool CalibratorPointCacheFromName(const FString& Name, FNodalOffsetPointsCalibratorPointData& CalibratorPointCache) const;

	/** Returns the world 3d location of the currently selected calibrator */
	bool GetCurrentCalibratorPointLocation(FVector& OutLocation);

	/** Selects the next available UCalibrationPointComponent of the currently selected calibrator object. Returns true when it wraps around */
	bool AdvanceCalibratorPoint();

	/** Add or remove the selected component from the set of active components and refresh the calibrator point combobox */
	void OnCalibrationComponentSelected(const UCalibrationPointComponent* const SelectedComponent);

	/** Returns true if the input component is in the set of active calibration components */
	bool IsCalibrationComponentSelected(const UCalibrationPointComponent* const SelectedComponent) const;

	/** Update any calibration components in the set of active components if they were replaced by an in-editor event */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Validates a new calibration point to determine if it should be added as a new sample row */
	virtual bool ValidateNewRow(TSharedPtr<FNodalOffsetPointsRowData>& Row, FText& OutErrorMessage) const;

	/** Applies the nodal offset to the calibrator */
	bool ApplyNodalOffsetToCalibrator();

	/** Applies the nodal offset to the tracker origin (normally the camera parent) */
	bool ApplyNodalOffsetToTrackingOrigin();

	/** Applies the nodal offset to the parent of the calibrator */
	bool ApplyNodalOffsetToCalibratorParent();

	/** Applies the nodal offset to the parent of the calibrator */
	bool ApplyNodalOffsetToCalibratorComponents();

	/** Does basic checks on the data before performing the actual calibration */
	bool BasicCalibrationChecksPass(const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, FText& OutErrorMessage) const;

	/** 
	 * Gets the step controller and the lens file. 
	 *
	 * @param OutStepsController Steps Controller
	 * @param OutLensFile Lens File
	 * 
	 * @return True if successful
	 */
	bool GetStepsControllerAndLensFile(const FCameraCalibrationStepsController** OutStepsController, const ULensFile** OutLensFile) const;

	/** 
	 * Calculates the optimal camera component pose that minimizes the reprojection error 
	 * 
	 * @param OutDesiredCameraTransform The camera transform in world coordinates that minimizes the reprojection error.
	 * @param Rows Calibration samples.
	 * @param OutErrorMessage Documents any error that may have happened.
	 * 
	 * @return True if successful.
	 */
	bool CalculatedOptimalCameraComponentPose(
		FTransform& OutDesiredCameraTransform, 
		const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, 
		FText& OutErrorMessage) const;

	/**
	 * Calculates nodal offset based on a single camera pose 
	 * 
	 * @param OutNodalOffset Camera nodal offset
	 * @param OutFocus Focus distance associated with this nodal offset
	 * @param OutZoom Focal length associated with this nodal offset
	 * @param OutError Reprojection error
	 * @param Rows Calibration samples
	 * @param OutErrorMessage Describes any error that may have happened
	 * 
	 * @return True if successful
	 */
	bool GetNodalOffsetSinglePose(
		FNodalPointOffset& OutNodalOffset, 
		float& OutFocus, 
		float& OutZoom, 
		float& OutError, 
		const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows, 
		FText& OutErrorMessage) const;

	/**
	 * Groups the calibration samples by camera pose.
	 *
	 * @param OutSamePoseRowGroups Output array of arrays of samples, where the samples in each group have the same camera pose.
	 * @param Rows Ungrouped calibration samples.
	 */
	void GroupRowsByCameraPose(
		TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>>& OutSamePoseRowGroups, 
		const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows) const;

	/** 
	 * Detects if the calibrator moved significantly in the given sample points 
	 * 
	 * @param OutSamePoseRowGroups Array of array of calibration samples.
	 * 
	 * @return True if the calibrator moved significantly across all the samples.
	 */
	bool CalibratorMovedAcrossGroups(const TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>>& OutSamePoseRowGroups) const;

	/**
	 * Detects if the calibrator moved significantly in the given sample points
	 *
	 * @param Rows Array of calibration samples.
	 *
	 * @return True if the calibrator moved significantly across all the samples.
	 */
	bool CalibratorMovedInAnyRow(const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows) const;

	/**
	 * Calculates the optimal camera parent transform that minimizes the reprojection of the sample points.
	 * 
	 * @param Rows The calibration points
	 * @param OutTransform The optimal camera parent transform
	 * @param OutErrorMessage Describes any errors encountered
	 * 
	 * @return True if successful
	 */
	bool CalcTrackingOriginPoseForSingleCamPose(
		const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows,
		FTransform& OutTransform,
		FText& OutErrorMessage);

	/**
	 * Calculates the optimal calibrator transform that minimizes the reprojection of the sample points.
	 *
	 * @param Rows The calibration points
	 * @param OutTransform The optimal calibrator transform
	 * @param OutErrorMessage Describes any errors encountered
	 *
	 * @return True if successful
	 */
	bool CalcCalibratorPoseForSingleCamPose(
		const TArray<TSharedPtr<FNodalOffsetPointsRowData>>& Rows,
		FTransform& OutTransform,
		FText& OutErrorMessage);

	/* Refine the input nodal offset by running a minimization algorithm designed to minimize the sum of the reprojection errors for the camera views in the set of input pose groups*/
	double MinimizeReprojectionError(FTransform& InOutNodalOffset, const TArray<TSharedPtr<TArray<TSharedPtr<FNodalOffsetPointsRowData>>>>& SamePoseRowGroups) const;

	/** Export global session data to a .json file */
	void ExportSessionData();

	/** Export the row data to a json file */
	void ExportRow(TSharedPtr<FNodalOffsetPointsRowData> Row);
};
