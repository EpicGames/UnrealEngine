// Copyright Epic Games, Inc. All Rights Reserved.


#include "CameraCalibrationEditorModule.h"

#include "ActorFactories/ActorFactoryBlueprint.h"
#include "AssetEditor/CameraCalibrationCommands.h"
#include "AssetEditor/Curves/LensDataCurveModel.h"
#include "AssetEditor/SCameraCalibrationCurveEditorView.h"
#include "AssetToolsModule.h"
#include "CameraCalibrationCharucoBoard.h"
#include "CameraCalibrationEditorLog.h"
#include "CameraCalibrationSubsystem.h"
#include "CameraCalibrationTypes.h"
#include "CameraCalibrationUtilsPrivate.h"
#include "Engine/Texture2D.h"
#include "IAssetTools.h"
#include "ICurveEditorModule.h"
#include "IPlacementModeModule.h"
#include "LensFile.h"
#include "LevelEditor.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "OpenCVHelper.h"
#include "UI/CameraCalibrationEditorStyle.h"
#include "UI/CameraCalibrationMenuEntry.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "LevelEditorOutlinerSettings.h"

#define LOCTEXT_NAMESPACE "CameraCalibrationEditor"

DEFINE_LOG_CATEGORY(LogCameraCalibrationEditor);

void FCameraCalibrationEditorModule::StartupModule()
{
	FCameraCalibrationCommands::Register();
	FCameraCalibrationEditorStyle::Get();

	{
		const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
		TSharedRef<FWorkspaceItem> BrowserGroup = MenuStructure.GetDeveloperToolsMiscCategory()->GetParent()->AddGroup(
			LOCTEXT("WorkspaceMenu_VirtualProduction", "Virtual Production"),
			FSlateIcon(),
			true);
	}

	FCameraCalibrationMenuEntry::Register();

	RegisterPlacementModeItems();

	RegisterOverlayMaterials();

	RegisterCharucoBoardTextureGenerator();

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));

	ICurveEditorModule& CurveEditorModule = FModuleManager::LoadModuleChecked<ICurveEditorModule>("CurveEditor");
	FLensDataCurveModel::ViewId = CurveEditorModule.RegisterView(FOnCreateCurveEditorView::CreateStatic(
		[](TWeakPtr<FCurveEditor> WeakCurveEditor) -> TSharedRef<SCurveEditorView>
		{
			return SNew(SCameraCalibrationCurveEditorView, WeakCurveEditor);
		}
	));
}

const FPlacementCategoryInfo* FCameraCalibrationEditorModule::GetVirtualProductionCategoryRegisteredInfo() const
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	
	if (const FPlacementCategoryInfo* RegisteredInfo = PlacementModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
	{
		return RegisteredInfo;
	}
	else
	{
		FPlacementCategoryInfo Info(
			LOCTEXT("VirtualProductionCategoryName", "Virtual Production"),
			FSlateIcon(FCameraCalibrationEditorStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.VirtualProduction"),
			FLevelEditorOutlinerBuiltInCategories::VirtualProduction(),
			TEXT("PMVirtualProduction"),
			25
		);
		Info.ShortDisplayName = LOCTEXT("VirtualProductionShortCategoryName", "VP");
		IPlacementModeModule::Get().RegisterPlacementCategory(Info);

		return PlacementModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction());
	}
}

void FCameraCalibrationEditorModule::RegisterPlacementModeItems()
{
	auto RegisterPlaceActors = [&]() -> void
	{
		if (!GEditor)
		{
			return;
		}

		const FPlacementCategoryInfo* Info = GetVirtualProductionCategoryRegisteredInfo();

		if (!Info)
		{
			UE_LOGF(LogCameraCalibrationEditor, Warning, "Could not find or create VirtualProduction Place Actor Category");
			return;
		}

		// Register the Trackers, Version 2 and 3
		{
			const FAssetData TrackerAssetDataV2(
				TEXT("/CameraCalibration/Devices/Tracker/BP_UE_Tracker"),
				TEXT("/CameraCalibration/Devices/Tracker"),
				TEXT("BP_UE_Tracker"),
				FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Blueprint"))
			);

			PlaceActors.Add(IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
				*UActorFactoryBlueprint::StaticClass(),
				TrackerAssetDataV2,
				NAME_None,
				NAME_None,
				TOptional<FLinearColor>(),
				TOptional<int32>(),
				NSLOCTEXT("PlacementMode", "TrackerV2", "TrackerV2")
			)));

			const FAssetData TrackerAssetDataV3(
				TEXT("/CameraCalibration/Devices/Tracker/BP_UE_Tracker3"),
				TEXT("/CameraCalibration/Devices/Tracker"),
				TEXT("BP_UE_Tracker3"),
				FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Blueprint"))
			);

			PlaceActors.Add(IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
				*UActorFactoryBlueprint::StaticClass(),
				TrackerAssetDataV3,
				NAME_None,
				NAME_None,
				TOptional<FLinearColor>(),
				TOptional<int32>(),
				NSLOCTEXT("PlacementMode", "TrackerV3", "TrackerV3")
				)));
		}

		// Register the Charuco Board
		{
			PlaceActors.Add(IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
				*UActorFactory::StaticClass(),
				FAssetData(ACameraCalibrationCharucoBoard::StaticClass()),
				NAME_None,
				NAME_None,
				TOptional<FLinearColor>(),
				TOptional<int32>(),
				NSLOCTEXT("PlacementMode", "CharucoBoard", "Charuco Board")
			)));
		}
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			RegisterPlaceActors();
		}
		else
		{
			PostEngineInitHandle_PlacementMode = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterPlaceActors);
		}
	}
}

void FCameraCalibrationEditorModule::RegisterCharucoBoardTextureGenerator()
{
	auto RegisterGenerator = [this]()
	{
		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		
		// Create delegate that converts shadow types to OpenCV types and generates texture
		FGenerateCharucoBoardDelegate GeneratorDelegate;
		GeneratorDelegate.BindLambda([](const FCharucoBoardConfigShadow& BoardConfig, FIntPoint ImageSize, int32 MarginSize) -> UTexture2D*
		{
			// Convert shadow config to OpenCV config
			FCharucoBoardConfig OpenCVConfig;
			OpenCVConfig.SquaresX = BoardConfig.SquaresX;
			OpenCVConfig.SquaresY = BoardConfig.SquaresY;
			OpenCVConfig.SquareSize = BoardConfig.SquareSize;
			OpenCVConfig.MarkerSize = BoardConfig.MarkerSize;
			OpenCVConfig.Dictionary = UE::CameraCalibration::Private::ShadowToOpenCV(BoardConfig.Dictionary);
			
			// Generate texture using OpenCV
			return FOpenCVHelper::GenerateCharucoBoard(OpenCVConfig, ImageSize, MarginSize);
		});
		
		SubSystem->SetCharucoBoardTextureGenerator(GeneratorDelegate);
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			RegisterGenerator();
		}
		else
		{
			PostEngineInitHandle_CharucoGenerator = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterGenerator);
		}
	}
}

void FCameraCalibrationEditorModule::UnregisterCharucoBoardTextureGenerator()
{
	if (GEngine)
	{
		if (UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
		{
			// Clear the generator delegate
			SubSystem->SetCharucoBoardTextureGenerator(FGenerateCharucoBoardDelegate());
		}
	}
}

void FCameraCalibrationEditorModule::RegisterOverlayMaterials()
{
	auto RegisterOverlays = [this]()
	{
		// Register all overlay materials defined in this module
		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		SubSystem->RegisterOverlayMaterial(TEXT("Crosshair"), FSoftObjectPath(TEXT("/CameraCalibration/Materials/M_Crosshair.M_Crosshair")));
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			RegisterOverlays();
		}
		else
		{
			PostEngineInitHandle_OverlayMaterials = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterOverlays);
		}
	}
}

void FCameraCalibrationEditorModule::UnregisterOverlayMaterials()
{
	if (GEngine)
	{
		// Unregister all overlay materials defined in this module
		if (UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
		{
			SubSystem->UnregisterOverlayMaterial(TEXT("Crosshair"));
		}
	}
}

void FCameraCalibrationEditorModule::ShutdownModule()
{
	if (!IsEngineExitRequested() && GEditor && UObjectInitialized())
	{
		FCameraCalibrationMenuEntry::Unregister();

		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");

		FCameraCalibrationCommands::Unregister();

		UnregisterPlacementModeItems();

		UnregisterOverlayMaterials();

		UnregisterCharucoBoardTextureGenerator();
	}

	if (PostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
	}
	
	if (PostEngineInitHandle_PlacementMode.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle_PlacementMode);
	}
	
	if (PostEngineInitHandle_OverlayMaterials.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle_OverlayMaterials);
	}
	
	if (PostEngineInitHandle_CharucoGenerator.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle_CharucoGenerator);
	}

	if (ICurveEditorModule* CurveEditorModule = FModuleManager::GetModulePtr<ICurveEditorModule>("CurveEditor"))
	{
		CurveEditorModule->UnregisterView(FLensDataCurveModel::ViewId);
	}
}


void FCameraCalibrationEditorModule::UnregisterPlacementModeItems()
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

	for (TOptional<FPlacementModeID>& PlaceActor : PlaceActors)
	{
		if (PlaceActor.IsSet())
		{
			PlacementModeModule.UnregisterPlaceableItem(*PlaceActor);
		}
	}

	PlaceActors.Empty();
}

IMPLEMENT_MODULE(FCameraCalibrationEditorModule, CameraCalibrationEditor);

#undef LOCTEXT_NAMESPACE
