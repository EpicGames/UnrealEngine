// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"

class FPlacementModeID;
class IAssetTypeActions;

struct FPlacementCategoryInfo;

/**
 * Implements the CameraCalibrationEditor module.
 */
class FCameraCalibrationEditorModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:

	/** Register items to show up in the Place Actors panel. */
	void RegisterPlacementModeItems();

	/** Unregister items in Place Actors panel */
	void UnregisterPlacementModeItems();

	/** Register overlay materials to use in the calibration tool */
	void RegisterOverlayMaterials();

	/** Unregister overlay materials */
	void UnregisterOverlayMaterials();

	/** Register Charuco board texture generator with the calibration subsystem */
	void RegisterCharucoBoardTextureGenerator();

	/** Unregister Charuco board texture generator */
	void UnregisterCharucoBoardTextureGenerator();

	/** Gathers the Info on the Virtual Production Place Actors Category */
	const FPlacementCategoryInfo* GetVirtualProductionCategoryRegisteredInfo() const;

private:

	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;
	FDelegateHandle PostEngineInitHandle;
	FDelegateHandle PostEngineInitHandle_PlacementMode;
	FDelegateHandle PostEngineInitHandle_OverlayMaterials;
	FDelegateHandle PostEngineInitHandle_CharucoGenerator;

	TArray<TOptional<FPlacementModeID>> PlaceActors;
};
