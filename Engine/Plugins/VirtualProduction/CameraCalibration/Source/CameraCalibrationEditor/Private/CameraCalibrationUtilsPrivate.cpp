// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationUtilsPrivate.h"

#include "CalibrationPointComponent.h"
#include "Calibrators/CameraCalibrationSolver.h"
#include "CameraCalibrationUtils.h"
#include "Editor.h"
#include "Engine/Level.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "UObject/UObjectIterator.h"

namespace UE::CameraCalibration::Private
{
	inline static const TMap<EArucoDictionary, FString> ArucoDictionaries =
	{
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_4X4_1000, TEXT("DICT_4X4_1000")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_4X4_250, TEXT("DICT_4X4_250")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_4X4_100, TEXT("DICT_4X4_100")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_4X4_50, TEXT("DICT_4X4_50")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_5X5_1000, TEXT("DICT_5X5_1000")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_5X5_250, TEXT("DICT_5X5_250")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_5X5_100, TEXT("DICT_5X5_100")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_5X5_50, TEXT("DICT_5X5_50")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_6X6_1000, TEXT("DICT_6X6_1000")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_6X6_250, TEXT("DICT_6X6_250")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_6X6_100, TEXT("DICT_6X6_100")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_6X6_50, TEXT("DICT_6X6_50")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_7X7_1000, TEXT("DICT_7X7_1000")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_7X7_250, TEXT("DICT_7X7_250")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_7X7_100, TEXT("DICT_7X7_100")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_7X7_50, TEXT("DICT_7X7_50")),
		TPair<EArucoDictionary, FString>(EArucoDictionary::DICT_ARUCO_ORIGINAL, TEXT("DICT_ARUCO_ORIGINAL"))
	};

	EArucoDictionary GetArucoDictionaryFromName(FString Name)
	{
		TSet<EArucoDictionary> Keys;
		ArucoDictionaries.GetKeys(Keys);

		for (EArucoDictionary Key : Keys)
		{
			FString DictionaryName = GetArucoDictionaryName(Key);
			if (Name.StartsWith(DictionaryName))
			{
				return Key;
			}
		}

		return EArucoDictionary::None;
	}

	FString GetArucoDictionaryName(EArucoDictionary Dictionary)
	{
		const FString* DictionaryName = ArucoDictionaries.Find(Dictionary);

		if (DictionaryName)
		{
			FString FoundDictionaryName = *DictionaryName;
			return FoundDictionaryName;
		}

		return TEXT("");
	}

	EArucoDictionary GetArucoDictionaryForCalibrator(AActor* CalibratorActor)
	{
		if (!CalibratorActor)
		{
			return EArucoDictionary::None;
		}

		// Find all calibration components belonging to the input calibrator actor
		constexpr uint32 NumInlineAllocations = 32;
		TArray<UCalibrationPointComponent*, TInlineAllocator<NumInlineAllocations>> CalibrationComponents;
		CalibratorActor->GetComponents(CalibrationComponents);

		EArucoDictionary Dictionary = EArucoDictionary::None;
		for (const UCalibrationPointComponent* Component : CalibrationComponents)
		{
			if (!Component)
			{
				continue;
			}

			// Look up the dictionary enum value based on the name of the component (only works if it is prefixed with a dictionary name)
			Dictionary = GetArucoDictionaryFromName(Component->GetName());

			if (Dictionary != EArucoDictionary::None)
			{
				break;
			}

			// Look up the dictionary enum value based on the name of each of the component's subpoints (only works if it is prefixed with a dictionary name)
			for (const TPair<FString, FVector>& SubPoint : Component->SubPoints)
			{
				Dictionary = GetArucoDictionaryFromName(SubPoint.Key);

				if (Dictionary != EArucoDictionary::None)
				{
					break;
				}
			}
		}

		return Dictionary;
	}

	void FindActorsWithCalibrationComponents(TArray<AActor*>& ActorsWithCalibrationComponents)
	{
		ActorsWithCalibrationComponents.Empty();

		const UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
		const EObjectFlags ExcludeFlags = RF_ClassDefaultObject; // We don't want the calibrator CDOs.

		for (TObjectIterator<UCalibrationPointComponent> It(ExcludeFlags, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			AActor* Actor = It->GetOwner();
			
			if (Actor && (Actor->GetWorld() == World))
			{
				// Exclude any actors belonging to a level that is not currently visible
				ULevel* Level = Actor->GetLevel();
				if (Level && Level->bIsVisible)
				{
					ActorsWithCalibrationComponents.Add(Actor);
				}
			}
		}
	}

	bool FindArucoCalibrationPoint(const TArray<TWeakObjectPtr<UCalibrationPointComponent>>& CalibrationComponents, EArucoDictionary ArucoDictionary, const FArucoMarker& ArucoMarker, FArucoCalibrationPoint& OutArucoCalibrationPoint)
	{
		// Build calibrator point name based on the detected marker
		const FString DictionaryName = GetArucoDictionaryName(ArucoDictionary);

		constexpr int32 NumExpectedCorners = 4;
		static const TArray<FString> CornerNames = { TEXT("TL"), TEXT("TR"), TEXT("BR"), TEXT("BL") };

		OutArucoCalibrationPoint.MarkerID = ArucoMarker.MarkerID;
		OutArucoCalibrationPoint.Name = FString::Printf(TEXT("%s-%d"), *DictionaryName, ArucoMarker.MarkerID);

		for (TWeakObjectPtr<UCalibrationPointComponent> Component : CalibrationComponents)
		{
			if (!Component.IsValid())
			{
				continue;
			}

			// Check each corner to see if it belongs to the current calibration component (lookup is based on the corner name)
			int32 FoundCorners = 0;
			for (int32 CornerIndex = 0; CornerIndex < NumExpectedCorners; ++CornerIndex)
			{
				const FString CornerName = FString::Printf(TEXT("%s-%s"), *OutArucoCalibrationPoint.Name, *CornerNames[CornerIndex]);
				OutArucoCalibrationPoint.Corners2D[CornerIndex] = ArucoMarker.Corners[CornerIndex];

				// If the current corner is a subpoint of the current calibration component being inspected, get its 3D world location
				FVector Corner;
				if (Component->GetWorldLocation(CornerName, Corner))
				{
					OutArucoCalibrationPoint.Corners3D[CornerIndex] = Corner;
					++FoundCorners;
				}
			}

			// If all four corners were found, we're done. Otherwise, check the next calibration component for the four corners
			if (FoundCorners == NumExpectedCorners)
			{
				return true;
			}
		}

		return false;
	}

	void ClearTexture(UTexture2D* Texture, FColor ClearColor)
	{
		if (Texture)
		{
			TArray<FColor> Pixels;
			Pixels.Init(ClearColor, Texture->GetSizeX() * Texture->GetSizeY());

			{
				FTexture2DMipMap& Mip = Texture->GetPlatformData()->Mips[0];

				const SIZE_T SrcBytes = SIZE_T(Pixels.Num()) * SIZE_T(sizeof(FColor));
				const SIZE_T DstBytes = SIZE_T(Mip.BulkData.GetBulkDataSize());

				const SIZE_T NumBytes = FMath::Min(SrcBytes, DstBytes);

				void* TextureData = Mip.BulkData.Lock(LOCK_READ_WRITE);

				ON_SCOPE_EXIT
				{
					if (Mip.BulkData.IsLocked())
					{
						Mip.BulkData.Unlock();
					}
				};

				if (!TextureData)
				{
					return;
				}

				FMemory::Memcpy(TextureData, Pixels.GetData(), NumBytes);
			}
			Texture->UpdateResource();
		}
	}

	void SetTextureData(UTexture2D* Texture, const TArray<FColor>& PixelData)
	{
		if (Texture)
		{
			{
				FTexture2DMipMap& Mip = Texture->GetPlatformData()->Mips[0];

				const SIZE_T SrcBytes = SIZE_T(PixelData.Num()) * SIZE_T(sizeof(FColor));
				const SIZE_T DstBytes = SIZE_T(Mip.BulkData.GetBulkDataSize());

				const SIZE_T NumBytes = FMath::Min(SrcBytes, DstBytes);

				void* TextureData = Mip.BulkData.Lock(LOCK_READ_WRITE);

				ON_SCOPE_EXIT
				{
					if (Mip.BulkData.IsLocked())
					{
						Mip.BulkData.Unlock();
					}
				};

				if (!TextureData)
				{
					return;
				}

				FMemory::Memcpy(TextureData, PixelData.GetData(), NumBytes);
			}

			Texture->UpdateResource();
		}
	}

	void GroupPointsByCameraPose(TArray<FObjectPoints>& InOutObjectPoints, TArray<FImagePoints>& InOutImagePoints, TArray<FTransform>& InOutCameraPoses)
	{
		const int NumPoses = InOutCameraPoses.Num();
		if (!ensure(NumPoses == InOutObjectPoints.Num()) && (NumPoses == InOutImagePoints.Num()))
		{
			return;
		}

		TArray<FObjectPoints> GroupedObjectPoints;
		TArray<FImagePoints> GroupedImagePoints;
		TArray<FTransform> GroupedCameraPoses;

		FTransform LastCameraPose = InOutCameraPoses[0];
		FTransform CurrentCameraPose;

		FObjectPoints CurrentObjectPoints;
		FImagePoints CurrentImagePoints;
		for (int32 PoseIndex = 0; PoseIndex < NumPoses; ++PoseIndex)
		{
			CurrentCameraPose = InOutCameraPoses[PoseIndex];

			if (!FCameraCalibrationUtils::IsNearlyEqual(CurrentCameraPose, LastCameraPose))
			{
				GroupedCameraPoses.Add(LastCameraPose);
				LastCameraPose = CurrentCameraPose;

				GroupedObjectPoints.Add(CurrentObjectPoints);
				CurrentObjectPoints.Points.Reset();

				GroupedImagePoints.Add(CurrentImagePoints);
				CurrentImagePoints.Points.Reset();
			}

			CurrentObjectPoints.Points.Append(InOutObjectPoints[PoseIndex].Points);
			CurrentImagePoints.Points.Append(InOutImagePoints[PoseIndex].Points);
		}

		if (!CurrentObjectPoints.Points.IsEmpty())
		{
			GroupedObjectPoints.Add(CurrentObjectPoints);
			GroupedImagePoints.Add(CurrentImagePoints);
			GroupedCameraPoses.Add(LastCameraPose);
		}

		InOutObjectPoints = GroupedObjectPoints;
		InOutImagePoints = GroupedImagePoints;
		InOutCameraPoses = GroupedCameraPoses;
	}
}
