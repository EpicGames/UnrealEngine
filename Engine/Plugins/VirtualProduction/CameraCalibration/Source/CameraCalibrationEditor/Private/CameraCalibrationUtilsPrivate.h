// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CameraCalibrationCharucoBoard.h" // For EArucoDictionaryShadow
#include "OpenCVHelper.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "CameraCalibrationUtilsPrivate.generated.h"

class UCalibrationPointComponent;

struct FObjectPoints;
struct FImagePoints;

/** Structure representing an aruco marker, including ID, Name, and corner coordinates in 2D image space and 3D world space */
USTRUCT()
struct FArucoCalibrationPoint
{
	GENERATED_BODY()

	/** 3D locations (in world space) of each corner */
	UPROPERTY()
	FVector Corners3D[4] = { FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector };

	/** 2D locations (in pixels) of each corner */
	UPROPERTY()
	FVector2f Corners2D[4] = { FVector2f::ZeroVector, FVector2f::ZeroVector, FVector2f::ZeroVector, FVector2f::ZeroVector };

	/** Aruco Marker ID */
	UPROPERTY()
	int32 MarkerID = -1;
	
	/** Name of the marker (follows the naming convention of "[DictionaryName]-[MarkerID]") */
	UPROPERTY()
	FString Name;
};

namespace UE::CameraCalibration::Private
{
	/** Returns the aruco dictionary that matches the calibration point component(s) of the input calibrator actor */
	EArucoDictionary GetArucoDictionaryForCalibrator(AActor* CalibratorActor);

	/** Get the aruco dictionary from a string representation matching the dictionary name */
	EArucoDictionary GetArucoDictionaryFromName(FString Name);

	/** Get the string representation of the input aruco dictionary */
	FString GetArucoDictionaryName(EArucoDictionary Dictionary);

	/** Convert shadow enum to OpenCV enum */
	inline EArucoDictionary ShadowToOpenCV(EArucoDictionaryShadow Shadow)
	{
		return static_cast<EArucoDictionary>(Shadow);
	}

	/** Convert OpenCV enum to shadow enum */
	inline EArucoDictionaryShadow OpenCVToShadow(EArucoDictionary OpenCV)
	{
		return static_cast<EArucoDictionaryShadow>(OpenCV);
	}

	/** Find all actors in the current world that have a calibration point component attached (excluding CDOs and actors in levels that are not currently visible) */
	void FindActorsWithCalibrationComponents(TArray<AActor*>& ActorsWithCalibrationComponents);

	/** Find an aruco marker calibration point in one of the input calibration components that matches the input dictionary and marker ID */
	bool FindArucoCalibrationPoint(const TArray<TWeakObjectPtr<UCalibrationPointComponent>>& CalibrationComponents, EArucoDictionary ArucoDictionary, const FArucoMarker& ArucoMarker, FArucoCalibrationPoint& OutArucoCalibrationPoint);

	/** Set every pixel in the input texture to the clear color */
	void ClearTexture(UTexture2D* Texture, FColor ClearColor = FColor::Transparent);

	/** Set the texture data to the input array of pixels */
	void SetTextureData(UTexture2D* Texture, const TArray<FColor>& PixelData);

	/** Group the input points together if they share the same camera pose, and output the unique camera poses */
	void GroupPointsByCameraPose(TArray<FObjectPoints>& InOutObjectPoints, TArray<FImagePoints>& InOutImagePoints, TArray<FTransform>& InOutCameraPoses);
}
