// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensFileExporter.h"
#include "JsonObjectConverter.h"
#include "LensFile.h"
#include "LensFileExchangeFormat.h"
#include "Models/LensModel.h"


ULensFileExporter::ULensFileExporter(const FObjectInitializer& ObjectInitializer)
	: Super{ ObjectInitializer }
{
	SupportedClass = ULensFile::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("ulens"));
	FormatDescription.Add(TEXT("Unreal LensFile"));
}

bool ULensFileExporter::ExportText(const class FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	const ULensFile* LensFile = Cast<ULensFile>(Object);
	if (LensFile == nullptr)
	{
		return false;
	}

	FLensFileExchange LensFileExchange{ LensFile };

	FJsonObjectConverter::CustomExportCallback CustomExportCallback;
	CustomExportCallback.BindLambda([](FProperty* Property, const void* Value) -> TSharedPtr<FJsonValue>
	{
		TSharedPtr<FJsonValue> Result = nullptr;

		if (FStructProperty* PropertyAsStruct = CastField<FStructProperty>(Property))
		{
			// If this is a FLensFileParameterTable we need to do a custom export
			if (PropertyAsStruct->Struct->IsChildOf(FLensFileParameterTable::StaticStruct()))
			{
				// Cast the property to our known struct type
				const FLensFileParameterTable* ParameterTable = (const FLensFileParameterTable*)Value;

				// Use the reflection system to get the FProperty representing the fields of the table
				FProperty* ParameterNameProperty = ParameterTable->StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FLensFileParameterTable, ParameterName));
				FProperty* HeaderProperty = ParameterTable->StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FLensFileParameterTable, Header));
				FProperty* DataProperty = ParameterTable->StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FLensFileParameterTable, Data));

				// Convert the Header and Data field to Strings
				FString HeaderValuesStr;
				const int32 NumHeaders = ParameterTable->Header.Num();
				for (int32 HeaderIndex = 0; HeaderIndex < NumHeaders; ++HeaderIndex)
				{
					const FName& Header = ParameterTable->Header[HeaderIndex];
					HeaderValuesStr.Append(Header.ToString());
					if (HeaderIndex < (NumHeaders - 1))
					{
						HeaderValuesStr.Append(TEXT(", "));
					}
				}

				FString DataStr;
				const int32 NumDataRows = ParameterTable->Data.Num();
				for (int32 DataRowIndex = 0; DataRowIndex < NumDataRows; ++DataRowIndex)
				{
					const FLensFileParameterTableRow& DataRow = ParameterTable->Data[DataRowIndex];
					const int32 NumDataElements = DataRow.Values.Num();

					for (int32 DataIndex = 0; DataIndex < NumDataElements; ++DataIndex)
					{
						const float DataElement = DataRow.Values[DataIndex];
						DataStr.Append(FString::SanitizeFloat(DataElement));
						if (DataIndex < (NumDataElements - 1))
						{
							DataStr.Append(TEXT(", "));
						}
					}

					if (DataRowIndex < (NumDataRows - 1))
					{
						DataStr.Append(TEXT("; "));
					}
				}

				// Convert the ParameterNameProperty to a Json representation
				TSharedPtr<FJsonValue> ParameterNameJson = FJsonObjectConverter::UPropertyToJsonValue(ParameterNameProperty, &ParameterTable->ParameterName);

				// Json Object that will hold the serialized FLensFileParameterTable
				TSharedPtr<FJsonObject> ParameterTableJson = MakeShared<FJsonObject>();
				ParameterTableJson->SetField(FJsonObjectConverter::StandardizeCase(ParameterNameProperty->GetName()), ParameterNameJson);
				ParameterTableJson->SetStringField(FJsonObjectConverter::StandardizeCase(HeaderProperty->GetName()), HeaderValuesStr);
				ParameterTableJson->SetStringField(FJsonObjectConverter::StandardizeCase(DataProperty->GetName()), DataStr);

				Result = MakeShared<FJsonValueObject>(ParameterTableJson);
			}
		}

		return Result;
	});

	const int64 CheckFlags = 0;
	const int64 SkipFlags = 0;
	const int32 Indent = 0;
	const bool bPrettyPrint = true;
	FString SerializedJson;
	if (FJsonObjectConverter::UStructToJsonObjectString<FLensFileExchange>(LensFileExchange, SerializedJson, CheckFlags, SkipFlags, Indent, &CustomExportCallback, bPrettyPrint))
	{
		Ar.Log(SerializedJson);

		return true;
	}
	else
	{
		return false;
	}
}