// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "LensFileExporter.generated.h"



UCLASS()
class ULensFileExporter : public UExporter
{
	GENERATED_UCLASS_BODY()

	//~ Begin UExporter interface
	bool ExportText(const class FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags/* =0 */) override;
	//~ End UExporter interface
};