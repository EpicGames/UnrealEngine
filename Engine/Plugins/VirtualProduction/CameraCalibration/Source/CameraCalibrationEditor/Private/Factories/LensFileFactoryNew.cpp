// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/LensFileFactoryNew.h"

#include "AssetToolsModule.h"
#include "LensFile.h"


/* ULensFileFactoryNew structors
 *****************************************************************************/

ULensFileFactoryNew::ULensFileFactoryNew()
{
	SupportedClass = ULensFile::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* ULensFileFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<ULensFile>(InParent, InClass, InName, Flags);
}


uint32 ULensFileFactoryNew::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}


bool ULensFileFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
