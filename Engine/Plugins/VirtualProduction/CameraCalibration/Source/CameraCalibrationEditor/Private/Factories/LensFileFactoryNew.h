// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "UObject/ObjectMacros.h"

#include "LensFileFactoryNew.generated.h"

/**
 * Implements a factory for ULensFile objects.
 */
UCLASS(hidecategories=Object)
class ULensFileFactoryNew : public UFactory
{
	GENERATED_BODY()

public:

	ULensFileFactoryNew();

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
	//~ End UFactory Interface
};
