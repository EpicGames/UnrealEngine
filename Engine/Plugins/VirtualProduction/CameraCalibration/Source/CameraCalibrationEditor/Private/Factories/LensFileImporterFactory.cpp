// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensFileImporterFactory.h"

#include "CoreMinimal.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "JsonObjectConverter.h"
#include "LensFile.h"
#include "LensFileExchangeFormat.h"
#include "Misc/FileHelper.h"


#define LOCTEXT_NAMESPACE "LensFile Importer"

ULensFileImporterFactory::ULensFileImporterFactory(const FObjectInitializer& ObjectInitializer)
	: Super{ ObjectInitializer }
{
	bCreateNew = false;
	bEditAfterNew = false;
	SupportedClass = ULensFile::StaticClass();
	bEditorImport = true;
	bText = true;

	Formats.Add(TEXT("ulens;Unreal LensFile"));
}

FText ULensFileImporterFactory::GetToolTip() const
{
	return LOCTEXT("LensFileImporterFactoryDescription", "LensFile conformant with the ulens specification");
}

bool ULensFileImporterFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);
	return Extension.Compare(TEXT("ulens"), ESearchCase::IgnoreCase) == 0;
}

UObject* ULensFileImporterFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	ULensFile* LensFile = nullptr;

	FJsonObjectWrapper JsonWrapper;
	if (JsonWrapper.JsonObjectFromString(Buffer))
	{
		bool bLoadedSuccesfully = true;

		const FLensFileExchange LensFileExchange{ JsonWrapper.JsonObject.ToSharedRef(), bLoadedSuccesfully, Warn };

		if (bLoadedSuccesfully)
		{
			LensFile = NewObject<ULensFile>(InParent, InName, Flags | RF_Transactional);
			LensFileExchange.PopulateLensFile(*LensFile);

			if (LensFile->AssetImportData != nullptr)
			{
				LensFile->AssetImportData->Update(GetCurrentFilename());
			}
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, LensFile);

	return LensFile;
}

bool ULensFileImporterFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (ULensFile* LensFile = Cast<ULensFile>(Obj))
	{
		if (LensFile->AssetImportData != nullptr)
		{
			LensFile->AssetImportData->ExtractFilenames(OutFilenames);
		}
		else
		{
			OutFilenames.Add(FString{});
		}

		return true;
	}
	else
	{
		return false;
	}
}

void ULensFileImporterFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (NewReimportPaths.IsEmpty())
	{
		return;
	}

	if (ULensFile* LensFile = Cast<ULensFile>(Obj))
	{
		if (FactoryCanImport(NewReimportPaths[0]))
		{
			LensFile->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
		}
	}
}

EReimportResult::Type ULensFileImporterFactory::Reimport(UObject* Obj)
{
	if (ULensFile* LensFile = Cast<ULensFile>(Obj))
	{
		// Make sure the file is valid and exists
		const FString Filename = LensFile->AssetImportData->GetFirstFilename();
		if (Filename.Len() == 0 || IFileManager::Get().FileSize(*Filename) == INDEX_NONE)
		{
			return EReimportResult::Failed;
		}

		bool bImportCancelled = false;
		if (ImportObject(LensFile->GetClass(), LensFile->GetOuter(), *LensFile->GetName(), RF_Public | RF_Standalone, Filename, nullptr, bImportCancelled))
		{
			LensFile->AssetImportData->Update(Filename);

			if (LensFile->GetOuter())
			{
				LensFile->GetOuter()->MarkPackageDirty();
			}

			LensFile->MarkPackageDirty();

			return EReimportResult::Succeeded;
		}
		else
		{
			if (bImportCancelled)
			{
				return EReimportResult::Cancelled;
			}
			else
			{
				return EReimportResult::Failed;
			}
		}
	}

	return EReimportResult::Failed;
}

#undef LOCTEXT_NAMESPACE
