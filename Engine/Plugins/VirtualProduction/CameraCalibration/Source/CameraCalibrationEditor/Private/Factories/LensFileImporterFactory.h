// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "Factories/Factory.h"

#include "LensFileImporterFactory.generated.h"

UCLASS()
class ULensFileImporterFactory
	: public UFactory
	, public FReimportHandler
{
	GENERATED_BODY()

public:
	ULensFileImporterFactory(const FObjectInitializer& ObjectInitializer);

protected:

	//~ Begin UFactory Interface
	virtual FText GetToolTip() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn) override;
	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;

	virtual TObjectPtr<UObject>* GetFactoryObject() const override
	{
		return &GCMark;
	}

	mutable TObjectPtr<UObject> GCMark{this};
	//~ End FReimiporterHandler Interface
};
