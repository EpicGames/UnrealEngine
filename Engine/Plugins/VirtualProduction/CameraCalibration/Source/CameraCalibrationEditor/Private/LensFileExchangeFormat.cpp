// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensFileExchangeFormat.h"

#include "CameraCalibrationEditorLog.h"
#include "JsonObjectConverter.h"
#include "LensData.h"
#include "LensFile.h"
#include "Misc/FeedbackContext.h"
#include "Models/LensModel.h"
#include "Models/SphericalLensModel.h"
#include "OpenCVHelper.h"

#define LOCTEXT_NAMESPACE "LensFileExchange"

FLensInfoExchange::FLensInfoExchange(const ULensFile* LensFile)
{
	if (LensFile == nullptr)
	{
		return;
	}

	const FLensInfo& LensInfo = LensFile->LensInfo;

	SerialNumber = *LensInfo.LensSerialNumber;
	ModelName = *LensInfo.LensModelName;

	if (const ULensModel* LensModelObject = LensInfo.LensModel->GetDefaultObject<ULensModel>())
	{
		DistortionModel = LensModelObject->GetShortModelName();
	}
}

FLensFileMetadata::FLensFileMetadata(const ULensFile* LensFile)
	: LensInfo{ LensFile }
{
	Type = FLensFileExchange::LensFileType;
	Version = FLensFileExchange::LensFileVersion;

	if (LensFile == nullptr)
	{
		return;
	}

	// Set the default values for metadata when exporting from the ULensFile
	Name = *LensFile->GetName();

	// Add the UserMetadata
	UserMetadata.Reserve(LensFile->UserMetadata.Num());
	for (const TPair<FString, FString>& UserMetadataPair : LensFile->UserMetadata)
	{
		FLensFileUserMetadataEntry UserMetadataEntry;
		UserMetadataEntry.Name = *UserMetadataPair.Key;
		UserMetadataEntry.Value = *UserMetadataPair.Value;
		UserMetadata.Add(UserMetadataEntry);
	}
}

FLensFileParameterTableRow::FLensFileParameterTableRow(int32 NumHeaders)
{
	Values.Reserve(NumHeaders);
}

bool FLensFileParameterTableImporter::ConvertToParameterTable(FLensFileParameterTable& OutParamterTable, FFeedbackContext* InWarn) const
{
	OutParamterTable.ParameterName = ParameterName;

	// Parse the header string
	TArray<FString> HeaderTokens;
	Header.ParseIntoArrayWS(HeaderTokens, TEXT(","));

	const int32 NumHeaders = HeaderTokens.Num();

	OutParamterTable.Header.Reserve(NumHeaders);
	for (const FString& HeaderToken : HeaderTokens)
	{
		OutParamterTable.Header.Add(*HeaderToken);
	}

	// Parse the data string
	TArray<FString> DataRowTokens;
	Data.ParseIntoArray(DataRowTokens, TEXT(";"), 1);

	// Reserve memory to fill in the Data array of the parameter table
	OutParamterTable.Data.Reserve(NumHeaders * DataRowTokens.Num());

	// Split each row and convert values to floats
	// TODO: Need to handle other data types
	for (const FString& DataRowToken : DataRowTokens)
	{
		TArray<FString> DataTokens;
		DataRowToken.ParseIntoArrayWS(DataTokens, TEXT(","));

		FLensFileParameterTableRow DataRow{ NumHeaders };

		for (const FString& DataToken : DataTokens)
		{
			float DataValue = 0.0f;
			if (LexTryParseString(DataValue, *DataToken))
			{
				DataRow.Values.Add(DataValue);
			}
			else
			{
				InWarn->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
										ELogVerbosity::Error,
										TEXT("Error converting '%s' value in table '%s'"),
										*DataToken, *ParameterName.ToString());
			}
		}

		OutParamterTable.Data.Add(DataRow);
	}

	return true;
}

FLensFileExchange::FLensFileExchange(const ULensFile* LensFile)
	: Metadata{ LensFile }
	, FeedbackContext{ nullptr }
{
	if (LensFile == nullptr)
	{
		return;
	}

	const FLensInfo& LensInfo = LensFile->LensInfo;
	const FVector2D& Sensor = LensInfo.SensorDimensions;

	SensorDimensions.Width = Sensor.X;
	SensorDimensions.Height = Sensor.Y;
	SensorDimensions.Units = ELensFileUnit::Millimeters;

	ImageDimensions.Width = LensInfo.ImageDimensions.X;
	ImageDimensions.Height = LensInfo.ImageDimensions.Y;

	ExtractFocalLengthTable(LensFile);
	ExtractImageCenterTable(LensFile);
	ExtractNodalOffsetTable(LensFile);
	ExtractEncoderTables(LensFile);
	ExtractDistortionParameters(LensFile);
	ExtractSTMaps(LensFile);
}

FLensFileExchange::FLensFileExchange(const TSharedRef<FJsonObject>& InLensFileJson, bool& bOutIsValidJson, FFeedbackContext* InWarn)
	: FeedbackContext{ InWarn }
{
	// Start by assuming the Json is valid
	bOutIsValidJson = true;

	const int64 CheckFlags = 0;
	const int64 SkipFlags = 0;
	const bool bStrictMode = true;

	const TSharedPtr<FJsonObject>* MetadataJsonObject;
	const FString MetadataJsonFieldName = FJsonObjectConverter::StandardizeCase(GET_MEMBER_NAME_CHECKED(FLensFileExchange, Metadata).ToString());
	if (InLensFileJson->TryGetObjectField(*MetadataJsonFieldName, MetadataJsonObject))
	{
		if (FJsonObjectConverter::JsonObjectToUStruct(MetadataJsonObject->ToSharedRef(), &Metadata, CheckFlags, SkipFlags, bStrictMode))
		{
			if (Metadata.Type != FLensFileExchange::LensFileType)
			{
				GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
													  ELogVerbosity::Error,
													  TEXT("Invalid value for field 'type'. Expected '%s' but got '%s'"),
													  FLensFileExchange::LensFileType, *Metadata.Type.ToString());
				bOutIsValidJson = false;
			}

			if (Metadata.Version != FLensFileExchange::LensFileVersion)
			{
				// TODO: Add proper version validation, a string comparison is enough for now
				GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
													  ELogVerbosity::Error,
													  TEXT("Invalid value for field 'version'. Expected '%s' but got '%s'"),
													  FLensFileExchange::LensFileVersion, *Metadata.Type.ToString());
				bOutIsValidJson = false;
			}
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Error,
												  TEXT("Error parsing '%s' field"),
												  *MetadataJsonFieldName);
			bOutIsValidJson = false;
		}
	}
	else
	{
		GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
											  ELogVerbosity::Error, TEXT("Field '%s' not found"),
											  *MetadataJsonFieldName);
		bOutIsValidJson = false;
	}

	if (!bOutIsValidJson)
	{
		return;
	}

	const TSharedPtr<FJsonObject>* SensorDimensionsJsonObject;
	const FString SensorDimensionsJsonFieldName = FJsonObjectConverter::StandardizeCase(GET_MEMBER_NAME_CHECKED(FLensFileExchange, SensorDimensions).ToString());
	if (InLensFileJson->TryGetObjectField(SensorDimensionsJsonFieldName, SensorDimensionsJsonObject))
	{
		if (FJsonObjectConverter::JsonObjectToUStruct(SensorDimensionsJsonObject->ToSharedRef(), &SensorDimensions, CheckFlags, SkipFlags, bStrictMode))
		{
			if (SensorDimensions.Units != ELensFileUnit::Millimeters)
			{
				GetFeedbackContext()->Log(LogCameraCalibrationEditor.GetCategoryName(), ELogVerbosity::Warning, TEXT("Invalid unit for sensor dimensions"));
			}
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Warning,
												  TEXT("Error parsing '%s' field"),
												  *SensorDimensionsJsonFieldName);
		}
	}
	else
	{
		GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
											  ELogVerbosity::Error,
											  TEXT("Field '%s' not found"),
											  *SensorDimensionsJsonFieldName);
		bOutIsValidJson = false;
	}

	if (!bOutIsValidJson)
	{
		return;
	}

	const TSharedPtr<FJsonObject>* ImageDimensionsJsonObject;
	const FString ImageDimensionsJsonFieldName = FJsonObjectConverter::StandardizeCase(GET_MEMBER_NAME_CHECKED(FLensFileExchange, ImageDimensions).ToString());
	if (InLensFileJson->TryGetObjectField(ImageDimensionsJsonFieldName, ImageDimensionsJsonObject))
	{
		if (!FJsonObjectConverter::JsonObjectToUStruct(ImageDimensionsJsonObject->ToSharedRef(), &ImageDimensions, CheckFlags, SkipFlags, bStrictMode))
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Error,
												  TEXT("Error parsing '%s' field"),
												  *ImageDimensionsJsonFieldName);
			bOutIsValidJson = false;
		}
	}
	else
	{
		GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(), ELogVerbosity::Error, TEXT("Field '%s' not found"), *ImageDimensionsJsonFieldName);
		bOutIsValidJson = false;
	}

	if (!bOutIsValidJson)
	{
		return;
	}

	const TArray<TSharedPtr<FJsonValue>>* CameraParameterTablesValueArray;
	const FString CameraParameterTablesJsonFieldName = FJsonObjectConverter::StandardizeCase(GET_MEMBER_NAME_CHECKED(FLensFileExchange, CameraParameterTables).ToString());
	if (InLensFileJson->TryGetArrayField(CameraParameterTablesJsonFieldName, CameraParameterTablesValueArray))
	{
		CameraParameterTables.Reserve(CameraParameterTablesValueArray->Num());

		for (const TSharedPtr<FJsonValue>& CameraParameterTableObjectValue : *CameraParameterTablesValueArray)
		{
			const TSharedPtr<FJsonObject>* ParameterTableJsonObject;
			if (CameraParameterTableObjectValue->TryGetObject(ParameterTableJsonObject))
			{
				FLensFileParameterTableImporter ParameterTableImporter;
				if (FJsonObjectConverter::JsonObjectToUStruct(ParameterTableJsonObject->ToSharedRef(), &ParameterTableImporter, CheckFlags, SkipFlags, bStrictMode))
				{
					FLensFileParameterTable CameraParameterTable;
					ParameterTableImporter.ConvertToParameterTable(CameraParameterTable, GetFeedbackContext());
					CameraParameterTables.Add(CameraParameterTable);
				}
				else
				{
					GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
														  ELogVerbosity::Warning,
														  TEXT("Error parsing array entry in '%s' field"),
														  *CameraParameterTablesJsonFieldName);
				}
			}
			else
			{
				GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
													  ELogVerbosity::Warning,
													  TEXT("Entries in the '%s' array should be objects"),
													  *CameraParameterTablesJsonFieldName);
			}
		}
	}
	else
	{
		GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
											  ELogVerbosity::Warning,
											  TEXT("Array field '%s' not found"),
											  *CameraParameterTablesJsonFieldName);
	}

	const TArray<TSharedPtr<FJsonValue>>* EncoderTablesValueArray;
	const FString EncoderTablesJsonFieldName = FJsonObjectConverter::StandardizeCase(GET_MEMBER_NAME_CHECKED(FLensFileExchange, EncoderTables).ToString());
	if (InLensFileJson->TryGetArrayField(EncoderTablesJsonFieldName, EncoderTablesValueArray))
	{
		EncoderTables.Reserve(EncoderTablesValueArray->Num());

		for (const TSharedPtr<FJsonValue>& EncoderTableObjectValue : *EncoderTablesValueArray)
		{
			const TSharedPtr<FJsonObject>* EncoderTableJsonObject;
			if (EncoderTableObjectValue->TryGetObject(EncoderTableJsonObject))
			{
				FLensFileParameterTableImporter EncoderTableImporter;
				if (FJsonObjectConverter::JsonObjectToUStruct(EncoderTableJsonObject->ToSharedRef(), &EncoderTableImporter, CheckFlags, SkipFlags, bStrictMode))
				{
					FLensFileParameterTable EncoderTable;
					EncoderTableImporter.ConvertToParameterTable(EncoderTable, GetFeedbackContext());
					EncoderTables.Add(EncoderTable);
				}
				else
				{
					GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
														  ELogVerbosity::Warning,
														  TEXT("Error parsing array entry in '%s' field"),
														  *EncoderTablesJsonFieldName);
				}
			}
			else
			{
				GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
													  ELogVerbosity::Warning,
													  TEXT("Entries in the '%s' array should be objects"),
													  *EncoderTablesJsonFieldName);
			}
		}
	}
	else
	{
		GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
											  ELogVerbosity::Warning,
											  TEXT("Array field '%s' not found"),
											  *EncoderTablesJsonFieldName);
	}
}

bool FLensFileExchange::PopulateLensFile(ULensFile& OutLensFile) const
{
	FLensInfo& LensInfo = OutLensFile.LensInfo;

	const USphericalLensModel* SphericalLensModel = USphericalLensModel::StaticClass()->GetDefaultObject<USphericalLensModel>();
	const bool bIsSphericalLensModel = Metadata.LensInfo.DistortionModel == SphericalLensModel->GetShortModelName();
	if (bIsSphericalLensModel)
	{
		LensInfo.LensModel = USphericalLensModel::StaticClass();
	}
	else
	{
		GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
											  ELogVerbosity::Error,
											  TEXT("Only %s is supported at the moment"),
											  *SphericalLensModel->GetShortModelName().ToString());
		return false;
	}

	LensInfo.LensModelName = Metadata.LensInfo.ModelName.ToString();
	LensInfo.LensSerialNumber = Metadata.LensInfo.SerialNumber.ToString();

	// TODO: Handle unit conversions, assuming units are stored in millimeters
	LensInfo.SensorDimensions.X = SensorDimensions.Width;
	LensInfo.SensorDimensions.Y = SensorDimensions.Height;

	LensInfo.ImageDimensions.X = ImageDimensions.Width;
	LensInfo.ImageDimensions.Y = ImageDimensions.Height;

	for (const FLensFileParameterTable& CameraParameterTable : CameraParameterTables)
	{
		if (CameraParameterTable.ParameterName == GET_MEMBER_NAME_CHECKED(ULensFile, FocalLengthTable))
		{
			PopulateFocalLengthTable(OutLensFile, CameraParameterTable);
		}
		else if (CameraParameterTable.ParameterName == GET_MEMBER_NAME_CHECKED(ULensFile, ImageCenterTable))
		{
			PopulateImageCenterTable(OutLensFile, CameraParameterTable);
		}
		else if (CameraParameterTable.ParameterName == GET_MEMBER_NAME_CHECKED(ULensFile, DistortionTable))
		{
			PopulateDistortionTable(OutLensFile, CameraParameterTable);
		}
		else if (CameraParameterTable.ParameterName == GET_MEMBER_NAME_CHECKED(ULensFile, NodalOffsetTable))
		{
			PopulateNodalOffsetTable(OutLensFile, CameraParameterTable);
		}
	}

	for (const FLensFileParameterTable& EncoderTable : EncoderTables)
	{
		if (EncoderTable.ParameterName == GET_MEMBER_NAME_CHECKED(FEncodersTable, Focus))
		{
			PopulateEncoderTable(OutLensFile.EncodersTable.Focus, EncoderTable, FocusEncoderHeaderName, FocusCMHeaderName);
		}
		else if (EncoderTable.ParameterName == GET_MEMBER_NAME_CHECKED(FEncodersTable, Iris))
		{
			PopulateEncoderTable(OutLensFile.EncodersTable.Iris, EncoderTable, IrisEncoderHeaderName, IrisFstopHeaderName);
		}
	}

	return true;
}

void FLensFileExchange::ExtractFocalLengthTable(const ULensFile* LensFile)
{
	// Read the FocalLength table
	FLensFileParameterTable FocalLengthParametersTable;
	FocalLengthParametersTable.ParameterName = GET_MEMBER_NAME_CHECKED(ULensFile, FocalLengthTable);
	FocalLengthParametersTable.Header = { FocusEncoderHeaderName, ZoomEncoderHeaderName, FocalLengthFxHeaderName, FocalLengthFyHeaderName };

	const FFocalLengthTable& LensFileFocalLengthTable = LensFile->FocalLengthTable;

	for (int32 FocusPointIndex = 0; FocusPointIndex < LensFileFocalLengthTable.GetFocusPointNum(); ++FocusPointIndex)
	{
		const FFocalLengthFocusPoint& FocalLengthFocusPoint = LensFileFocalLengthTable.FocusPoints[FocusPointIndex];
		const float Focus = FocalLengthFocusPoint.GetFocus();

		for (int32 FocalLengthInfoIndex = 0; FocalLengthInfoIndex < FocalLengthFocusPoint.GetNumPoints(); ++FocalLengthInfoIndex)
		{
			FFocalLengthInfo FocalLengthInfo;
			if (FocalLengthFocusPoint.GetValue(FocalLengthInfoIndex, FocalLengthInfo))
			{
				const float Zoom = FocalLengthFocusPoint.GetZoom(FocalLengthInfoIndex);
				const float Fx = FocalLengthInfo.FxFy.X;
				const float Fy = FocalLengthInfo.FxFy.Y;

				FLensFileParameterTableRow DataRow{ FocalLengthParametersTable.Header.Num() };

				DataRow.Values.Add(Focus);
				DataRow.Values.Add(Zoom);
				DataRow.Values.Add(Fx);
				DataRow.Values.Add(Fy);

				FocalLengthParametersTable.Data.Add(DataRow);
			}
		}
	}

	CameraParameterTables.Add(FocalLengthParametersTable);
}

void FLensFileExchange::ExtractImageCenterTable(const ULensFile* LensFile)
{
	// Read the Image Center parameter table
	FLensFileParameterTable ImageCenterParametersTable;
	ImageCenterParametersTable.ParameterName = GET_MEMBER_NAME_CHECKED(ULensFile, ImageCenterTable);
	ImageCenterParametersTable.Header = { FocusEncoderHeaderName, ZoomEncoderHeaderName, ImageCenterCxHeaderName, ImageCenterCyHeaderName };

	const FImageCenterTable& LensFileImageCenterTable = LensFile->ImageCenterTable;

	for (int32 ImageCenterFocusPointIndex = 0; ImageCenterFocusPointIndex < LensFileImageCenterTable.GetFocusPointNum(); ++ImageCenterFocusPointIndex)
	{
		const FImageCenterFocusPoint& ImageCenterFocusPoint = LensFileImageCenterTable.FocusPoints[ImageCenterFocusPointIndex];
		const float Focus = ImageCenterFocusPoint.GetFocus();

		for (int32 ImageCenterInfoIndex = 0; ImageCenterInfoIndex < ImageCenterFocusPoint.GetNumPoints(); ++ImageCenterInfoIndex)
		{
			const float Zoom = ImageCenterFocusPoint.GetZoom(ImageCenterInfoIndex);

			FImageCenterInfo ImageCenterInfo;
			if (ImageCenterFocusPoint.GetPoint(Zoom, ImageCenterInfo))
			{
				const float Cx = ImageCenterInfo.PrincipalPoint.X;
				const float Cy = ImageCenterInfo.PrincipalPoint.Y;

				FLensFileParameterTableRow DataRow{ ImageCenterParametersTable.Header.Num() };

				DataRow.Values.Add(Focus);
				DataRow.Values.Add(Zoom);
				DataRow.Values.Add(Cx);
				DataRow.Values.Add(Cy);

				ImageCenterParametersTable.Data.Add(DataRow);
			}
		}
	}

	CameraParameterTables.Add(ImageCenterParametersTable);
}

void FLensFileExchange::ExtractNodalOffsetTable(const ULensFile* LensFile)
{
	// Read the NodalOffset parameter table
	FLensFileParameterTable NodalOffsetParametersTable;
	NodalOffsetParametersTable.ParameterName = GET_MEMBER_NAME_CHECKED(ULensFile, NodalOffsetTable);
	NodalOffsetParametersTable.Header = {
		FocusEncoderHeaderName,
		ZoomEncoderHeaderName,
		NodalOffsetQxHeaderName,
		NodalOffsetQyHeaderName,
		NodalOffsetQzHeaderName,
		NodalOffsetQwHeaderName,
		NodalOffsetTxHeaderName,
		NodalOffsetTyHeaderName,
		NodalOffsetTzHeaderName
	};

	const FNodalOffsetTable& LensFileNodalOffsetTable = LensFile->NodalOffsetTable;

	for (int32 NodalOffsetFocusPointIndex = 0; NodalOffsetFocusPointIndex < LensFileNodalOffsetTable.GetFocusPointNum(); ++NodalOffsetFocusPointIndex)
	{
		const FNodalOffsetFocusPoint& NodalOffsetFocusPoint = LensFileNodalOffsetTable.FocusPoints[NodalOffsetFocusPointIndex];
		const float Focus = NodalOffsetFocusPoint.GetFocus();

		for (int32 NodalOffsetPointIndex = 0; NodalOffsetPointIndex < NodalOffsetFocusPoint.GetNumPoints(); ++NodalOffsetPointIndex)
		{
			const float Zoom = NodalOffsetFocusPoint.GetZoom(NodalOffsetPointIndex);

			FNodalPointOffset NodalOffsetInfo;
			if (NodalOffsetFocusPoint.GetPoint(Zoom, NodalOffsetInfo))
			{
				FTransform NodalOffsetTransform{ NodalOffsetInfo.RotationOffset, NodalOffsetInfo.LocationOffset };
				FOpenCVHelper::ConvertUnrealToOpenCV(NodalOffsetTransform);

				const FQuat RotationOffset = NodalOffsetTransform.GetRotation();
				const FVector LocationOffset = NodalOffsetTransform.GetTranslation();

				FLensFileParameterTableRow DataRow{ NodalOffsetParametersTable.Header.Num() };

				DataRow.Values.Add(Focus);
				DataRow.Values.Add(Zoom);
				DataRow.Values.Add(RotationOffset.X);
				DataRow.Values.Add(RotationOffset.Y);
				DataRow.Values.Add(RotationOffset.Z);
				DataRow.Values.Add(RotationOffset.W);
				DataRow.Values.Add(LocationOffset.X);
				DataRow.Values.Add(LocationOffset.Y);
				DataRow.Values.Add(LocationOffset.Z);

				NodalOffsetParametersTable.Data.Add(DataRow);
			}
		}
	}

	CameraParameterTables.Add(NodalOffsetParametersTable);
}

void FLensFileExchange::ExtractEncoderTables(const ULensFile* LensFile)
{
	// Read the Encoder Tables from the LensFile
	FLensFileParameterTable FocusEncordersTable;
	FLensFileParameterTable IrisEncodersTable;

	FocusEncordersTable.ParameterName = GET_MEMBER_NAME_CHECKED(FEncodersTable, Focus);
	FocusEncordersTable.Header = { FocusEncoderHeaderName, FocusCMHeaderName };

	IrisEncodersTable.ParameterName = GET_MEMBER_NAME_CHECKED(FEncodersTable, Iris);
	IrisEncodersTable.Header = { IrisEncoderHeaderName, IrisFstopHeaderName };

	const FEncodersTable& LensFileEncodersTable = LensFile->EncodersTable;

	FocusEncordersTable.Data.Reserve(LensFileEncodersTable.GetNumFocusPoints() * 2);
	IrisEncodersTable.Data.Reserve(LensFileEncodersTable.GetNumIrisPoints() * 2);

	for (int32 FocusPointIndex = 0; FocusPointIndex < LensFileEncodersTable.GetNumFocusPoints(); ++FocusPointIndex)
	{
		const float FocusInput = LensFileEncodersTable.GetFocusInput(FocusPointIndex);
		const float FocusValue = LensFileEncodersTable.GetFocusValue(FocusPointIndex);

		FLensFileParameterTableRow DataRow{ FocusEncordersTable.Header.Num() };

		DataRow.Values.Add(FocusInput);
		DataRow.Values.Add(FocusValue);

		FocusEncordersTable.Data.Add(DataRow);
	}

	for (int32 IrisPointIndex = 0; IrisPointIndex < LensFileEncodersTable.GetNumIrisPoints(); ++IrisPointIndex)
	{
		const float IrisInput = LensFileEncodersTable.GetIrisInput(IrisPointIndex);
		const float IrisValue = LensFileEncodersTable.GetIrisValue(IrisPointIndex);

		FLensFileParameterTableRow DataRow{ IrisEncodersTable.Header.Num() };

		DataRow.Values.Add(IrisInput);
		DataRow.Values.Add(IrisValue);

		IrisEncodersTable.Data.Add(DataRow);
	}

	EncoderTables.Add(FocusEncordersTable);
	EncoderTables.Add(IrisEncodersTable);
}

void FLensFileExchange::ExtractDistortionParameters(const ULensFile* LensFile)
{
	// Read the Distortion parameter table
	FLensFileParameterTable DistortionParametersTable;
	DistortionParametersTable.ParameterName = GET_MEMBER_NAME_CHECKED(ULensFile, DistortionTable);

	if (const ULensModel* LensModelObject = LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>())
	{
		const int32 NumDistortionParams = LensModelObject->GetNumParameters();
		DistortionParametersTable.Header.Reserve(NumDistortionParams + 2);

		// Add the Focus and Zoom encoders headers that are common to all camera parameters tables
		DistortionParametersTable.Header.Add(FocusEncoderHeaderName);
		DistortionParametersTable.Header.Add(ZoomEncoderHeaderName);

		for (const FText& ParamName : LensModelObject->GetParameterDisplayNames())
		{
			DistortionParametersTable.Header.Add(*ParamName.ToString());
		}

		const FDistortionTable& LensFileDistortionTable = LensFile->DistortionTable;
		DistortionParametersTable.Data.Reserve(LensFileDistortionTable.GetFocusPointNum());

		for (int32 DistortionFocusPointIndex = 0; DistortionFocusPointIndex < LensFileDistortionTable.GetFocusPointNum(); ++DistortionFocusPointIndex)
		{
			const FDistortionFocusPoint& DistortionFocusPoint = LensFileDistortionTable.FocusPoints[DistortionFocusPointIndex];
			const float Focus = DistortionFocusPoint.GetFocus();

			for (int32 DistortionPointIndex = 0; DistortionPointIndex < DistortionFocusPoint.GetNumPoints(); ++DistortionPointIndex)
			{
				const float Zoom = DistortionFocusPoint.GetZoom(DistortionPointIndex);

				FDistortionInfo DistortionInfo;
				if (DistortionFocusPoint.GetPoint(Zoom, DistortionInfo))
				{
					if (DistortionInfo.Parameters.Num() == NumDistortionParams)
					{
						FLensFileParameterTableRow DataRow{ DistortionParametersTable.Header.Num() };

						// Copy the parameters to the exchange struct
						DataRow.Values.Add(Focus);
						DataRow.Values.Add(Zoom);

						for (const float ParamValue : DistortionInfo.Parameters)
						{
							DataRow.Values.Add(ParamValue);
						}

						DistortionParametersTable.Data.Add(DataRow);
					}
					else
					{
						UE_LOGF(LogCameraCalibrationEditor, Warning, "Expected %d parameters for zoom %f but only %d are available.", NumDistortionParams, Zoom, DistortionInfo.Parameters.Num());
					}
				}
			}
		}
	}

	CameraParameterTables.Add(DistortionParametersTable);
}

void FLensFileExchange::ExtractSTMaps(const ULensFile* LensFile)
{
	if (LensFile->STMapTable.GetFocusPointNum() > 0)
	{
		// TODO: Extract the STMaps from the LensFile to create a new FLensFileParameterTable
		UE_LOGF(LogCameraCalibrationEditor, Warning, "STMaps are not yet supported when exporting a LensFile");
	}
}

void FLensFileExchange::PopulateFocalLengthTable(ULensFile& OutLensFile, const FLensFileParameterTable& InFocalLengthTable) const
{
	const int32 NumHeaders = InFocalLengthTable.Header.Num();

	for (int32 DataRowIndex = 0; DataRowIndex < InFocalLengthTable.Data.Num(); ++DataRowIndex)
	{
		const FLensFileParameterTableRow& DataRow = InFocalLengthTable.Data[DataRowIndex];
		const int32 NumDataValues = DataRow.Values.Num();

		if (NumDataValues == NumHeaders)
		{
			float Focus = 0.0f;
			float Zoom = 0.0f;
			FVector2D FxFy = FVector2D::ZeroVector;

			for (int32 HeaderIndex = 0; HeaderIndex < NumHeaders; ++HeaderIndex)
			{
				const FName& HeaderName = InFocalLengthTable.Header[HeaderIndex];
				const float Value = DataRow.Values[HeaderIndex];

				if (HeaderName == FocusEncoderHeaderName)
				{
					Focus = Value;
				}
				else if (HeaderName == ZoomEncoderHeaderName)
				{
					Zoom = Value;
				}
				else if (HeaderName == FocalLengthFxHeaderName)
				{
					FxFy.X = Value;
				}
				else if (HeaderName == FocalLengthFyHeaderName)
				{
					FxFy.Y = Value;
				}
			}

			OutLensFile.AddFocalLengthPoint(Focus, Zoom, FFocalLengthInfo{ FxFy });
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Warning,
												  TEXT("Number of values in row %d of table '%s' doesn't match with the number of headers. Expected %d values but got %d. Skipping line."),
												  DataRowIndex, *InFocalLengthTable.ParameterName.ToString(), NumHeaders, NumDataValues);
		}
	}
}

void FLensFileExchange::PopulateImageCenterTable(ULensFile& OutLensFile, const FLensFileParameterTable& InImageCenterTable) const
{
	const int32 NumHeaders = InImageCenterTable.Header.Num();

	for (int32 DataRowIndex = 0; DataRowIndex < InImageCenterTable.Data.Num(); ++DataRowIndex)
	{
		const FLensFileParameterTableRow& DataRow = InImageCenterTable.Data[DataRowIndex];
		const int32 NumDataValues = DataRow.Values.Num();

		if (NumDataValues == NumHeaders)
		{
			float Focus = 0.0f;
			float Zoom = 0.0f;
			FVector2D FxFy = FVector2D::ZeroVector;

			for (int32 HeaderIndex = 0; HeaderIndex < NumHeaders; ++HeaderIndex)
			{
				const FName& HeaderName = InImageCenterTable.Header[HeaderIndex];
				const float Value = DataRow.Values[HeaderIndex];

				if (HeaderName == FocusEncoderHeaderName)
				{
					Focus = Value;
				}
				else if (HeaderName == ZoomEncoderHeaderName)
				{
					Zoom = Value;
				}
				else if (HeaderName == ImageCenterCxHeaderName)
				{
					FxFy.X = Value;
				}
				else if (HeaderName == ImageCenterCyHeaderName)
				{
					FxFy.Y = Value;
				}
			}

			OutLensFile.AddImageCenterPoint(Focus, Zoom, FImageCenterInfo{ FxFy });
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Warning,
												  TEXT("Number of values in row %d of table '%s' doesn't match with the number of headers. Expected %d values but got %d. Skipping line."),
												  DataRowIndex, *InImageCenterTable.ParameterName.ToString(), NumHeaders, NumDataValues);
		}
	}
}

void FLensFileExchange::PopulateNodalOffsetTable(ULensFile& OutLensFile, const FLensFileParameterTable& InNodalOffsetTable) const
{
	const int32 NumHeaders = InNodalOffsetTable.Header.Num();

	for (int32 DataRowIndex = 0; DataRowIndex < InNodalOffsetTable.Data.Num(); ++DataRowIndex)
	{
		const FLensFileParameterTableRow& DataRow = InNodalOffsetTable.Data[DataRowIndex];
		const int32 NumDataValues = DataRow.Values.Num();

		if (NumDataValues == NumHeaders)
		{
			float Focus = 0.0f;
			float Zoom = 0.0f;
			FQuat RotationOffset;
			FVector LocationOffset;

			for (int32 HeaderIndex = 0; HeaderIndex < NumHeaders; ++HeaderIndex)
			{
				const FName& HeaderName = InNodalOffsetTable.Header[HeaderIndex];
				const float Value = DataRow.Values[HeaderIndex];

				if (HeaderName == FocusEncoderHeaderName)
				{
					Focus = Value;
				}
				else if (HeaderName == ZoomEncoderHeaderName)
				{
					Zoom = Value;
				}
				else if (HeaderName == NodalOffsetQxHeaderName)
				{
					RotationOffset.X = Value;
				}
				else if (HeaderName == NodalOffsetQyHeaderName)
				{
					RotationOffset.Y = Value;
				}
				else if (HeaderName == NodalOffsetQzHeaderName)
				{
					RotationOffset.Z = Value;
				}
				else if (HeaderName == NodalOffsetQwHeaderName)
				{
					RotationOffset.W = Value;
				}
				else if (HeaderName == NodalOffsetTxHeaderName)
				{
					LocationOffset.X = Value;
				}
				else if (HeaderName == NodalOffsetTyHeaderName)
				{
					LocationOffset.Y = Value;
				}
				else if (HeaderName == NodalOffsetTzHeaderName)
				{
					LocationOffset.Z = Value;
				}
			}

			FTransform NodalOffsetTransform{ RotationOffset, LocationOffset };

			if (Metadata.NodalOffsetCoordinateSystem == ENodalOffsetCoordinateSystem::OpenCV)
			{
				FOpenCVHelper::ConvertOpenCVToUnreal(NodalOffsetTransform);
			}

			FNodalPointOffset NodalPointOffset;
			NodalPointOffset.LocationOffset = NodalOffsetTransform.GetLocation();
			NodalPointOffset.RotationOffset = NodalOffsetTransform.GetRotation();
			OutLensFile.AddNodalOffsetPoint(Focus, Zoom, NodalPointOffset);
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Warning,
												  TEXT("Number of values in row %d of table '%s' doesn't match with the number of headers. Expected %d values but got %d. Skipping line."),
												  DataRowIndex, *InNodalOffsetTable.ParameterName.ToString(), NumHeaders, NumDataValues);
		}
	}
}

void FLensFileExchange::PopulateDistortionTable(ULensFile& OutLensFile, const FLensFileParameterTable& InDistortionTable) const
{
	const int32 NumHeaders = InDistortionTable.Header.Num();

	for (int32 DataRowIndex = 0; DataRowIndex < InDistortionTable.Data.Num(); ++DataRowIndex)
	{
		const FLensFileParameterTableRow& DataRow = InDistortionTable.Data[DataRowIndex];
		const int32 NumDataValues = DataRow.Values.Num();

		if (NumDataValues == NumHeaders)
		{
			float Focus = 0.0f;
			float Zoom = 0.0f;
			TArray<float> DistortionParams;

			for (int32 HeaderIndex = 0; HeaderIndex < NumHeaders; ++HeaderIndex)
			{
				const FName& HeaderName = InDistortionTable.Header[HeaderIndex];
				const float Value = DataRow.Values[HeaderIndex];

				if (HeaderName == FocusEncoderHeaderName)
				{
					Focus = Value;
				}
				else if (HeaderName == ZoomEncoderHeaderName)
				{
					Zoom = Value;
				}
				else
				{
					// TODO: Need to add some validation here based on ParameterDisplayNames, however, this function is WITH_EDITOR only
					DistortionParams.Add(Value);
				}
			}

			FDistortionInfo DistortionInfo;
			DistortionInfo.Parameters = DistortionParams;
			OutLensFile.DistortionTable.AddPoint(Focus, Zoom, DistortionInfo, OutLensFile.InputTolerance, false);
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Warning,
												  TEXT("Number of values in row %d of table '%s' doesn't match with the number of headers. Expected %d values but got %d. Skipping line."),
												  DataRowIndex, *InDistortionTable.ParameterName.ToString(), NumHeaders, NumDataValues);
		}
	}
}

void FLensFileExchange::PopulateEncoderTable(FRichCurve& OutCurve, const FLensFileParameterTable& InEncoderTable, const TCHAR* InputHeaderName, const TCHAR* OutputHeaderName) const
{
	const int32 NumHeaders = InEncoderTable.Header.Num();

	for (int32 DataRowIndex = 0; DataRowIndex < InEncoderTable.Data.Num(); ++DataRowIndex)
	{
		const FLensFileParameterTableRow& DataRow = InEncoderTable.Data[DataRowIndex];
		const int32 NumDataValues = DataRow.Values.Num();

		if (NumDataValues == NumHeaders)
		{
			float Input = 0.0f;
			float Output = 0.0f;

			for (int32 HeaderIndex = 0; HeaderIndex < NumHeaders; ++HeaderIndex)
			{
				const FName& HeaderName = InEncoderTable.Header[HeaderIndex];
				const float Value = DataRow.Values[HeaderIndex];

				if (HeaderName == InputHeaderName)
				{
					Input = Value;
				}
				else if (HeaderName == OutputHeaderName)
				{
					Output = Value;
				}
			}

			OutCurve.AddKey(Input, Output);
		}
		else
		{
			GetFeedbackContext()->CategorizedLogf(LogCameraCalibrationEditor.GetCategoryName(),
												  ELogVerbosity::Warning,
												  TEXT("Number of values in row %d of table '%s' doesn't match with the number of headers. Expected %d values but got %d. Skipping line."),
												  DataRowIndex, *InEncoderTable.ParameterName.ToString(), NumHeaders, NumDataValues);
		}
	}
}

FFeedbackContext* FLensFileExchange::GetFeedbackContext() const
{
	return FeedbackContext ? FeedbackContext : GWarn;
}

#undef LOCTEXT_NAMESPACE