// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "LensFileExchangeFormat.generated.h"

UENUM()
enum class ELensFileUnit
{
	Millimeters,
	Normalized
};

UENUM()
enum class ENodalOffsetCoordinateSystem
{
	OpenCV
};

USTRUCT()
struct CAMERACALIBRATIONEDITOR_API FLensInfoExchange
{
	GENERATED_BODY()

	FLensInfoExchange(const class ULensFile* LensFile = nullptr);

	UPROPERTY()
	FName SerialNumber;

	UPROPERTY()
	FName ModelName;

	UPROPERTY()
	FName DistortionModel;
};

USTRUCT()
struct FLensFileUserMetadataEntry
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName Value;
};

USTRUCT()
struct CAMERACALIBRATIONEDITOR_API FLensFileMetadata
{
	GENERATED_BODY()

	FLensFileMetadata(const class ULensFile* LensFile = nullptr);

	UPROPERTY()
	FName Type;

	UPROPERTY()
	FName Version;

	UPROPERTY()
	FLensInfoExchange LensInfo;

	UPROPERTY()
	FName Name;

	UPROPERTY()
	ENodalOffsetCoordinateSystem NodalOffsetCoordinateSystem = ENodalOffsetCoordinateSystem::OpenCV;

	UPROPERTY()
	ELensFileUnit FxFyUnits = ELensFileUnit::Normalized;

	UPROPERTY()
	ELensFileUnit CxCyUnits = ELensFileUnit::Normalized;

	UPROPERTY()
	TArray<FLensFileUserMetadataEntry> UserMetadata;
};


USTRUCT()
struct FLensFileSensorDimensions
{
	GENERATED_BODY()

	UPROPERTY()
	float Width = 0.0f;

	UPROPERTY()
	float Height = 0.0f;

	UPROPERTY()
	ELensFileUnit Units = ELensFileUnit::Normalized;
};

USTRUCT()
struct FLensFileImageDimensions
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Width = 0;

	UPROPERTY()
	int32 Height = 0;
};

USTRUCT()
struct FLensFileParameterTableRow
{
	GENERATED_BODY()

	explicit FLensFileParameterTableRow(int32 InNumHeaders = 0);

	UPROPERTY()
	TArray<float> Values;
};

USTRUCT()
struct FLensFileParameterTable
{
	GENERATED_BODY()

	UPROPERTY()
	FName ParameterName;

	UPROPERTY()
	TArray<FName> Header;

	UPROPERTY()
	TArray<FLensFileParameterTableRow> Data;
};

USTRUCT()
struct FLensFileParameterTableImporter
{
	GENERATED_BODY()

	UPROPERTY()
	FName ParameterName;

	UPROPERTY()
	FString Header;

	UPROPERTY()
	FString Data;

	bool ConvertToParameterTable(struct FLensFileParameterTable& OutParamterTable, class FFeedbackContext* InWarn) const;
};

USTRUCT()
struct CAMERACALIBRATIONEDITOR_API FLensFileExchange
{
	GENERATED_BODY()

	FLensFileExchange(const class ULensFile* LensFile = nullptr);

	FLensFileExchange(const TSharedRef<class FJsonObject>& InLensFileJson, bool& bOutIsValidJson, class FFeedbackContext* InWarn);

	bool PopulateLensFile(class ULensFile& LensFile) const;

	UPROPERTY()
	FLensFileMetadata Metadata;

	UPROPERTY()
	FLensFileSensorDimensions SensorDimensions;

	UPROPERTY()
	FLensFileImageDimensions ImageDimensions;

	UPROPERTY()
	TArray<FLensFileParameterTable> CameraParameterTables;

	UPROPERTY()
	TArray<FLensFileParameterTable> EncoderTables;

	constexpr static TCHAR LensFileType[] = TEXT("LensFile");
	constexpr static TCHAR LensFileVersion[] = TEXT("0.0.0");

	constexpr static TCHAR FocusEncoderHeaderName[] = TEXT("FocusEncoder");
	constexpr static TCHAR ZoomEncoderHeaderName[] = TEXT("ZoomEncoder");
	constexpr static TCHAR IrisEncoderHeaderName[] = TEXT("IrisEncoder");

	constexpr static TCHAR FocusCMHeaderName[] = TEXT("FocusCM");
	constexpr static TCHAR IrisFstopHeaderName[] = TEXT("IrisFstop");

	constexpr static TCHAR FocalLengthFxHeaderName[] = TEXT("Fx");
	constexpr static TCHAR FocalLengthFyHeaderName[] = TEXT("Fy");

	constexpr static TCHAR ImageCenterCxHeaderName[] = TEXT("Cx");
	constexpr static TCHAR ImageCenterCyHeaderName[] = TEXT("Cy");

	constexpr static TCHAR NodalOffsetQxHeaderName[] = TEXT("Qx");
	constexpr static TCHAR NodalOffsetQyHeaderName[] = TEXT("Qy");
	constexpr static TCHAR NodalOffsetQzHeaderName[] = TEXT("Qz");
	constexpr static TCHAR NodalOffsetQwHeaderName[] = TEXT("Qw");

	constexpr static TCHAR NodalOffsetTxHeaderName[] = TEXT("Tx");
	constexpr static TCHAR NodalOffsetTyHeaderName[] = TEXT("Ty");
	constexpr static TCHAR NodalOffsetTzHeaderName[] = TEXT("Tz");

private:
	FFeedbackContext* FeedbackContext;

	FFeedbackContext* GetFeedbackContext() const;

	void ExtractFocalLengthTable(const class ULensFile* LensFile);
	void ExtractImageCenterTable(const class ULensFile* LensFile);
	void ExtractNodalOffsetTable(const class ULensFile* LensFile);
	void ExtractEncoderTables(const class ULensFile* LensFile);
	void ExtractDistortionParameters(const class ULensFile* LensFile);
	void ExtractSTMaps(const class ULensFile* LensFile);

	void PopulateFocalLengthTable(class ULensFile& OutLensFile, const FLensFileParameterTable& FocalLengthTable) const;
	void PopulateImageCenterTable(class ULensFile& OutLensFile, const FLensFileParameterTable& InImageCenterTable) const;
	void PopulateNodalOffsetTable(class ULensFile& OutLensFile, const FLensFileParameterTable& InNodalOffsetTable) const;
	void PopulateDistortionTable(class ULensFile& OutLensFile, const FLensFileParameterTable& InDistortionTable) const;
	void PopulateEncoderTable(struct FRichCurve& OutCurve, const FLensFileParameterTable& InEncoderTable, const TCHAR* InputHeaderName, const TCHAR* OutputHeaderName) const;
};
