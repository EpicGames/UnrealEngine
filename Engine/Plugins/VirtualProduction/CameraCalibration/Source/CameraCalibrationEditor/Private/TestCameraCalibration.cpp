// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestCameraCalibration.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Calibrators/CameraCalibrationSolver.h"
#include "CameraCalibrationTypes.h"
#include "CameraCalibrationUtilsPrivate.h"
#include "Dialog/SCustomDialog.h"
#include "Engine/Texture2D.h"
#include "HAL/FileManager.h"
#include "JsonObjectConverter.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Models/SphericalLensModel.h"
#include "OpenCVHelper.h"
#include "TestCameraCalibrationSettings.h"
#include "UI/SImageTexture.h"

DEFINE_LOG_CATEGORY_STATIC(LogTestCameraCalibration, Log, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTestDistortionSpherical, "Plugins.CameraCalibration.TestDistortionSpherical", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTestNodalOffset, "Plugins.CameraCalibration.TestNodalOffset", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

namespace UE::Private::CameraCalibration::AutomatedTests
{
	namespace Logging
	{
		FString GetSolverFlagString(const TOptional<bool>& Flag)
		{
			if (Flag.IsSet() && Flag.GetValue())
			{
				return TEXT("true");
			}
			return TEXT("false");
		}

		void LogDistortionParameters(const TArray<float>& DistortionParameters)
		{
			if (ensure(DistortionParameters.Num() == 5))
			{
				UE_LOGF(LogTestCameraCalibration, Log, "\t\t(%lf, %lf, %lf, %lf, %lf)",
					DistortionParameters[0], // K1
					DistortionParameters[1], // K2
					DistortionParameters[3], // P1
					DistortionParameters[4], // P2
					DistortionParameters[2]  // K3
				);
			}
		}

		void LogPoses(const TArray<FTransform>& Poses)
		{
			for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
			{
				const FTransform& Pose = Poses[PoseIndex];
				const FVector Location = Pose.GetTranslation();
				const FRotator Rotation = Pose.GetRotation().Rotator();

				UE_LOGF(LogTestCameraCalibration, Log, "Pose %d:", PoseIndex);
				UE_LOGF(LogTestCameraCalibration, Log, "\t\tLocation: (%lf, %lf, %lf)", Location.X, Location.Y, Location.Z);
				UE_LOGF(LogTestCameraCalibration, Log, "\t\tRotation: (%lf, %lf, %lf)", Rotation.Roll, Rotation.Pitch, Rotation.Yaw);
			}
		}

		void LogPoses(const TArray<FLocationRotation>& Poses)
		{
			for (int32 PoseIndex = 0; PoseIndex < Poses.Num(); ++PoseIndex)
			{
				const FLocationRotation& Pose = Poses[PoseIndex];

				UE_LOGF(LogTestCameraCalibration, Log, "Pose %d:", PoseIndex);
				UE_LOGF(LogTestCameraCalibration, Log, "\t\tLocation: (%lf, %lf, %lf)", Pose.Location.X, Pose.Location.Y, Pose.Location.Z);
				UE_LOGF(LogTestCameraCalibration, Log, "\t\tRotation: (%lf, %lf, %lf)", Pose.Rotation.Roll, Pose.Rotation.Pitch, Pose.Rotation.Yaw);
			}
		}

		void LogTestDescription(const FCalibrationTest& Test)
		{
			UE_LOGF(LogTestCameraCalibration, Log, "Test #%d:", Test.TestIndex);

			UE_LOGF(LogTestCameraCalibration, Log, " ");

			if (Test.DatasetPath.IsEmpty())
			{
				UE_LOGF(LogTestCameraCalibration, Log, "Camera Profile:");
				UE_LOGF(LogTestCameraCalibration, Log, "Sensor Dimensions: (%.3lfmm x %.3lfmm)", Test.CameraProfile.SensorSize.X, Test.CameraProfile.SensorSize.Y);
				UE_LOGF(LogTestCameraCalibration, Log, "Image Resolution: (%d, %d)", Test.CameraProfile.ImageSize.X, Test.CameraProfile.ImageSize.Y);

				const FVector2D FxFyInPixels = Test.CameraProfile.GetFxFyInPixels();
				UE_LOGF(LogTestCameraCalibration, Log, "Focal Length: %lf mm (%lf pixels)", Test.CameraProfile.FocalLengthInMM, FxFyInPixels.X);

				UE_LOGF(LogTestCameraCalibration, Log, "Image Center: (%lf, %lf)", Test.CameraProfile.ImageCenter.X, Test.CameraProfile.ImageCenter.Y);

				UE_LOGF(LogTestCameraCalibration, Log, "Distortion Parameters:");
				LogDistortionParameters(Test.CameraProfile.DistortionParameters);

				UE_LOGF(LogTestCameraCalibration, Log, " ");

				UE_LOGF(LogTestCameraCalibration, Log, "Checkerboard Profile:");
				UE_LOGF(LogTestCameraCalibration, Log, "Checkerboard Dimensions: %d columns by %d rows", Test.CheckerboardProfile.CheckerboardDimensions.X, Test.CheckerboardProfile.CheckerboardDimensions.Y);
				UE_LOGF(LogTestCameraCalibration, Log, "Checkerboard Square Size: %.2lfcm", Test.CheckerboardProfile.SquareSize);

				UE_LOGF(LogTestCameraCalibration, Log, " ");

				UE_LOGF(LogTestCameraCalibration, Log, "Camera Poses:");
				LogPoses(Test.CameraPoses);

				UE_LOGF(LogTestCameraCalibration, Log, " ");

				UE_LOGF(LogTestCameraCalibration, Log, "Checkerboard Poses:");
				LogPoses(Test.CheckerboardPoses);
			}
			else
			{
				UE_LOGF(LogTestCameraCalibration, Log, "Using dataset: %ls", *Test.DatasetPath);
			}

			UE_LOGF(LogTestCameraCalibration, Log, " ");

			UE_LOGF(LogTestCameraCalibration, Log, "Solver Settings:");
			UE_LOGF(LogTestCameraCalibration, Log, "Focal Length Guess: %lf mm", Test.SolverSettings.FocalLengthGuess);
			UE_LOGF(LogTestCameraCalibration, Log, "Image Center Guess: (%lf, %lf)", Test.SolverSettings.ImageCenterGuess.X, Test.SolverSettings.ImageCenterGuess.Y);
			UE_LOGF(LogTestCameraCalibration, Log, "Use Intrinsic Guess: %ls", *GetSolverFlagString(Test.SolverSettings.bUseIntrinsicGuess));
			UE_LOGF(LogTestCameraCalibration, Log, "Use Extrinsic Guess: %ls", *GetSolverFlagString(Test.SolverSettings.bUseExtrinsicGuess));
			UE_LOGF(LogTestCameraCalibration, Log, "Fix Focal Length: %ls", *GetSolverFlagString(Test.SolverSettings.bFixFocalLength));
			UE_LOGF(LogTestCameraCalibration, Log, "Fix Image Center: %ls", *GetSolverFlagString(Test.SolverSettings.bFixPrincipalPoint));
			UE_LOGF(LogTestCameraCalibration, Log, "Fix Extrinsics: %ls", *GetSolverFlagString(Test.SolverSettings.bFixExtrinsics));
			UE_LOGF(LogTestCameraCalibration, Log, "Fix Distortion: %ls", *GetSolverFlagString(Test.SolverSettings.bFixDistortion));
			UE_LOGF(LogTestCameraCalibration, Log, "Fix Aspect Ratio: %ls", *GetSolverFlagString(Test.SolverSettings.bFixAspectRatio));
		}

		void LogCalibrationResult(const FDistortionCalibrationResult& Result, double FocalLengthInMM)
		{
			UE_LOGF(LogTestCameraCalibration, Log, " ");
			UE_LOGF(LogTestCameraCalibration, Log, "Calibration Result:");
			UE_LOGF(LogTestCameraCalibration, Log, " ");
			UE_LOGF(LogTestCameraCalibration, Log, "FxFy: (%lf, %lf) pixels (%lf mm)", Result.FocalLength.FxFy.X, Result.FocalLength.FxFy.Y, FocalLengthInMM);
			UE_LOGF(LogTestCameraCalibration, Log, "Image Center: (%lf, %lf)", Result.ImageCenter.PrincipalPoint.X, Result.ImageCenter.PrincipalPoint.Y);

			UE_LOGF(LogTestCameraCalibration, Log, "Distortion Parameters:");
			LogDistortionParameters(Result.Parameters.Parameters);

			UE_LOGF(LogTestCameraCalibration, Log, "Camera Poses:");
			LogPoses(Result.CameraPoses);

			UE_LOGF(LogTestCameraCalibration, Log, "RMSE: %.9lf", Result.ReprojectionError);
		}
	}

	void TestDistortionCalibration(FAutomationTestBase& Test)
	{
		const UTestCameraCalibrationSettings* TestSettings = GetDefault<UTestCameraCalibrationSettings>();

		// Extract the solver settings from the automated test settings
		ECalibrationFlags SolverFlags = ECalibrationFlags::None;
		if (TestSettings->bUseCameraIntrinsicGuess)
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::UseIntrinsicGuess);
		}
		if (TestSettings->bUseCameraExtrinsicGuess)
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::UseExtrinsicGuess);
		}
		if (TestSettings->bFixFocalLength)
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixFocalLength);
		}
		if (TestSettings->bFixImageCenter)
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixPrincipalPoint);
		}
		if (TestSettings->bFixExtrinsics)
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixExtrinsics);
		}
		if (TestSettings->bFixDistortion)
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixDistortion);
		}

		/** 
		 * Step 1: Establish the ground-truth 3D object point and 2D image point data
		 */

		 // Initialize the ground-truth image properties
		const FIntPoint ImageSize = TestSettings->ImageSize;
		const float SensorWidth = TestSettings->SensorDimensions.X;

		// Initialize the ground-truth camera intrinsics
		const double TrueFocalLength = TestSettings->FocalLength;
		const double TrueFocalLengthPixels = (TrueFocalLength / SensorWidth) * ImageSize.X;
		const FVector2D TrueFxFy = FVector2D(TrueFocalLengthPixels, TrueFocalLengthPixels);

		const FVector2D ImageCenter = FVector2D((ImageSize.X - 1) * 0.5, (ImageSize.Y - 1) * 0.5);

		// Initialize the ground-truth camera extrinsics
		const FTransform TrueCameraPose = TestSettings->CameraTransform;

		// Initialize the ground-truth distortion parameters
		const FSphericalDistortionParameters SphericalParams = TestSettings->SphericalDistortionParameters;
		const TArray<float> DistortionParameters = { SphericalParams.K1, SphericalParams.K2, SphericalParams.P1, SphericalParams.P2, SphericalParams.K3 };

		// Initialize the 3D object points. For a distortion calibration, these points represent co-planar points that would be found on a real calibrator at a reasonable distance from the physical camera.
		// In the current implementation, they simulate the corners of a checkerboard with size and dimensions defined in the test settings.
		// In the future, this test could be expanded to support simulating an aruco or charuco board, or other known patterns.
		TArray<TArray<FVector>> ObjectPoints;
		TArray<TArray<FVector2f>> ImagePoints;
		TArray<FTransform> TrueCameraPoses;
		TArray<FTransform> EstimatedCameraPoses;

		TArray<FVector> Points3d;

		// We assume that the checkerboard lies perfectly in the YZ plane (same axes as our image plane)
		// To compute the 3D points, we place the board at a fixed distance from the camera (set by the corresponding test setting).
		// We then place the center of the board at (X, 0, 0), and find the top left intersection of checkerboard squares (not the top left corner of the checkerboard). 
		// The points start at the top left corner and proceed to the right, then start over for the next row. This matches how OpenCV would order the detected corners of a checkerboard.
		const double DistanceFromCamera = TestSettings->CalibratorDistanceFromCamera;
		const double SquareSize = TestSettings->CheckerboardSquareSize;

		// The dimensions defined in the test settings are the number of squares in the board, however we only care about the number of intersections, so we reduce each dimension by 1.
		const FIntPoint CheckerboardCornerDimensions = TestSettings->CheckerboardDimensions - FIntPoint(1, 1);

		// The top left intersection is one square length to the right and down from the actual top left corner of the board
		const double TopLeftCornerX = -(TestSettings->CheckerboardDimensions.X * 0.5 * SquareSize) + SquareSize;
		const double TopLeftCornerY =  (TestSettings->CheckerboardDimensions.Y * 0.5 * SquareSize) - SquareSize;

		for (int32 RowIndex = 0; RowIndex < CheckerboardCornerDimensions.Y; ++RowIndex)
		{
			for (int32 ColumnIndex = 0; ColumnIndex < CheckerboardCornerDimensions.X; ++ColumnIndex)
			{
				const double Y = TopLeftCornerX + (SquareSize * ColumnIndex);
				const double Z = TopLeftCornerY - (SquareSize * RowIndex);
				Points3d.Add(FVector(DistanceFromCamera, Y, Z));
			}
		}

		// In a real calibration, there are likely to be images of the checkerboard taken from multiple camera angles
		// The test settings have a setting for the number of camera views to use.
		// The current strategy is to only rotate the camera a maximum of 30 degrees to the left and to the right, and translate it in space to keep the board in view.
		// The number of views, therefore, determines how far to move the camera to generate each view
		double StartRotation = 0.0;
		double StartTranslation = 0.0;
		double RotationStep = 0.0;
		double TranslationStep = 0.0;
		if (TestSettings->NumCameraViews > 1)
		{
			StartRotation = 30.0;
			StartTranslation = TestSettings->CalibratorDistanceFromCamera * 0.6;

			RotationStep =    (StartRotation    * 2) / (TestSettings->NumCameraViews - 1);
			TranslationStep = (StartTranslation * 2) / (TestSettings->NumCameraViews - 1);
		}

		// For each of the camera views, project the 3D calibrator points to the image plane
		for (int32 ViewIndex = 0; ViewIndex < TestSettings->NumCameraViews; ++ViewIndex)
		{
			const FVector ViewTranslation = FVector(0, StartTranslation - (TranslationStep * ViewIndex), 0);
			const FRotator ViewRotation = FRotator(0, -StartRotation + (RotationStep * ViewIndex), 0);

			FTransform CameraMotion = FTransform::Identity;
			CameraMotion.SetTranslation(ViewTranslation);
			CameraMotion.SetRotation(ViewRotation.Quaternion());

			FTransform CameraPoseForView = CameraMotion * TrueCameraPose;
			FTransform EstimatedCameraPoseForView = CameraMotion * TestSettings->EstimatedCameraTransform;

			TArray<FVector2f> Points2d;
			bool bResult = FOpenCVHelper::ProjectPoints(Points3d, TrueFxFy, ImageCenter, DistortionParameters, CameraPoseForView, Points2d);
			if (!bResult)
			{
				Test.AddError(TEXT("Project Points failed. Test could not be completed"));
				return;
			}

			ObjectPoints.Add(Points3d);
			ImagePoints.Add(Points2d);
			TrueCameraPoses.Add(CameraPoseForView);
			EstimatedCameraPoses.Add(EstimatedCameraPoseForView);
		}

		/**
		 * Step 2: Introduce errors into the 3D and 2D point data to simulate real-world inaccuracies that occur when doing calibration
		 */

		// Introduce some random noise to the 3D points. The checkerboard is a rigid object, so the individual 3D positions of each corner cannot change randomly with respect to one another.
		// However, the entire board could have the wrong pose if, for example, the tracking data is noisy, or if the tracked rigid-body pose sent to UE from the tracking system is not precise.
		TArray<FObjectPoints> NoisyObjectPoints;
		NoisyObjectPoints.Reserve(ObjectPoints.Num());
		for (const TArray<FVector>& Object : ObjectPoints)
		{
			FObjectPoints NoisyPoints;
			NoisyPoints.Points.Reserve(Object.Num());

			const double NoiseScale = TestSettings->ObjectPointNoiseScale;
			const double NoiseX = (FMath::SRand() - 0.5) * (NoiseScale * 2);
			const double NoiseY = (FMath::SRand() - 0.5) * (NoiseScale * 2);
			const double NoiseZ = (FMath::SRand() - 0.5) * (NoiseScale * 2);

			for (const FVector& Point : Object)
			{
				NoisyPoints.Points.Add(Point + FVector(NoiseX, NoiseY, NoiseZ));
			}

			NoisyObjectPoints.Add(NoisyPoints);
		}

		// Introduce some random noise to the 2D points. This simulates poor checkerboard detection, which could occur if the checkerboard is not perfectly in-focus, if the image resolution is too low,
		// or if there is some other imprecision in the corner detection algorithm. 
		TArray<FImagePoints> NoisyImagePoints;
		NoisyImagePoints.Reserve(ImagePoints.Num());
		for (const TArray<FVector2f>& Image : ImagePoints)
		{
			FImagePoints NoisyPoints;
			NoisyPoints.Points.Reserve(Image.Num());

			// Unlike the 3D points, the 2D image points could all be randomly noisy compared to one another
			for (const FVector2f& Point : Image)
			{
				const double NoiseScale = TestSettings->ImagePointNoiseScale;
				const double NoiseX = (FMath::SRand() - 0.5) * (NoiseScale * 2);
				const double NoiseY = (FMath::SRand() - 0.5) * (NoiseScale * 2);

				const FVector2f PointWithNoise = Point + FVector2f(NoiseX, NoiseY);
				NoisyPoints.Points.Add(FVector2D(PointWithNoise.X, PointWithNoise.Y));
			}

			NoisyImagePoints.Add(NoisyPoints);
		}

		/**
		 * Step 3: Run the calibration solver to compute the focal length, image center, distortion parameters, and camera poses for each view.
		 * If no errors were introduced into the data, the expectation is that the solver will be able to compute the ground-truth for all of these properties.
		 * If this is not the case, then we either uncover bugs in the solver, or learn more about the limitations of the solver
		 * The introduction of errors should reveal how real-world calibrations can produce poor results if the quality of the input data is poor.
		 */

		// This is fixed until the tests support testing anamorphic calibration
		constexpr float PixelAspect = 1.0f;

		FVector2D CalibratedFxFy = TrueFxFy;
		if (TestSettings->bUseCameraIntrinsicGuess)
		{
			const double EstimatedFocalLength = TestSettings->EstimatedFocalLength;
			const double EstimatedFocalLengthPixels = (EstimatedFocalLength / SensorWidth) * ImageSize.X;
			CalibratedFxFy = FVector2D(EstimatedFocalLengthPixels, EstimatedFocalLengthPixels);
		}

		FVector2D CalibratedImageCenter = ImageCenter;
		TArray<float> DistortionGuess;

		ULensDistortionSolverOpenCV* TestSolver = NewObject<ULensDistortionSolverOpenCV>();

		TArray<FTransform> TargetPoses;

		FDistortionCalibrationResult Result = TestSolver->Solve(
			NoisyObjectPoints,
			NoisyImagePoints,
			ImageSize,
			CalibratedFxFy,
			CalibratedImageCenter,
			DistortionGuess,
			EstimatedCameraPoses,
			TargetPoses,
			USphericalLensModel::StaticClass(),
			PixelAspect,
			SolverFlags
		);

		/**
		 * Step 4: Output the test results
		 */

		Test.AddInfo(TEXT("Ground-Truth Image Properties:"));
		Test.AddInfo(FString::Printf(TEXT("\t\tImage Dimensions: (%d, %d)"), TestSettings->ImageSize.X, TestSettings->ImageSize.Y));
		Test.AddInfo(FString::Printf(TEXT("\t\tSensor Dimensions: (%f, %f)"), TestSettings->SensorDimensions.X, TestSettings->SensorDimensions.Y));

		Test.AddInfo(TEXT("Ground-Truth Camera Intrinsics:"));
		Test.AddInfo(FString::Printf(TEXT("\t\tFocal Length: %lf mm (%lf pixels)"), TrueFocalLength, TrueFxFy.X));
		Test.AddInfo(FString::Printf(TEXT("\t\tImage Center: (%lf, %lf)"), ImageCenter.X, ImageCenter.Y));

		Test.AddInfo(TEXT("Ground-Truth Distortion Coefficients:"));
		Test.AddInfo(FString::Printf(TEXT("\t\tK1: %f"), SphericalParams.K1));
		Test.AddInfo(FString::Printf(TEXT("\t\tK2: %f"), SphericalParams.K2));
		Test.AddInfo(FString::Printf(TEXT("\t\tK3: %f"), SphericalParams.K3));
		Test.AddInfo(FString::Printf(TEXT("\t\tP1: %f"), SphericalParams.P1));
		Test.AddInfo(FString::Printf(TEXT("\t\tP2: %f"), SphericalParams.P2));

		Test.AddInfo(TEXT("\n"));
 		Test.AddInfo(FString::Printf(TEXT("Result RMS Error: %lf"), Result.ReprojectionError));
		Test.AddInfo(TEXT("\n"));

		Test.AddInfo(TEXT("Calibrated Camera Intrinsics:"));
 		Test.AddInfo(FString::Printf(TEXT("\t\tFocal Length: %lf mm (%lf pixels)"), (Result.FocalLength.FxFy.X / ImageSize.X)* SensorWidth, Result.FocalLength.FxFy.X));
		Test.AddInfo(FString::Printf(TEXT("\t\tImage Center: (%lf, %lf)"), Result.ImageCenter.PrincipalPoint.X, Result.ImageCenter.PrincipalPoint.Y));

 		Test.AddInfo(TEXT("Calibrated Distortion Coefficients:"));
		Test.AddInfo(FString::Printf(TEXT("\t\tK1: %f"), Result.Parameters.Parameters[0]));
		Test.AddInfo(FString::Printf(TEXT("\t\tK2: %f"), Result.Parameters.Parameters[1]));
		Test.AddInfo(FString::Printf(TEXT("\t\tK3: %f"), Result.Parameters.Parameters[2]));
		Test.AddInfo(FString::Printf(TEXT("\t\tP1: %f"), Result.Parameters.Parameters[3]));
		Test.AddInfo(FString::Printf(TEXT("\t\tP2: %f"), Result.Parameters.Parameters[4]));
	}

	void TestNodalOffsetCalibration(FAutomationTestBase& Test)
	{
		const UTestCameraCalibrationSettings* TestSettings = GetDefault<UTestCameraCalibrationSettings>();

		/**
		 * Step 1: Establish the ground-truth 3D object point and 2D image point data
		 */
		
		// Initialize the ground-truth image properties
		const FIntPoint ImageSize = TestSettings->ImageSize;
		const float SensorWidth = TestSettings->SensorDimensions.X;

		// Initialize the ground-truth camera intrinsics
		const double TrueFocalLength = TestSettings->FocalLength;
		const double TrueFocalLengthPixels = (TrueFocalLength / SensorWidth) * ImageSize.X;
		const FVector2D TrueFxFy = FVector2D(TrueFocalLengthPixels, TrueFocalLengthPixels);

		const FVector2D ImageCenter = FVector2D((ImageSize.X - 1) * 0.5, (ImageSize.Y - 1) * 0.5);

		// Initialize the ground-truth camera extrinsics
		const FTransform TrueCameraPose = TestSettings->CameraTransform;

		// Initialize the ground-truth distortion parameters
		const FSphericalDistortionParameters SphericalParams = TestSettings->SphericalDistortionParameters;
		const TArray<float> DistortionParameters = { SphericalParams.K1, SphericalParams.K2, SphericalParams.P1, SphericalParams.P2, SphericalParams.K3 };

		// Initialize the 3D object points. For a distortion calibration, these points represent co-planar points that would be found on a real calibrator at a reasonable distance from the physical camera.
		// In the current implementation, the camera is fixed at the world origin (0, 0, 0).
		// These object points represent an object that is a known distance from the camera and that might come from a calibration object that is 120cm x 60cm (~4ft x ~2ft)
		const double DistanceFromCamera = TestSettings->CalibratorDistanceFromCamera;
		const double SquareSize = TestSettings->CheckerboardSquareSize;

		// The dimensions defined in the test settings are the number of squares in the board, however we only care about the number of intersections, so we reduce each dimension by 1.
		const FIntPoint CheckerboardCornerDimensions = TestSettings->CheckerboardDimensions - FIntPoint(1, 1);

		// The top left intersection is one square length to the right and down from the actual top left corner of the board
		const double TopLeftCornerX = -(TestSettings->CheckerboardDimensions.X * 0.5 * SquareSize) + SquareSize;
		const double TopLeftCornerY = (TestSettings->CheckerboardDimensions.Y * 0.5 * SquareSize) - SquareSize;

		TArray<FVector> ObjectPoints;
		for (int32 RowIndex = 0; RowIndex < CheckerboardCornerDimensions.Y; ++RowIndex)
		{
			for (int32 ColumnIndex = 0; ColumnIndex < CheckerboardCornerDimensions.X; ++ColumnIndex)
			{
				const double Y = TopLeftCornerX + (SquareSize * ColumnIndex);
				const double Z = TopLeftCornerY - (SquareSize * RowIndex);
				ObjectPoints.Add(FVector(DistanceFromCamera, Y, Z));
			}
		}

		TArray<FVector2f> ImagePoints;
		bool bResult = FOpenCVHelper::ProjectPoints(ObjectPoints, TrueFxFy, ImageCenter, DistortionParameters, TrueCameraPose, ImagePoints);
		if (!bResult)
		{
			Test.AddError(TEXT("Project Points failed. Test could not be completed"));
			return;
		}

		/**
		 * Step 2: Run SolvePnP to solve for the camera pose using perfect input data. The expectation is that the solver will be able to compute the ground-truth camera pose.
		 */

		FTransform PerfectCameraPoseResult;
		FOpenCVHelper::SolvePnP(ObjectPoints, ImagePoints, TrueFxFy, ImageCenter, DistortionParameters, PerfectCameraPoseResult);

		/**
		 * Step 3: Introduce errors into the 3D and 2D point data to simulate real-world inaccuracies that occur when doing calibration
		 */

		 // Introduce some random noise to the 3D points. The checkerboard is a rigid object, so the individual 3D positions of each corner cannot change randomly with respect to one another.
		 // However, the entire board could have the wrong pose if, for example, the tracking data is noisy, or if the tracked rigid-body pose sent to UE from the tracking system is not precise.
		TArray<FVector> NoisyObjectPoints;
		NoisyObjectPoints.Reserve(ObjectPoints.Num());
		{
			const double NoiseScale = TestSettings->ObjectPointNoiseScale;
			const double NoiseX = (FMath::SRand() - 0.5) * (NoiseScale * 2);
			const double NoiseY = (FMath::SRand() - 0.5) * (NoiseScale * 2);
			const double NoiseZ = (FMath::SRand() - 0.5) * (NoiseScale * 2);

			for (const FVector& Point : ObjectPoints)
			{
				NoisyObjectPoints.Add(Point + FVector(NoiseX, NoiseY, NoiseZ));
			}
		}

		// Introduce some random noise to the 2D points. This simulates poor checkerboard detection, which could occur if the checkerboard is not perfectly in-focus, if the image resolution is too low,
		// or if there is some other imprecision in the corner detection algorithm. 
		TArray<FVector2f> NoisyImagePoints;
		NoisyImagePoints.Reserve(ImagePoints.Num());
		for (const FVector2f& Point : ImagePoints)
		{
			const double NoiseScale = TestSettings->ImagePointNoiseScale;
			const double NoiseX = (FMath::SRand() - 0.5) * (NoiseScale * 2);
			const double NoiseY = (FMath::SRand() - 0.5) * (NoiseScale * 2);

			NoisyImagePoints.Add(Point + FVector2f(NoiseX, NoiseY));
		}

		/**
		 * Step 4: Run SolvePnP to solve for the camera pose using imperfect input data, including noisy data and an incorrect guess for focal length
		 */

		const double EstimatedFocalLength = TestSettings->EstimatedFocalLength;
		const double EstimatedFocalLengthPixels = (EstimatedFocalLength / SensorWidth) * ImageSize.X;
		const FVector2D EstimatedFxFy = FVector2D(EstimatedFocalLengthPixels, EstimatedFocalLengthPixels);

		FTransform ImperfectCameraPoseResult;
		FOpenCVHelper::SolvePnP(NoisyObjectPoints, NoisyImagePoints, EstimatedFxFy, ImageCenter, DistortionParameters, ImperfectCameraPoseResult);

		/**
		 * Step 4: Output the test results
		 */

		{
			Test.AddInfo(TEXT("Ground-Truth Camera Pose"));
			const FVector Translation = TrueCameraPose.GetTranslation();
			const FRotator Rotator = TrueCameraPose.GetRotation().Rotator();

			Test.AddInfo(FString::Printf(TEXT("\t\t\t\tTranslation: (%lf, %lf, %lf)"), Translation.X, Translation.Y, Translation.Z));
			Test.AddInfo(FString::Printf(TEXT("\t\t\t\tRotation:    (%lf, %lf, %lf)"), Rotator.Roll, Rotator.Pitch, Rotator.Yaw));
		}

		{
			Test.AddInfo(TEXT("Perfectly Solved Camera Pose"));
			const FVector Translation = PerfectCameraPoseResult.GetTranslation();
			const FRotator Rotator = PerfectCameraPoseResult.GetRotation().Rotator();

			Test.AddInfo(FString::Printf(TEXT("\t\t\t\tTranslation: (%lf, %lf, %lf)"), Translation.X, Translation.Y, Translation.Z));
			Test.AddInfo(FString::Printf(TEXT("\t\t\t\tRotation:    (%lf, %lf, %lf)"), Rotator.Roll, Rotator.Pitch, Rotator.Yaw));
		}

		{
			Test.AddInfo(TEXT("Imperfectly Solved Camera Pose"));
			const FVector Translation = ImperfectCameraPoseResult.GetTranslation();
			const FRotator Rotator = ImperfectCameraPoseResult.GetRotation().Rotator();

			Test.AddInfo(FString::Printf(TEXT("\t\t\t\tTranslation: (%lf, %lf, %lf)"), Translation.X, Translation.Y, Translation.Z));
			Test.AddInfo(FString::Printf(TEXT("\t\t\t\tRotation:    (%lf, %lf, %lf)"), Rotator.Roll, Rotator.Pitch, Rotator.Yaw));
		}
	}

	void CopyDefaultsFromBase(const UStruct* StructDef, void* TestStruct, void* BaseStruct, void* DefaultsStruct)
	{
		// Iterate over each of the struct properties
		for (TFieldIterator<FProperty> PropIt(StructDef); PropIt; ++PropIt)
		{
			FProperty* Property = *PropIt;

			// If the current property is another struct, recursively check its properties
			if (Property->IsA<FStructProperty>())
			{
				FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);

				void* TmpTestValue = Property->ContainerPtrToValuePtr<uint8>(TestStruct);
				void* TmpBaseValue = Property->ContainerPtrToValuePtr<uint8>(BaseStruct);
				void* TmpDefaultValue = Property->ContainerPtrToValuePtr<uint8>(DefaultsStruct);

				CopyDefaultsFromBase(StructProperty->Struct, TmpTestValue, TmpBaseValue, TmpDefaultValue);
			}
			else
			{
				// If the value of the property in the current test struct is the default value of that property,
				// copy the value of the property from the base struct into the current test struct.
				void* TestValue = Property->ContainerPtrToValuePtr<uint8>(TestStruct);
				void* DefaultValue = Property->ContainerPtrToValuePtr<uint8>(DefaultsStruct);
				if (Property->Identical(TestValue, DefaultValue, PPF_None))
				{
					void* ValueToCopy = Property->ContainerPtrToValuePtr<uint8>(BaseStruct);
					Property->SetValue_InContainer(TestStruct, ValueToCopy);
				}
			}
		}
	}

	void LoadCalibrationTestsFromFile(const FString FileName, FCalibrationTestSet& TestSet)
	{
		if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*FileName)))
		{
			TSharedRef< TJsonReader<UTF8CHAR> > JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(FileReader.Get());

			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
			{
				if (FJsonObjectConverter::JsonObjectToUStruct<FCalibrationTestSet>(JsonObject.ToSharedRef(), &TestSet))
				{
					FCalibrationTest DefaultCalibrationTest;
					for (FCalibrationTest& TestStruct : TestSet.Tests)
					{
						// If any of the tests specify a base test index, all of its properties that were not explicitly written in the json file will be copied from that base test
						const int32 BaseTestIndex = TestStruct.BaseTestIndex;
						if (BaseTestIndex >= 0)
						{
							FCalibrationTest* BaseTestPtr = TestSet.Tests.FindByPredicate([BaseTestIndex](const FCalibrationTest& TestStruct) { return TestStruct.TestIndex == BaseTestIndex; });
							CopyDefaultsFromBase(FCalibrationTest::StaticStruct(), &TestStruct, BaseTestPtr, &DefaultCalibrationTest);
						}
					}
				}
				else
				{
					UE_LOGF(LogTestCameraCalibration, Error, "Failed to convert json object to Calibration Test Set structure.");
				}
			}
			else
			{
				UE_LOGF(LogTestCameraCalibration, Error, "Failed to deserialize json file. Check that it is properly formatted.");
			}
		}
		else
		{
			UE_LOGF(LogTestCameraCalibration, Error, "Failed to read test filename");
		}
	}

	void LoadDatasetFromFile(FCalibrationTest& CalibrationTest, TArray<FObjectPoints>& ObjectPoints, TArray<FImagePoints>& ImagePoints)
	{
		// If the dataset path is not absolute, assume that it is relative to the project content directory
		if (FPaths::IsRelative(CalibrationTest.DatasetPath))
		{
			CalibrationTest.DatasetPath = FPaths::ProjectContentDir() / CalibrationTest.DatasetPath;
		}

		// Find all json files in the selected directory
		TArray<FString> FoundFiles;
		const FString FileExtension = TEXT(".json");
		IFileManager::Get().FindFiles(FoundFiles, *CalibrationTest.DatasetPath, *FileExtension);

		// Early-out if selected directory has no json files to import
		if (FoundFiles.Num() < 1)
		{
			UE_LOGF(LogTestCameraCalibration, Error, "The following dataset had no json files to import: %ls", *CalibrationTest.DatasetPath);
			return;
		}

		for (const FString& File : FoundFiles)
		{
			const FString JsonFileName = CalibrationTest.DatasetPath / File;

			// Open the Json file for reading, and initialize a JsonReader to parse the contents
			if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*JsonFileName)))
			{
				TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(FileReader.Get());

				// Deserialize the row data from the Json file into a Json object
				TSharedPtr<FJsonObject> JsonData = MakeShared<FJsonObject>();
				if (FJsonSerializer::Deserialize(JsonReader, JsonData))
				{
					FCalibrationDatasetImage LoadedDataset;
					if (FJsonObjectConverter::JsonObjectToUStruct<FCalibrationDatasetImage>(JsonData.ToSharedRef(), &LoadedDataset))
					{
						// Copy the loaded 3D and 2D points from this json file into the output point sets
						FObjectPoints Point3d;
						Point3d.Points = LoadedDataset.Points3d;

						FImagePoints Points2d;
						Points2d.Points = LoadedDataset.Points2d;

						ObjectPoints.Add(Point3d);
						ImagePoints.Add(Points2d);

						// Copy the image dimensions in the json file to the calibration test's camera profile
						CalibrationTest.CameraProfile.ImageSize = FIntPoint(LoadedDataset.ImageWidth, LoadedDataset.ImageHeight);
					}
				}
			}
		}
	}

	void GenerateCalibratorPoints(const FCheckerboardProfile& CheckerboardProfile, const TArray<FTransform>& CheckerboardPoses, TArray<FObjectPoints>& OutCheckerboardPoints)
	{
		OutCheckerboardPoints.Empty();
		OutCheckerboardPoints.Reserve(CheckerboardPoses.Num());

		const FIntPoint CheckerboardCornerDimensions = CheckerboardProfile.GetCornerDimensions();
		const int32 NumInnerCorners = CheckerboardCornerDimensions.X * CheckerboardCornerDimensions.Y;

		// Compute the location of each checkerboard corner as if the board were centered at the origin and lying in the YZ plane
		FObjectPoints CheckerboardPointsAtOrigin;

		// The top left intersection is one square length to the right and down from the actual top left corner of the board
		const double TopLeftCornerX = 0.0 - (CheckerboardProfile.CheckerboardDimensions.X * 0.5 * CheckerboardProfile.SquareSize) + CheckerboardProfile.SquareSize;
		const double TopLeftCornerY = 0.0 + (CheckerboardProfile.CheckerboardDimensions.Y * 0.5 * CheckerboardProfile.SquareSize) - CheckerboardProfile.SquareSize;

		for (int32 RowIndex = 0; RowIndex < CheckerboardCornerDimensions.Y; ++RowIndex)
		{
			for (int32 ColumnIndex = 0; ColumnIndex < CheckerboardCornerDimensions.X; ++ColumnIndex)
			{
				const double Y = TopLeftCornerX + (CheckerboardProfile.SquareSize * ColumnIndex);
				const double Z = TopLeftCornerY - (CheckerboardProfile.SquareSize * RowIndex);
				CheckerboardPointsAtOrigin.Points.Add(FVector(0, Y, Z));
			}
		}

		// Generate a set of checkerboard points for each pose by transforming the points centered at the origin by each checkerboard pose
		for (const FTransform& Pose : CheckerboardPoses)
		{
			FObjectPoints CheckerboardPoints;
			CheckerboardPoints.Points.Reserve(NumInnerCorners);
			for (const FVector& Point : CheckerboardPointsAtOrigin.Points)
			{
				CheckerboardPoints.Points.Add(Pose.TransformPosition(Point));
			}

			OutCheckerboardPoints.Add(CheckerboardPoints);
		}
	}

	bool ProjectCalibratorPoints(const FCameraProfile& CameraProfile, const TArray<FTransform>& CameraPoses, const TArray<FObjectPoints>& CalibratorPoints, TArray<FImagePoints>& OutImagePoints)
	{
		const int32 NumTestImages = CalibratorPoints.Num();
		if (NumTestImages != CameraPoses.Num())
		{
			return false;
		}

		OutImagePoints.Empty();
		OutImagePoints.Reserve(NumTestImages);

		for (int32 ImageIndex = 0; ImageIndex < NumTestImages; ++ImageIndex)
		{
			const FObjectPoints& CalibratorPointsForImage = CalibratorPoints[ImageIndex];
			const FTransform& CameraPoseForImage = CameraPoses[ImageIndex];

			FImagePoints Image;
			bool bResult = FOpenCVHelper::ProjectPoints(CalibratorPointsForImage.Points, CameraProfile.GetFxFyInPixels(), CameraProfile.ImageCenter, CameraProfile.DistortionParameters, CameraPoseForImage, Image.Points);
			if (!bResult)
			{
				return false;
			}

			OutImagePoints.Add(Image);
		}

		return true;
	}

	void DrawDebugCoverage(const TArray<FImagePoints>& CheckerboardImages, FIntPoint CheckerboardCornerDimensions, FIntPoint ImageSize)
	{
		const UTestCameraCalibrationSettings* TestSettings = GetDefault<UTestCameraCalibrationSettings>();
		if (TestSettings->bShowCheckerboardImage)
		{
			UTexture2D* DebugTexture = UTexture2D::CreateTransient(ImageSize.X, ImageSize.Y, EPixelFormat::PF_B8G8R8A8);
			UE::CameraCalibration::Private::ClearTexture(DebugTexture, FColor::Black);

			for (const FImagePoints& Image : CheckerboardImages)
			{
				FOpenCVHelper::DrawCheckerboardCorners(Image.Points, CheckerboardCornerDimensions, DebugTexture);
			}

			TSharedRef<SCustomDialog> DebugImageDialog =
				SNew(SCustomDialog)
				.UseScrollBox(false)
				.Content()
				[
					SNew(SImageTexture, DebugTexture)
				]
			.Buttons
			({
				SCustomDialog::FButton(NSLOCTEXT("TestCameraCalibration", "OkButton", "Ok")),
				});

			DebugImageDialog->Show();
		}
	}

	ECalibrationFlags GetCalibrationFlags(const FSolverSettings& SolverSettings)
	{
		static auto TestOptional = [](const TOptional<bool>& Value) {return Value && *Value; };

		ECalibrationFlags SolverFlags = ECalibrationFlags::None;
		if (TestOptional(SolverSettings.bUseIntrinsicGuess))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::UseIntrinsicGuess);
		}
		if (TestOptional(SolverSettings.bUseExtrinsicGuess))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::UseExtrinsicGuess);
		}
		if (TestOptional(SolverSettings.bFixFocalLength))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixFocalLength);
		}
		if (TestOptional(SolverSettings.bFixPrincipalPoint))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixPrincipalPoint);
		}
		if (TestOptional(SolverSettings.bFixExtrinsics))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixExtrinsics);
		}
		if (TestOptional(SolverSettings.bFixDistortion))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixDistortion);
		}
		if (TestOptional(SolverSettings.bFixAspectRatio))
		{
			EnumAddFlags(SolverFlags, ECalibrationFlags::FixAspectRatio);
		}
		return SolverFlags;
	}

	void TestCameraCalibration(FCalibrationTest& CalibrationTest, bool bLogVerboseTestDescription)
	{
		if (bLogVerboseTestDescription)
		{
			Logging::LogTestDescription(CalibrationTest);
		}

		TArray<FObjectPoints> CalibratorPoints3D;
		TArray<FImagePoints> ImagePoints2D;

		// If no dataset path is provided, the 3D and 2D points will be generated from the test description
		if (CalibrationTest.DatasetPath.IsEmpty())
		{
			// Generate the 3D checkerboard points based on the checkerboard description and poses
			TArray<FTransform> CheckerboardPoses;
			CalibrationTest.GetCheckerboardTransforms(CheckerboardPoses);

			GenerateCalibratorPoints(CalibrationTest.CheckerboardProfile, CheckerboardPoses, CalibratorPoints3D);

			// Generate the 2D image points by projecting the 3D checkerboard points using the camera properties and poses
			TArray<FTransform> CameraPoses;
			CalibrationTest.GetCameraTransforms(CameraPoses);

			ProjectCalibratorPoints(CalibrationTest.CameraProfile, CameraPoses, CalibratorPoints3D, ImagePoints2D);

			// Pop up a dialog window with an image showing a debug view of the calibration patterns for this set of images
			DrawDebugCoverage(ImagePoints2D, CalibrationTest.CheckerboardProfile.GetCornerDimensions(), CalibrationTest.CameraProfile.ImageSize);
		}
		else
		{
			LoadDatasetFromFile(CalibrationTest, CalibratorPoints3D, ImagePoints2D);
		}

		ULensDistortionSolverOpenCV* TestSolver = NewObject<ULensDistortionSolverOpenCV>();

		const FSolverSettings SolverSettings = CalibrationTest.SolverSettings;
		const ECalibrationFlags SolverFlags = GetCalibrationFlags(SolverSettings);

		// TODO: Allow for camera pose guesses in test description
		TArray<FTransform> CameraPoseGuesses;
		CameraPoseGuesses.Empty();

		TArray<float> InitialDistortion;
		InitialDistortion.Empty();

		const FVector2D FocalLengthGuessInPixels = CalibrationTest.CameraProfile.ConvertFocalLengthToPixels(SolverSettings.FocalLengthGuess);

		TArray<FTransform> TargetPoses;

		FDistortionCalibrationResult Result = TestSolver->Solve(
			CalibratorPoints3D,
			ImagePoints2D,
			CalibrationTest.CameraProfile.ImageSize,
			FocalLengthGuessInPixels,
			SolverSettings.ImageCenterGuess,
			InitialDistortion,
			CameraPoseGuesses,
			TargetPoses,
			USphericalLensModel::StaticClass(),
			1.0,
			SolverFlags
		);

		const double FocalLengthInMM = (Result.FocalLength.FxFy.X / CalibrationTest.CameraProfile.ImageSize.X) * CalibrationTest.CameraProfile.SensorSize.X;
		Logging::LogCalibrationResult(Result, FocalLengthInMM);
	}
}

bool FTestDistortionSpherical::RunTest(const FString& Parameters)
{
	UE::Private::CameraCalibration::AutomatedTests::TestDistortionCalibration(*this);
	return true;
}

bool FTestNodalOffset::RunTest(const FString& Parameters)
{
	UE::Private::CameraCalibration::AutomatedTests::TestNodalOffsetCalibration(*this);
	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS

void UAutoCalibrationTest::RunTests(FString Filename, bool bLogVerboseTestDescription)
{

#if WITH_DEV_AUTOMATION_TESTS
	if (FPaths::IsRelative(Filename))
	{
		Filename = FPaths::ProjectContentDir() / Filename;
	}

	FCalibrationTestSet CalibrationTestSet;
	UE::Private::CameraCalibration::AutomatedTests::LoadCalibrationTestsFromFile(Filename, CalibrationTestSet);

	for (FCalibrationTest& Test : CalibrationTestSet.Tests)
	{
		UE::Private::CameraCalibration::AutomatedTests::TestCameraCalibration(Test, bLogVerboseTestDescription);
	}
#endif // WITH_DEV_AUTOMATION_TESTS

}
