// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TestCameraCalibration.generated.h"

/** Exposes RunTests to BP and Python for ease of running a test in the editor */
UCLASS()
class UAutoCalibrationTest : public UObject
{
	GENERATED_BODY()

public:
	/** Runs the calibration test set defined in the input file */
	UFUNCTION(BlueprintCallable, Category = "Calibration")
	static void RunTests(FString Filename, bool bLogVerboseTestDescription = false);
};

/** Useful to deserialize a single json file from a calibration dataset */
USTRUCT()
struct FCalibrationDatasetImage
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FVector2D> Points2d;

	UPROPERTY()
	TArray<FVector> Points3d;

	UPROPERTY()
	int32 ImageWidth = -1;

	UPROPERTY()
	int32 ImageHeight = -1;
};

/**
 * Useful because this struct will be more human-readable when serialized to json than FTransform.
 * FRotator is easier to read/write than FTransform's FQuat, and the automated tests do not use FTransform's scale.
 */
USTRUCT()
struct FLocationRotation
{
	GENERATED_BODY()

	UPROPERTY()
	FVector Location = FVector::ZeroVector;

	UPROPERTY()
	FRotator Rotation = FRotator::ZeroRotator;
};

/** Description of a camera / lens, useful for projecting 3D calibrator points to 2D */
USTRUCT()
struct FCameraProfile
{
	GENERATED_BODY()

	UPROPERTY()
	FVector2D SensorSize = FVector2D(-1.0);

	UPROPERTY()
	FIntPoint ImageSize = FIntPoint(-1);

	UPROPERTY()
	double FocalLengthInMM = -1.0;

	UPROPERTY()
	FVector2D ImageCenter = FVector2D(-1.0);

	UPROPERTY()
	TArray<float> DistortionParameters;

	FVector2D ConvertFocalLengthToPixels(double FocalLength) const
	{
		return (FVector2D(FocalLength) / SensorSize) * FVector2D(ImageSize);
	}

	FVector2D GetFxFyInPixels() const
	{
		return ConvertFocalLengthToPixels(FocalLengthInMM);
	}
};

/** Description of a checkerboard calibrator, useful for generating sets of 3D calibration points in world space */
USTRUCT()
struct FCheckerboardProfile
{
	GENERATED_BODY()

	UPROPERTY()
	FIntPoint CheckerboardDimensions = FIntPoint(-1);

	UPROPERTY()
	double SquareSize = -1.0;

	FIntPoint GetCornerDimensions() const
	{
		return CheckerboardDimensions - FIntPoint(1, 1);
	}
};

/** Settings for the camera calibration solver */
USTRUCT()
struct FSolverSettings
{
	GENERATED_BODY()

	UPROPERTY()
	double FocalLengthGuess = -1.0;

	UPROPERTY()
	FVector2D ImageCenterGuess = FVector2D(-1.0);

	UPROPERTY()
	TOptional<bool> bUseIntrinsicGuess;
	
	UPROPERTY()
	TOptional<bool> bUseExtrinsicGuess;
	
	UPROPERTY()
	TOptional<bool> bFixFocalLength;
	
	UPROPERTY()
	TOptional<bool> bFixPrincipalPoint;
	
	UPROPERTY()
	TOptional<bool> bFixExtrinsics;
	
	UPROPERTY()
	TOptional<bool> bFixDistortion;
	
	UPROPERTY()
	TOptional<bool> bFixAspectRatio;
};

/** Full camera calibration test description */
USTRUCT()
struct FCalibrationTest
{
	GENERATED_BODY()

	UPROPERTY()
	int32 TestIndex = 0;

	UPROPERTY()
	int32 BaseTestIndex = -1;

	UPROPERTY()
	FString DatasetPath;

	UPROPERTY()
	FCameraProfile CameraProfile;
	
	UPROPERTY()
	FCheckerboardProfile CheckerboardProfile;

	UPROPERTY()
	TArray<FLocationRotation> CameraPoses;
	
	UPROPERTY()
	TArray<FLocationRotation> CheckerboardPoses;

	UPROPERTY()
	FSolverSettings SolverSettings;

	void GetCameraTransforms(TArray<FTransform>& Transforms) const
	{
		GetTransforms(CameraPoses, Transforms);
	}

	void GetCheckerboardTransforms(TArray<FTransform>& Transforms) const
	{
		GetTransforms(CheckerboardPoses, Transforms);
	}

	void GetTransforms(const TArray<FLocationRotation>& Poses, TArray<FTransform>& Transforms) const
	{
		Transforms.Empty();
		Transforms.Reserve(Poses.Num());
		for (const FLocationRotation& Pose : Poses)
		{
			Transforms.Add(FTransform(Pose.Rotation, Pose.Location));
		}
	}
};

/** Set of camera calibration test descriptions */
USTRUCT()
struct FCalibrationTestSet
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FCalibrationTest> Tests;
};
