// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestCameraCalibrationSettings.h"

UTestCameraCalibrationSettings::UTestCameraCalibrationSettings() 
{
}

FName UTestCameraCalibrationSettings::GetCategoryName() const 
{ 
	return TEXT("Plugins"); 
}

#if WITH_EDITOR
FText UTestCameraCalibrationSettings::GetSectionText() const 
{ 
	return NSLOCTEXT("TestCameraCalibrationSettings", "TestCameraCalibrationSettingsSection", "Camera Calibration Test Settings"); 
}

FName UTestCameraCalibrationSettings::GetSectionName() const 
{	
	return TEXT("Camera Calibration Test"); 
}
#endif
