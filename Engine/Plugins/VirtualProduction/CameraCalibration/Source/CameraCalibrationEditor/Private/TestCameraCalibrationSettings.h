// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "Engine/DeveloperSettings.h"
#include "Models/SphericalLensModel.h"

#include "TestCameraCalibrationSettings.generated.h"

/**
 * Settings for the camera calibration testing
 */
UCLASS(config = EditorPerProjectUserSettings)
class UTestCameraCalibrationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UTestCameraCalibrationSettings();

	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FName GetSectionName() const override;
#endif
	//~ End UDeveloperSettings interface

public:

#if WITH_EDITORONLY_DATA

	/** Size of the image that the test camera would generate */
	UPROPERTY(config, EditAnywhere, Category = "True Camera Properties")
	FIntPoint ImageSize = FIntPoint(1920, 1080);

	/** Size of the sensor (in mm) of the test camera */
	UPROPERTY(config, EditAnywhere, Category = "True Camera Properties")
	FVector2f SensorDimensions = FVector2f(23.76f, 13.365f);

	/** True 3D pose of the test camera */
	UPROPERTY(config, EditAnywhere, Category = "True Camera Properties")
	FTransform CameraTransform = FTransform::Identity;

	/** True focal length of the test camera */
	UPROPERTY(config, EditAnywhere, Category = "True Camera Properties")
	double FocalLength = 35.0;

	/** True distortion coefficients of the test lens */
	UPROPERTY(config, EditAnywhere, Category = "True Camera Properties")
	FSphericalDistortionParameters SphericalDistortionParameters;

	/** Number of camera views to generate images from */
	UPROPERTY(config, EditAnywhere, Category = "True Camera Properties")
	int32 NumCameraViews = 1;

	/** Number of squares in the checkerboard (columns x rows) */
	UPROPERTY(config, EditAnywhere, Category = "Calibrator Properties")
	FIntPoint CheckerboardDimensions = FIntPoint(16, 9);

	/** Size (in cm) of the length of each square of the checkerboard */
	UPROPERTY(config, EditAnywhere, Category = "Calibrator Properties")
	double CheckerboardSquareSize = 5.0;

	/** Distance (in cm) to place the test calibrator from the test camera */
	UPROPERTY(config, EditAnywhere, Category = "Calibrator Properties")
	double CalibratorDistanceFromCamera = 200.0;

	/** Maximum amount of random noise (in cm) to add to 3D object points */
	UPROPERTY(config, EditAnywhere, Category = "Input Data Errors")
	double ObjectPointNoiseScale = 0.0;

	/** Maximum amount of random noise (in pixels) to add to the 2D image points */
	UPROPERTY(config, EditAnywhere, Category = "Input Data Errors")
	double ImagePointNoiseScale = 0.0;

	/** If true, the solver will use the estimated focal length as an initial guess for the calibrated focal length */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bUseCameraIntrinsicGuess = false;

	/** User-provided guess for the focal length */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings", meta = (EditCondition = "bUseCameraIntrinsicGuess == true"))
	double EstimatedFocalLength = 35.0;

	/** If true, the solver will use the estimate camera transform as an initial guess for the calibrated camera pose */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bUseCameraExtrinsicGuess = false;

	/** User-provided guess for the camera pose */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings", meta = (EditCondition = "bUseCameraExtrinsicGuess == true"))
	FTransform EstimatedCameraTransform = FTransform::Identity;

	/** If true, the solver will not try to solve for focal length, and will use the focal length estimate */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bFixFocalLength = false;

	/** If true, the solver will not try to solve for image center, and will use (0.5, 0.5) */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bFixImageCenter = false;

	/** If true, the solver will not try to solve for the camera pose, and will use the camera transform estimate */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bFixExtrinsics = false;

	/** If true, the solver will not try to solve for distortion */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bFixDistortion = false;

	/** If true, the solver will not try to solve for distortion, and will assume there is zero distortion in the image */
	UPROPERTY(config, EditAnywhere, Category = "Calibrate Camera Solver Settings")
	bool bFixZeroDistortion = false;

	/** If true, a debug image will be displayed showing the 2D projected points of the test checkerboard for each of the camera views */
	UPROPERTY(config, EditAnywhere, Category = "Debug")
	bool bShowCheckerboardImage = false;

#endif
};
