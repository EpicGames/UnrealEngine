// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"


/**
 * Implements the visual style of the camera calibration tools UI.
 */
class FCameraCalibrationEditorStyle	final : public FSlateStyleSet
{
public:

	/** Default constructor. */
	FCameraCalibrationEditorStyle()	: FSlateStyleSet("CameraCalibrationEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		// Set placement browser icons
		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/"));
		Set("PlacementBrowser.Icons.VirtualProduction", new IMAGE_BRUSH_SVG("Starship/Common/VirtualProduction", Icon16x16));

		// Toolbar
		Set("CameraCalibration.ShowMediaPlaybackControls", new IMAGE_BRUSH_SVG("Starship/AssetIcons/MediaPlayer_16", Icon16x16));

		// Set miscellaneous icons
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("VirtualProduction/CameraCalibration/Content/Editor/Icons/"));
		Set("ClassThumbnail.LensFile", new IMAGE_BRUSH("LensFileIcon_64x", Icon64x64));
		Set("ClassIcon.LensFile", new IMAGE_BRUSH("LensFileIcon_20x", Icon20x20));

		Set("CameraCalibration.RewindMedia.Small", new IMAGE_BRUSH("icon_rewind_40x", Icon20x20));
		Set("CameraCalibration.ReverseMedia.Small", new IMAGE_BRUSH("icon_reverse_40x", Icon20x20));
		Set("CameraCalibration.StepBackMedia.Small", new IMAGE_BRUSH("icon_step_back_40x", Icon20x20));
		Set("CameraCalibration.PlayMedia.Small", new IMAGE_BRUSH("icon_play_40x", Icon20x20));
		Set("CameraCalibration.PauseMedia.Small", new IMAGE_BRUSH("icon_pause_40x", Icon20x20));
		Set("CameraCalibration.StepForwardMedia.Small", new IMAGE_BRUSH("icon_step_40x", Icon20x20));
		Set("CameraCalibration.ForwardMedia.Small", new IMAGE_BRUSH("icon_forward_40x", Icon20x20));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	/** Virtual destructor. */
	virtual ~FCameraCalibrationEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FCameraCalibrationEditorStyle& Get()
	{
		static FCameraCalibrationEditorStyle Inst;
		return Inst;
	}
};
