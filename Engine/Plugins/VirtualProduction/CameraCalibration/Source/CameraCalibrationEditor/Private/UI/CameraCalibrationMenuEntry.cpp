// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/CameraCalibrationMenuEntry.h"

#include "AssetEditor/CameraCalibrationCommands.h"
#include "AssetToolsModule.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationSubsystem.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Factories/LensFileFactoryNew.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "Misc/FeedbackContext.h"
#include "LensFile.h"
#include "PropertyCustomizationHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Textures/SlateIcon.h"
#include "ToolMenus.h"
#include "UI/CameraCalibrationEditorStyle.h"

#define LOCTEXT_NAMESPACE "CameraCalibrationMenu"

struct FCameraCalibrationMenuEntryImpl
{
	FCameraCalibrationMenuEntryImpl()
	{
		TSharedPtr<FUICommandList> Actions = MakeShared<FUICommandList>();

		// Action to edit the current selected lens file
		Actions->MapAction(FCameraCalibrationCommands::Get().Edit,
			FExecuteAction::CreateLambda([this]()
			{
				if (ULensFile* LensFile = GetDefaultLensFile())
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LensFile);
				}
			}),
			FCanExecuteAction::CreateLambda([this] 
			{ 
				return GetDefaultLensFile() != nullptr;
			})
		);

		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
		FToolMenuSection& Section = Menu->FindOrAddSection("LensFile");

		auto ButtonTooltipLambda = [this]()
		{
			ULensFile* LensFile = GetDefaultLensFile();
			if (LensFile == nullptr)
			{
				return LOCTEXT("NoFile_ToolTip", "Select a Lens File to edit it.");
			}
			return FText::Format(LOCTEXT("LensFile_ToolTip", "Edit '{0}'") , FText::FromName(LensFile->GetFName()));
		};

		// Add a button to edit the default LensFile
		FToolMenuEntry LensFileButtonEntry = FToolMenuEntry::InitToolBarButton(
			FCameraCalibrationCommands::Get().Edit,
			LOCTEXT("LensFile_Label", "Lens File"),
			MakeAttributeLambda(ButtonTooltipLambda),
			FSlateIcon(FCameraCalibrationEditorStyle::Get().GetStyleSetName(), TEXT("ClassIcon.LensFile"))
		);
		LensFileButtonEntry.SetCommandList(Actions);

		FToolMenuEntry LensFileMenuEntry = FToolMenuEntry::InitComboButton(
		"LensFileMenu",
		FUIAction(),
		FOnGetContent::CreateRaw(this, &FCameraCalibrationMenuEntryImpl::GenerateMenuContent),
		LOCTEXT("LevelEditorToolbarLensFileButtonLabel", "LensFile"),
		LOCTEXT("LevelEditorToolbarLensFileButtonTooltip", "Configure default LensFile"),
		FSlateIcon(),
		true //bInSimpleComboBox
		);

		LensFileMenuEntry.StyleNameOverride = "CalloutToolbar";
		Section.AddEntry(LensFileButtonEntry);
		Section.AddEntry(LensFileMenuEntry);
	}

	~FCameraCalibrationMenuEntryImpl()
	{
		if (!IsEngineExitRequested())
		{
			UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "LensFile");
		}
	}

	ULensFile* GetDefaultLensFile()
	{
		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		return SubSystem->GetDefaultLensFile();
	}

	void CreateNewLensFile()
	{
		ULensFileFactoryNew* FactoryInstance = DuplicateObject<ULensFileFactoryNew>(GetDefault<ULensFileFactoryNew>(), GetTransientPackage());
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		ULensFile* NewAsset = Cast<ULensFile>(FAssetToolsModule::GetModule().Get().CreateAssetWithDialog(FactoryInstance->GetSupportedClass(), FactoryInstance));
		if (NewAsset != nullptr)
		{
			//If a new lens is created from the toolbar, assign it as startup user lens file 
			//and current default engine lens file.
			GetMutableDefault<UCameraCalibrationEditorSettings>()->SetUserLensFile(NewAsset);
			UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
			SubSystem->SetDefaultLensFile(NewAsset);

			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);
		}

	}

	void NewLensFileSelected(const FAssetData& AssetData)
	{
		FSlateApplication::Get().DismissAllMenus();

		GWarn->BeginSlowTask(LOCTEXT("LensFileLoadPackage", "Loading Lens File"), true, false);
		ULensFile* Asset = Cast<ULensFile>(AssetData.GetAsset());
		GWarn->EndSlowTask();

		//If a new lens is selected from the toolbar, assign it as startup user lens file 
		//and current default engine lens file.
		GetMutableDefault<UCameraCalibrationEditorSettings>()->SetUserLensFile(Asset);
		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		SubSystem->SetDefaultLensFile(Asset);
	}

	TSharedRef<SWidget> GenerateMenuContent()
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

		MenuBuilder.BeginSection("LensFile", LOCTEXT("NewLensFileSection", "New"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CreateMenuLabel", "New Empty Lens File"),
				LOCTEXT("CreateMenuTooltip", "Create a new Lens File asset."),
				FSlateIcon(FCameraCalibrationEditorStyle::Get().GetStyleSetName(), TEXT("ClassIcon.LensFile")),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FCameraCalibrationMenuEntryImpl::CreateNewLensFile)
				)
			);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("LensFile", LOCTEXT("LensFileSection", "Lens File"));
		{
			ULensFile* LensFile = GetDefaultLensFile();
			const bool bIsFileValid = (LensFile != nullptr);

			MenuBuilder.AddSubMenu(
				bIsFileValid ? FText::FromName(LensFile->GetFName()) : LOCTEXT("SelectMenuLabel", "Select the default Lens File"),
				LOCTEXT("SelectMenuTooltip", "Select the default lens file for the project."),
				FNewMenuDelegate::CreateRaw(this, &FCameraCalibrationMenuEntryImpl::AddObjectSubMenu),
				FUIAction(),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void AddObjectSubMenu(FMenuBuilder& MenuBuilder)
	{
		ULensFile* LensFile = GetDefaultLensFile();
		FAssetData CurrentAssetData = LensFile ? FAssetData(LensFile) : FAssetData();

		TArray<const UClass*> ClassFilters;
		ClassFilters.Add(ULensFile::StaticClass());

		MenuBuilder.AddWidget(
			PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
				CurrentAssetData,
				LensFile != nullptr,
				false,
				ClassFilters,
				TArray<UFactory*>(),
				FOnShouldFilterAsset::CreateLambda([CurrentAssetData](const FAssetData& InAssetData){ return InAssetData == CurrentAssetData; }),
				FOnAssetSelected::CreateRaw(this, &FCameraCalibrationMenuEntryImpl::NewLensFileSelected),
				FSimpleDelegate()
			),
			FText::GetEmpty(),
			true,
			false
		);
	}

public:

	static TUniquePtr<FCameraCalibrationMenuEntryImpl> Implementation;
};

TUniquePtr<FCameraCalibrationMenuEntryImpl> FCameraCalibrationMenuEntryImpl::Implementation;

void FCameraCalibrationMenuEntry::Register()
{
	if (!IsRunningCommandlet() && GetDefault<UCameraCalibrationEditorSettings>()->bShowEditorToolbarButton)
	{
		FCameraCalibrationMenuEntryImpl::Implementation = MakeUnique<FCameraCalibrationMenuEntryImpl>();
	}
}

void FCameraCalibrationMenuEntry::Unregister()
{
	FCameraCalibrationMenuEntryImpl::Implementation.Reset();
}

#undef LOCTEXT_NAMESPACE
