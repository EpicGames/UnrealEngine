// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationWidgetHelpers.h"

#include "AssetEditor/SSimulcamViewport.h"
#include "Dialog/SCustomDialog.h"
#include "Engine/Texture2D.h"
#include "Internationalization/Text.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CameraCalibrationWidgetHelpers"


const int32 FCameraCalibrationWidgetHelpers::DefaultRowHeight = 35;


TSharedRef<SWidget> FCameraCalibrationWidgetHelpers::BuildLabelWidgetPair(FText&& Text, TSharedRef<SWidget> Widget)
{
	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(5,5)
		.FillWidth(0.35f)
		[SNew(STextBlock).Text(Text)]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(5, 5)
		.FillWidth(0.65f)
		[Widget];
}

void FCameraCalibrationWidgetHelpers::DisplayTextureInWindowAlmostFullScreen(UTexture2D* Texture, FText&& Title, float ScreenMarginFactor)
{
	if (!Texture || (Texture->GetSurfaceWidth() < 1) || (Texture->GetSurfaceHeight() < 1))
	{
		return;
	}

	// Display the full resolution image a large as possible but clamped to the size of the primary display
	// and preserving the aspect ratio of the image.

	FDisplayMetrics Display;
	FDisplayMetrics::RebuildDisplayMetrics(Display);

	float DetectionWindowMaxWidth = Texture->GetSurfaceWidth();
	float DetectionWindowMaxHeight = Texture->GetSurfaceHeight();

	const float FactorWidth = ScreenMarginFactor * Display.PrimaryDisplayWidth / DetectionWindowMaxWidth;
	const float FactorHeight = ScreenMarginFactor * Display.PrimaryDisplayHeight / DetectionWindowMaxHeight;

	if (FactorWidth < FactorHeight)
	{
		DetectionWindowMaxWidth *= FactorWidth;
		DetectionWindowMaxHeight *= FactorWidth;
	}
	else
	{
		DetectionWindowMaxWidth *= FactorHeight;
		DetectionWindowMaxHeight *= FactorHeight;
	}

	TSharedPtr<SBox> ViewportWrapper;

	TSharedRef<SCustomDialog> DetectionWindow =
		SNew(SCustomDialog)
		.Title(Title)
		.ScrollBoxMaxHeight(DetectionWindowMaxHeight)
		.Content()
		[
			SAssignNew(ViewportWrapper, SBox)
			.MinDesiredWidth(DetectionWindowMaxWidth)
			.MinDesiredHeight(DetectionWindowMaxHeight)
			[
				SNew(SSimulcamViewport)
				.OverrideTexture(Texture)
			]
		]
		.Buttons
		({
			SCustomDialog::FButton(LOCTEXT("Ok", "Ok")),
		});

	DetectionWindow->Show();

	// Compensate for DPI scale the window size and its location
	{
		const float DPIScale = DetectionWindow->GetDPIScaleFactor();

		if (!FMath::IsNearlyEqual(DPIScale, 1.0f))
		{
			check(DPIScale > KINDA_SMALL_NUMBER);

			const int32 DetectionWindowMaxWidthScaled = DetectionWindowMaxWidth / DPIScale;
			const int32 DetectionWindowMaxHeightScaled = DetectionWindowMaxHeight / DPIScale;

			ViewportWrapper->SetMaxDesiredWidth(DetectionWindowMaxWidthScaled);
			ViewportWrapper->SetMaxDesiredHeight(DetectionWindowMaxHeightScaled);

			const int32 DisplayWidthScaled = Display.PrimaryDisplayWidth / DPIScale;
			const int32 DisplayHeightScaled = Display.PrimaryDisplayHeight / DPIScale;

			DetectionWindow->MoveWindowTo(FVector2D(
				(DisplayWidthScaled - DetectionWindowMaxWidthScaled) / 2,
				(DisplayHeightScaled - DetectionWindowMaxHeightScaled) / 2
			));
		}
	}
}

bool FCameraCalibrationWidgetHelpers::ShowMergeFocusWarning(bool& bOutReplaceExistingZoomPoints)
{
	bool bReplaceExistingZoomPoints = false;
	TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("FocusMergeWarningTitle", "Merge existing focus point?")))
		.ContentAreaPadding(16.0)
		.Content()
		[
			SNew(SVerticalBox)
					
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FocusMergeWarningLabel", "A focus point already exists with that value. Would you like to merge this point with that point?"))
			]

			+SVerticalBox::Slot()
			.Padding(0.0, 4.0, 0.0, 0.0)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("ReplaceExistingToolTip", "When checked, any existing zoom points in the destination focus will be replaced with those in the source focus"))
				.IsChecked_Lambda([&bReplaceExistingZoomPoints]() { return bReplaceExistingZoomPoints ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([&bReplaceExistingZoomPoints](ECheckBoxState CheckBoxState) { bReplaceExistingZoomPoints = CheckBoxState == ECheckBoxState::Checked; })
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock).Text(LOCTEXT("ReplaceExistingLabel", "Replace existing zoom points?"))
				]
			]
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("MergeButtonLabel", "Merge")),
			SCustomDialog::FButton(LOCTEXT("CancelButtonLabel", "Cancel")) }
		);

	// Dialog result corresponds to the button that was pressed (e.g. 0 means first button, 1 second, -1 no buttons were pressed)
	const int32 DialogResult = Dialog->ShowModal();
	bOutReplaceExistingZoomPoints = bReplaceExistingZoomPoints;
	return DialogResult == 0;
}

bool FCameraCalibrationWidgetHelpers::ShowReplaceZoomWarning()
{
	TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("ZoomReplaceWarningTitle", "Replace existing zoom point?")))
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ZoomReplaceWarningLabel", "A point with that zoom value already exists, would you like to replace it with this point?"))
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("ReplaceButtonLabel", "Replace")),
			SCustomDialog::FButton(LOCTEXT("CancelButtonLabel", "Cancel")) }
		);

	// Dialog result corresponds to the button that was pressed (e.g. 0 means first button, 1 second, -1 no buttons were pressed)
	const int32 DialogResult = Dialog->ShowModal();
	return DialogResult == 0;
}

#undef LOCTEXT_NAMESPACE
