// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Fonts/SlateFontInfo.h"

class FText;
class SWidget;
class UTexture2D;

/**
 * Helpers to build UI used by the Camera Calibration modules.
 */
class FCameraCalibrationWidgetHelpers
{
public:
	/** Builds a UI with a horizontal box with a lable on the left and the provided widget on the right */
	static TSharedRef<SWidget> BuildLabelWidgetPair(FText&& Text, TSharedRef<SWidget> Widget);

	/** Stores the default row height used throughout the Camera Calibration UI */
	static const int32 DefaultRowHeight;

	/** Displays a window with the given texture, preserving aspect ratio and almost full screen */
	static void DisplayTextureInWindowAlmostFullScreen(UTexture2D* Texture, FText&& Title, float ScreenMarginFactor = 0.85f);

	/** Displays a warning dialog box asking if a user wants to merge one focus point into another
	 * @param bOutReplaceExistingZoomPoints Whether the user wants to replace existing zoom points in the merge
	 * @return true if the user accepted the merge, false otherwise */
	static bool ShowMergeFocusWarning(bool& bOutReplaceExistingZoomPoints);

	/** Displays a warning dialog box asking if the user wants to replace one zoom point with another
	 * @return true if the user accepted the replace, false otherwise */
	static bool ShowReplaceZoomWarning();
};