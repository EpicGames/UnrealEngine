// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterableActorPicker.h"

#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Engine/Selection.h"
#include "PropertyCustomizationHelpers.h"
#include "SAssetDropTarget.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FilterableActorPicker"

void SFilterableActorPicker::Construct( const FArguments& InArgs )
{
	OnSetObject = InArgs._OnSetObject;
	OnShouldFilterAsset = InArgs._OnShouldFilterAsset;
	ActorAssetData = InArgs._ActorAssetData;

	TSharedPtr<SHorizontalBox> ValueContentBox;

	ChildSlot
	[
		// Allow drag and dropping valid actors onto the widget
		SNew(SAssetDropTarget)
		.OnAreAssetsAcceptableForDropWithReason_Lambda([&](TArrayView<FAssetData> InAssets, FText& OutReason) -> bool
		{
			return OnShouldFilterAsset.IsBound() && OnShouldFilterAsset.Execute(InAssets[0]);
		})
		.OnAssetsDropped_Lambda([&](const FDragDropEvent& Event, TArrayView<FAssetData> InAssets) -> void
		{
			AssetComboButton->SetIsOpen(false);

			OnSetObject.ExecuteIfBound(InAssets[0]);
		})
		[
			SAssignNew(ValueContentBox, SHorizontalBox)
		]
	];

	AssetComboButton = SNew(SComboButton)
		.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
		.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
		.OnGetMenuContent_Lambda([&]() -> TSharedRef<SWidget>
		{
			FAssetData AssetData;
			GetActorAssetData(AssetData);

			return PropertyCustomizationHelpers::MakeActorPickerWithMenu(
				Cast<AActor>(AssetData.GetAsset()),
				false,
				FOnShouldFilterActor::CreateLambda([&](const AActor* const InActor) -> bool // ActorFilter
				{
					// Validate using the bound filter.
					return OnShouldFilterAsset.IsBound() && OnShouldFilterAsset.Execute(InActor);
				}),
				FOnActorSelected::CreateLambda([&](AActor* InActor) -> void // OnSet
				{
					// InActor should already be validated
					SetActorAssetData(InActor);
				}),
				FSimpleDelegate:: CreateLambda([&]() -> void // OnClose
				{
					AssetComboButton->SetIsOpen(false);
				}),
				FSimpleDelegate::CreateLambda([&]() -> void // OnUseSelected (when you click on that option in the menu)
				{
					// Get selected actor
					UObject* Selection = GEditor->GetSelectedActors()->GetTop(AActor::StaticClass());
								
					// Validate it and it passes, assign it
					if (Selection && OnShouldFilterAsset.IsBound() && OnShouldFilterAsset.Execute(FAssetData(Selection)))
					{
						SetActorAssetData(Selection);
					}
				}));
		})
		.OnMenuOpenChanged_Lambda([&](bool bOpen) -> void
		{
			if (!bOpen)
			{
				AssetComboButton->SetMenuContent(SNullWidget::NullWidget);
			}
		})
		.IsEnabled(true)
		.ContentPadding(2.0f)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot() // Actor name text block

			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(this, &SFilterableActorPicker::OnGetAssetName)
			]
		];

	TSharedRef<SHorizontalBox> ButtonBox =
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot() // Browse for actor button

		.Padding(2.0f, 0.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeBrowseButton(
				FSimpleDelegate::CreateLambda([&]() -> void // OnFindClicked
				{
					FAssetData AssetData;
					GetActorAssetData(AssetData);

					TArray<FAssetData> AssetDataList;
					AssetDataList.Add(AssetData);
					GEditor->SyncBrowserToObjects(AssetDataList);
				})
			)
		];

	TSharedPtr<SWidget> ButtonBoxWrapper;
	TSharedPtr<SVerticalBox> CustomContentBox;

	ValueContentBox->AddSlot()
	[
		SAssignNew(CustomContentBox, SVerticalBox)

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot() // the actor combo button
			[
				AssetComboButton.ToSharedRef()
			]

			+ SHorizontalBox::Slot() // the buttons that are on the right of the combo button
			.AutoWidth()
			[
				SAssignNew(ButtonBoxWrapper, SBox)
				.Padding(FMargin(4.f, 0.f))
				[
					ButtonBox
				]
			]
		]
	];

	// Create actor picker button and add it to the ButtonBox wrapper.
	{
		TSharedRef<SWidget> ActorPicker = PropertyCustomizationHelpers::MakeInteractiveActorPicker(
			FOnGetAllowedClasses::CreateLambda([&](TArray<const UClass*>& AllowedClasses) -> void
			{
				AllowedClasses.Add(AActor::StaticClass());
			}),
			FOnShouldFilterActor(), 
			FOnActorSelected::CreateLambda([&](AActor* InActor) -> void
			{
				SetActorAssetData(InActor);
			})
		);

		ActorPicker->SetEnabled(true);

		ButtonBox->AddSlot()
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				ActorPicker
			];
	}

	if (ButtonBoxWrapper.IsValid())
	{
		ButtonBoxWrapper->SetVisibility(ButtonBox->NumSlots() > 0 ? EVisibility::Visible : EVisibility::Collapsed);
	}
}

FText SFilterableActorPicker::OnGetAssetName() const
{
	FAssetData AssetData;
	GetActorAssetData(AssetData);

	if (!AssetData.IsValid())
	{
		return LOCTEXT("None", "None");
	}

	const AActor* Actor = Cast<AActor>(AssetData.GetAsset());
	if (Actor)
	{
		return FText::AsCultureInvariant(Actor->GetActorNameOrLabel());
	}

	return FText::AsCultureInvariant(AssetData.AssetName.ToString());
}

void SFilterableActorPicker::SetActorAssetData(const FAssetData& AssetData)
{
	// Note that we don't assign the ActorAssetData here.
	// It is expected that OnSetObject will be bound and assign it externally,
	// and we then read it via the slate attribute.

	AssetComboButton->SetIsOpen(false);

	if (AssetData.IsValid() && Cast<AActor>(AssetData.GetAsset()))
	{
		OnSetObject.ExecuteIfBound(AssetData);
	}	
}

void SFilterableActorPicker::GetActorAssetData(FAssetData& OutAssetData) const
{
	OutAssetData = ActorAssetData.Get();
}

#undef LOCTEXT_NAMESPACE
