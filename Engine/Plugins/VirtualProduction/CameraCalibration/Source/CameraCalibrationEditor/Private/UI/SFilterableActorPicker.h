// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PropertyCustomizationHelpers.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Picker that can filter actors.
 */
class SFilterableActorPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFilterableActorPicker)
		: _ActorAssetData()
	{}

	SLATE_ATTRIBUTE(FAssetData, ActorAssetData)

	SLATE_EVENT(FOnSetObject, OnSetObject)
	SLATE_EVENT(FOnShouldFilterAsset, OnShouldFilterAsset)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	//** Returns the name of the selected actor, to be displayed in the dropdown */
	FText OnGetAssetName() const;

	//** Called when the actor has been selected */
	void SetActorAssetData(const FAssetData& AssetData);

	/** Fills out the asset data with the current selection */
	void GetActorAssetData(FAssetData& OutAssetData) const;

private:

	/** Delegate to call when our object actor is set */
	FOnSetObject OnSetObject;

	/** Delegate for filtering valid actors */
	FOnShouldFilterAsset OnShouldFilterAsset;

	/** Current actor asset data */
	TAttribute<FAssetData> ActorAssetData;

private:

	/** The ComboButton used by the picker */
	TSharedPtr<class SComboButton> AssetComboButton;
};
