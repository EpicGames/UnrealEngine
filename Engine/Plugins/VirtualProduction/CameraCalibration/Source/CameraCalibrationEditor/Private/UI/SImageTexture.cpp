// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImageTexture.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"

void SImageTexture::Construct(const FArguments& InArgs, UTexture2D* InTexture)
{
	if (!InTexture)
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];

		return;
	}

	Texture = TStrongObjectPtr<UTexture2D>(InTexture);

	Brush.SetResourceObject(InTexture);
	Brush.ImageSize = FVector2f(Texture->GetSurfaceWidth(), Texture->GetSurfaceHeight());

	float AspectRatio = 1.0f;
	if (!FMath::IsNearlyZero(Texture->GetSurfaceHeight()))
	{
		AspectRatio = Texture->GetSurfaceWidth() / Texture->GetSurfaceHeight();
	}

	ChildSlot
	[
		SNew(SBox)
		.MinAspectRatio(AspectRatio)
		.MaxAspectRatio(AspectRatio)
		.MinDesiredWidth(InArgs._MinDesiredWidth)
		.MinDesiredHeight(InArgs._MinDesiredHeight)
		.MaxDesiredWidth(InArgs._MaxDesiredWidth)
		.MaxDesiredHeight(InArgs._MaxDesiredHeight)
		[
			SNew(SImage).Image(&Brush)
		]
	];
}
