// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateStructs.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


/** Widget that holds a reference to a UTexture and creates an SImage for that Texture using a brush */
class SImageTexture : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SImageTexture)
		: _MinDesiredWidth(FOptionalSize())
		, _MinDesiredHeight(FOptionalSize())
		, _MaxDesiredWidth(FOptionalSize())
		, _MaxDesiredHeight(FOptionalSize())
	{}

	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredHeight)
	SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredWidth)
	SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredHeight)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UTexture2D* InTexture);

private:
	FSlateBrush Brush;
	TStrongObjectPtr<UTexture2D> Texture;
};
