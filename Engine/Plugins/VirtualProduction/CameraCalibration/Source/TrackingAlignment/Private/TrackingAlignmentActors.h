// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TrackingAlignmentActors.generated.h"

/** Struct of Source and Origin actors from a single Tracking space used for Space alignment. */
USTRUCT(BlueprintType)
struct FTrackingAlignmentActors
{
	GENERATED_BODY()

	/** Tracked source actor in the tracking space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracking Alignment")
	TSoftObjectPtr<AActor> SourceActor;

	/** Tracked origin actor in the tracking space. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracking Alignment")
	TSoftObjectPtr<AActor> OriginActor;
};
