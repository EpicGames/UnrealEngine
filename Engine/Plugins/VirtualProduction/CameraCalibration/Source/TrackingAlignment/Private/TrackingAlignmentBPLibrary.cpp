// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackingAlignmentBPLibrary.h"
#include "GameFramework/Actor.h"
#include "TrackingAlignmentActors.h"
#include "TrackingAlignmentCalibrationProfile.h"
#include "OpenCVHelper.h"

#if WITH_OPENCV
// IWYU pragma: begin_keep
#include "PreOpenCVHeaders.h"
#include <opencv2/calib3d.hpp>
#include "PostOpenCVHeaders.h"
// IWYU pragma: end_keep
#endif

#define LOCTEXT_NAMESPACE "TrackingAlignmentBPLibrary"

namespace UE::TrackingAlignmentFunctionLibrary
{

#if WITH_OPENCV

static int32 HandEyeCalibrationSolveMethod = static_cast<int32>(cv::HandEyeCalibrationMethod::CALIB_HAND_EYE_PARK);

void ExtractOpenCVTranslationAndRotation(const FTransform& InTransform, cv::Mat& OutRot, cv::Mat& OutTra)
{
	// Convert the input FTransform to OpenCV's coordinate system
	FTransform CvTransform = InTransform;
	FOpenCVHelper::ConvertUnrealToOpenCV(CvTransform);

	const FMatrix TransformationMatrix = CvTransform.ToMatrixNoScale();

	// Extract the translation vector from the transformation matrix
	OutTra = cv::Mat(3, 1, CV_64FC1);
	OutTra.at<double>(0) = TransformationMatrix.M[3][0];
	OutTra.at<double>(1) = TransformationMatrix.M[3][1];
	OutTra.at<double>(2) = TransformationMatrix.M[3][2];

	// Extract the rotation matrix from the transformation matrix
	OutRot = cv::Mat(3, 3, CV_64FC1);
	for (int32 Column = 0; Column < 3; ++Column)
	{
		const FVector ColumnVector = TransformationMatrix.GetColumn(Column);

		// Note that since Unreal matrices stores basis vectors in rows, we need to transpose them into the Cv column-vector rotation matrix.

		OutRot.at<double>(Column, 0) = ColumnVector.X;
		OutRot.at<double>(Column, 1) = ColumnVector.Y;
		OutRot.at<double>(Column, 2) = ColumnVector.Z;
	}
}

void ExtractUnrealTransform(const cv::Mat& InRotation, const cv::Mat& InTranslation, FTransform& OutTransform)
{
	FMatrix TransformationMatrix = FMatrix::Identity;

	// Add the rotation matrix to the transformation matrix
	// Note: We need to transpose when converting from OpenCV back to Unreal format due to their different basis vector convention.
	for (int32 Column = 0; Column < 3; ++Column)
	{
		TransformationMatrix.SetColumn(Column, FVector(InRotation.at<double>(Column, 0), InRotation.at<double>(Column, 1), InRotation.at<double>(Column, 2)));
	}

	// Add the translation vector to the transformation matrix
	TransformationMatrix.M[3][0] = InTranslation.at<double>(0);
	TransformationMatrix.M[3][1] = InTranslation.at<double>(1);
	TransformationMatrix.M[3][2] = InTranslation.at<double>(2);

	OutTransform.SetFromMatrix(TransformationMatrix);

	// Convert the output FTransform to UE's coordinate system
	FOpenCVHelper::ConvertOpenCVToUnreal(OutTransform);
}

bool GetSamplesAsEyeInHandMats(
	UTrackingAlignmentCalibrationProfile* InCalibrationProfile,
	std::vector<cv::Mat>& RGripper2Base,
	std::vector<cv::Mat>& TGripper2Base,
	std::vector<cv::Mat>& RTarget2Cam,
	std::vector<cv::Mat>& TTarget2Cam
)
{
	if (InCalibrationProfile == nullptr || InCalibrationProfile->Samples.Num() < UTrackingAlignmentFunctionLibrary::GetMinimumRequiredTrackerAligmentSampleCount())
	{
		return false;
	}

	const FTrackingAlignmentActors& TrackerAActors = InCalibrationProfile->TrackerAActors;
	const FTrackingAlignmentActors& TrackerBActors = InCalibrationProfile->TrackerBActors;

	for (const FTrackingAlignmentSample& ThisSample : InCalibrationProfile->Samples)
	{
		cv::Mat TrackerAOpenCVRotation    = cv::Mat(3, 3, CV_64FC1);
		cv::Mat TrackerAOpenCVTranslation = cv::Mat(3, 1, CV_64FC1);

		cv::Mat TrackerBOpenCVRotation    = cv::Mat(3, 3, CV_64FC1);
		cv::Mat TrackerBOpenCVTranslation = cv::Mat(3, 1, CV_64FC1);

		ExtractOpenCVTranslationAndRotation(ThisSample.TransformA, TrackerAOpenCVRotation, TrackerAOpenCVTranslation);
		ExtractOpenCVTranslationAndRotation(ThisSample.TransformB, TrackerBOpenCVRotation, TrackerBOpenCVTranslation);

		RGripper2Base.push_back(TrackerBOpenCVRotation);
		TGripper2Base.push_back(TrackerBOpenCVTranslation);
		RTarget2Cam.push_back(TrackerAOpenCVRotation);
		TTarget2Cam.push_back(TrackerAOpenCVTranslation);
	}

	return true;
}

FTransform GetCam2Base(UTrackingAlignmentCalibrationProfile* InCalibrationProfile)
{
	/*
	* U = Unreal/absolute coordinate system (3d scene origin).
	* b = OCELLUS coordinate system relative to U.
	* a = OPTITRACK coordinate system relative to U.
	* o = OCELLUS i-th position relative to Lo.
	* m = MARKERTREE i-th position relative to To.
	* Cam2Base = Rigid transform of MARKERTREE relative to OCELLUS.
	*/
	FTransform Cam2Base = FTransform::Identity;

	std::vector<cv::Mat> RGripper2Base;
	std::vector<cv::Mat> TGripper2Base;
	std::vector<cv::Mat> RTarget2Cam;
	std::vector<cv::Mat> TTarget2Cam;

	if (!GetSamplesAsEyeInHandMats(InCalibrationProfile, RGripper2Base, TGripper2Base, RTarget2Cam, TTarget2Cam))
	{
		return FTransform::Identity;
	}

	// Pass off to OpenCV to solve for X
	cv::Mat OutRot = cv::Mat::eye(3, 3, CV_64FC1);
	cv::Mat OutTra = cv::Mat::zeros(3, 1, CV_64FC1);

	const cv::HandEyeCalibrationMethod CalibrationMethod = static_cast<cv::HandEyeCalibrationMethod>(HandEyeCalibrationSolveMethod);

	// For eye-to-hand configuration, invert gripper transforms as base2gripper
	std::vector<cv::Mat> RBase2Gripper;
	std::vector<cv::Mat> TBase2Gripper;
	for (int32 SampleIdx = 0; SampleIdx < RGripper2Base.size(); ++SampleIdx)
	{
		cv::Mat InvertedR;
		cv::Mat InvertedT;
		cv::transpose(RGripper2Base[SampleIdx], InvertedR);
		InvertedT = -InvertedR * TGripper2Base[SampleIdx];   // Tinv = -(Rinv * T)

		RBase2Gripper.push_back(InvertedR);
		TBase2Gripper.push_back(InvertedT);
	}

	cv::calibrateHandEye(RBase2Gripper, TBase2Gripper, RTarget2Cam, TTarget2Cam, OutRot, OutTra, CalibrationMethod);

	ExtractUnrealTransform(OutRot, OutTra, Cam2Base);

	return Cam2Base;
}

#else // WITH_OPENCV

static int32 HandEyeCalibrationSolveMethod = 0;

#endif // WITH_OPENCV

} // namespace UE::TrackingAlignmentFunctionLibrary

static FAutoConsoleVariableRef CVarHandEyeCalibrationSolveMethod(
	TEXT("TrackingAlignment.SolveMethod"),
	UE::TrackingAlignmentFunctionLibrary::HandEyeCalibrationSolveMethod,
	TEXT("Set the OpenCV HandEye solve method to use when aligning tracking spaces.\n")
	TEXT("0 - TSAI. A New Technique for Fully Autonomous and Efficient 3D Robotics Hand / Eye Calibration @cite Tsai89.\n")
	TEXT("1 - PARK. Robot Sensor Calibration: Solving AX = XB on the Euclidean Group @cite Park94.\n")
	TEXT("2 - HORAUD. Hand-eye Calibration @cite Horaud95.\n")
	TEXT("3 - ANDREFF. On-line Hand-Eye Calibration @cite Andreff99.\n")
	TEXT("4 - DANIILIDIS. Hand-Eye Calibration Using Dual Quaternions @cite Daniilidis98.")
);

int32 UTrackingAlignmentFunctionLibrary::GetMinimumRequiredTrackerAligmentSampleCount()
{
	return 5;
}

FTransform UTrackingAlignmentFunctionLibrary::GetAlignedTrackerBOrigin(UTrackingAlignmentCalibrationProfile* InCalibrationProfile)
{
	if (InCalibrationProfile == nullptr)
	{
		return FTransform::Identity;
	}
	
	FTransform Cam2Base = FTransform::Identity;

#if WITH_OPENCV
	Cam2Base = UE::TrackingAlignmentFunctionLibrary::GetCam2Base(InCalibrationProfile);
	if (InCalibrationProfile->TrackerAActors.OriginActor.IsNull())
	{
		return Cam2Base.Inverse();
	}
#endif // WITH_OPENCV

	return Cam2Base.Inverse() * InCalibrationProfile->TrackerAActors.OriginActor->GetTransform();
}

bool UTrackingAlignmentFunctionLibrary::FindAndUpdateOriginActor(FTrackingAlignmentActors& InTrackingActors, AActor*& OutNewParentActor)
{
	if (InTrackingActors.SourceActor.IsNull())
	{
		return false;
	}

	AActor* SourceActorReal = InTrackingActors.SourceActor.LoadSynchronous();

	if (SourceActorReal == nullptr)
	{
		return false;
	}

	if (AActor* AttachParentActor = SourceActorReal->GetAttachParentActor())
	{
		InTrackingActors.OriginActor = AttachParentActor;
		OutNewParentActor = AttachParentActor;
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
