// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/Transform.h"

#include "TrackingAlignmentBPLibrary.generated.h"

struct FTrackingAlignmentActor;
class UTrackingAlignmentCalibrationProfile;

/** Blueprint functions to caluclate tracking space alignment. */
UCLASS(MinimalAPI)
class UTrackingAlignmentFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Gets the minimum number of require tracker alignment samples to be able to calculate a Cam2Base value. */
	UFUNCTION(BlueprintPure, Category = "Tracking Alignment")
	static int32 GetMinimumRequiredTrackerAligmentSampleCount();

	/** 
	* Given a UTrackingAlignmentCalibrationProfile, returns the calculated alignment origin for TrackerB space to Tracker A space.
	* Uses OpenCV to solve the space alignment using Eye-To-Hand calibration with the given calibration method.
	*/
	UFUNCTION(BlueprintCallable, Category = "Tracking Alignment")
	static FTransform GetAlignedTrackerBOrigin(UTrackingAlignmentCalibrationProfile* InCalibrationProfile);

	/** Given an actor being tracked, find and update its origin actor to the direct parent actor. */
	UFUNCTION(BlueprintCallable, Category = "Tracking Alignment")
	static bool FindAndUpdateOriginActor(UPARAM(ref) FTrackingAlignmentActors& InTrackingActors, AActor*& OutNewParentActor);
};

