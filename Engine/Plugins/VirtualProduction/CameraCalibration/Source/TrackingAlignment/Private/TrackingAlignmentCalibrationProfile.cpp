// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackingAlignmentCalibrationProfile.h"
#include "GameFramework/Actor.h"

bool UTrackingAlignmentCalibrationProfile::CaptureSample(FTrackingAlignmentSample& NewSample)
{
	if (TrackerAActors.SourceActor.IsNull() || TrackerBActors.SourceActor.IsNull())
	{
		return false;
	}

	auto GetCapturedTransform = [](const FTrackingAlignmentActors& InActors) -> FTransform
		{
			AActor* SourceActor = InActors.SourceActor.LoadSynchronous();

			FTransform OutTransform = SourceActor->GetTransform();

			if (!InActors.OriginActor.IsNull())
			{
				const AActor* OriginActor = InActors.OriginActor.LoadSynchronous();
				OutTransform = OutTransform.GetRelativeTransform(OriginActor->GetTransform());
			}

			return OutTransform;
		};

	Modify();
	NewSample.TransformA = GetCapturedTransform(TrackerAActors);
	NewSample.TransformB = GetCapturedTransform(TrackerBActors);

	Samples.Add(NewSample);

	return true;
}

bool UTrackingAlignmentCalibrationProfile::RemoveSample(int32 AtIndex)
{
	if (AtIndex >= 0 && AtIndex < Samples.Num())
	{
		Modify();
		Samples.RemoveAt(AtIndex);
		return true;
	}

	return false;
}

void UTrackingAlignmentCalibrationProfile::ClearSamples()
{
	Modify();
	Samples.Empty();
}
