// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "TrackingAlignmentActors.h"
#include "TrackingAlignmentSample.h"

#include "TrackingAlignmentCalibrationProfile.generated.h"

/** Calibration profile holding actors and samples for aligning two tracking spaces. */
UCLASS(MinimalAPI, BlueprintType, AutoExpandCategories = ("Tracking Alignment"))
class UTrackingAlignmentCalibrationProfile : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	/** Tracking space A tracked source and origin actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracking Alignment")
	FTrackingAlignmentActors TrackerAActors;

	/** Tracking space B tracked source and origin actor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Tracking Alignment")
	FTrackingAlignmentActors TrackerBActors;

	/** Captured samples from both Tracking space A and B. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Tracking Alignment")
	TArray<FTrackingAlignmentSample> Samples;

	/** Capture and add a new sample for TrackerAActors and TrackerBActors, and write it into the out parameter NewSample. */
	UFUNCTION(BlueprintCallable, Category = "Tracking Alignment")
	bool CaptureSample(FTrackingAlignmentSample& NewSample);

	/** Remove sample at the given index. */
	UFUNCTION(BlueprintCallable, Category = "Tracking Alignment")
	bool RemoveSample(int32 AtIndex);

	/** Clear all current samples. */
	UFUNCTION(BlueprintCallable, Category = "Tracking Alignment")
	void ClearSamples();
};
