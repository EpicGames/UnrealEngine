// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Transform.h"

#include "TrackingAlignmentSample.generated.h"

/** Single sample as part of a Tracking Alignment calibration, holding transform samples from both Tracking space A and B. */
USTRUCT(BlueprintType)
struct FTrackingAlignmentSample
{
	GENERATED_BODY()

	/** Transform sample from Tracking Space A. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Tracking Alignment")
	FTransform TransformA;

	/** Transform sample from Tracking Space B. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Tracking Alignment")
	FTransform TransformB;
};
