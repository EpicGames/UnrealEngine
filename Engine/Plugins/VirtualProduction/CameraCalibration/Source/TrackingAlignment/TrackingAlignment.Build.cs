// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TrackingAlignment : ModuleRules
{
	public TrackingAlignment(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"OpenCVHelper",
				"OpenCV",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
			}
		);
	}
}
