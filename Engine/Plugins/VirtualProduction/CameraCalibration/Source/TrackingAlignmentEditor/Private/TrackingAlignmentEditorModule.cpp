// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FTrackingAlignmentEditorModule"

class FTrackingAlignmentEditorModule : public IModuleInterface
{
public:
	inline static const FName TrackerAlignmentPanelTabName = FName("TrackerAlignment");

/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FTrackingAlignmentEditorModule::RegisterMenus));

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TrackerAlignmentPanelTabName, FOnSpawnTab::CreateRaw(this, &FTrackingAlignmentEditorModule::OnSpawnTrackerAlignment))
			.SetDisplayName(NSLOCTEXT("TrackerAlignment", "TrackerAlignmentTabTitle", "Tracker Alignment"))
			.SetTooltipText(NSLOCTEXT("TrackerAlignment", "TrackerAlignmentTooltipText", "Open the Tracker Alignment tool"))
			.SetMenuType(ETabSpawnerMenuType::Hidden);
	}

	virtual void ShutdownModule() override
	{
		//Clean up nomad tab spawner
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TrackerAlignmentPanelTabName);
	}

private:
	TSharedRef<class SDockTab> OnSpawnTrackerAlignment(const class FSpawnTabArgs& SpawnTabArgs)
	{
		UEditorUtilityWidgetBlueprint* TrackerAlignmentEUW = LoadObject<UEditorUtilityWidgetBlueprint>(NULL, TEXT("/CameraCalibration/Widgets/EUW_TrackerAlignment.EUW_TrackerAlignment"), NULL, LOAD_None, NULL);
		return TrackerAlignmentEUW->SpawnEditorUITab(SpawnTabArgs);
	}

	void RegisterMenus()
	{
		FToolMenuOwnerScoped OwnerScoped(this);

		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window.VirtualProduction");
			{
				FToolMenuSection& Section = Menu->FindOrAddSection("VirtualProduction");
				Section.AddMenuEntry(
					FName("TrackerAlignment"),
					LOCTEXT("TrackingAlignmentMenuSpawnTrackerAlignment", "Tracker Alignment"),
					LOCTEXT("TrackingAlignmentMenuSpawnTrackerAlignmentTooltip", "Open the Tracker Alignment tool"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Transform"),
					FUIAction(FExecuteAction::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(TrackerAlignmentPanelTabName); }))
				);
			}
		}
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FTrackingAlignmentEditorModule, TrackingAlignmentEditor);
