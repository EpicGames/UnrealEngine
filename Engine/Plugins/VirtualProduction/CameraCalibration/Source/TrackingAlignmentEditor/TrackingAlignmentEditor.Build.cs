// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TrackingAlignmentEditor : ModuleRules
{
	public TrackingAlignmentEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Blutility",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TrackingAlignment",
				"UMGEditor"
			}
		);
	}
}
